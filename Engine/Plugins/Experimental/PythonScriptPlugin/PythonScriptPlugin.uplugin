{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Python Editor Script Plugin",
	"Description": "Python integration for the Unreal Editor.",
	"Category": "Scripting",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/en-US/Engine/Editor/ScriptingAndAutomation/Python/index.html",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules": [
		{
			"Name": "PythonScriptPluginPreload",
			"Type": "Runtime",
			"LoadingPhase": "EarliestPossible"
		},
		{
			"Name": "PythonScriptPlugin",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "ContentBrowserFileDataSource",
			"Enabled": true,
			"ProgramAllowList": [ "LiveLinkHub" ],
			"TargetAllowList": [
				"Editor",
				"Program"
			]
		}
	]
}
