// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"

class FProperty;
class UClass;
class UFunction;


#if WITH_PYTHON


//
// PyGenUtilAccessor
//
// Provides some internal access to functionality in PyGenUtil. Not available for non-Engine use. See corresponding functions in PyGenUtil.
//


namespace PyGenUtilAccessor
{
	PYTHONSCRIPTPLUGIN_API void ExtractFunctionParamsAsProperties(const UFunction* InFunc, TArray<const FProperty*>& OutInputParams, TArray<const FProperty*>& OutOutputParams);
	
	PYTHONSCRIPTPLUGIN_API FString GetClassPythonName(const UClass* InClass);
	PYTHONSCRIPTPLUGIN_API FString GetFunctionPythonName(const UFunction* InFunc);
	PYTHONSCRIPTPLUGIN_API FString GetPropertyPythonName(const FProperty* InProp);
	PYTHONSCRIPTPLUGIN_API FString GetPropertyPythonType(const FProperty* InProp);
	
	PYTHONSCRIPTPLUGIN_API bool ShouldExportClass(const UClass* InClass);
	PYTHONSCRIPTPLUGIN_API bool ShouldExportFunction(const UFunction* InFunc);
	PYTHONSCRIPTPLUGIN_API bool ShouldExportProperty(const FProperty* InProp);
	PYTHONSCRIPTPLUGIN_API bool ShouldExportEditorOnlyProperty(const FProperty* InProp);
};


#endif	// WITH_PYTHON
