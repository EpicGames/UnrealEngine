// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorPythonExecuter.h"

#if WITH_EDITOR

#include "AssetRegistry/IAssetRegistry.h"
#include "Editor.h"
#include "EditorPythonScriptingLibrary.h"
#include "IPythonScriptPlugin.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/CommandLine.h"
#include "TickableEditorObject.h"

DEFINE_LOG_CATEGORY_STATIC(LogEditorPythonExecuter, Log, All);

#define LOCTEXT_NAMESPACE "EditorPythonRunner"

namespace InternalEditorPythonRunner
{
	class FExecuterTickable;

	TUniquePtr<FAsyncTaskNotification> Notification;
	TUniquePtr<FExecuterTickable> Executer;

	void CreateNotification(const FString& ScriptAndArgs)
	{
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.TitleText = LOCTEXT("ExecutingPythonScript", "Executing Python Script...");
		NotificationConfig.ProgressText = FText::AsCultureInvariant(ScriptAndArgs);
		NotificationConfig.bCanCancel = true;
		Notification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
	}

	void DestroyNotification()
	{
		if (Notification)
		{
			Notification->SetComplete(true);
			Notification.Reset();
		}
	}

	/*
	 * Tick until we are ready.
	 * We could also listen to events like FAssetRegistryModule::FileLoadedEvent but Python script can possibly be executed on multiple frames and we need to wait until it is completed to return.
	 * And we can't close the editor on the same frame that we execute the Python script because a full tick needs to happen first.
	 */
	class FExecuterTickable : FTickableEditorObject
	{
	public:
		FExecuterTickable(FString&& InScriptAndArgs, bool InErrorsAreFatal)
			: ScriptAndArgs(MoveTemp(InScriptAndArgs))
			, bErrorsAreFatal(InErrorsAreFatal)
		{
		}

		virtual void Tick(float DeltaTime) override
		{
			if (IsExitRequested())
			{
				return;
			}

			if (Notification && Notification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel)
			{
				RequestExit();
				return;
			}

			if (bIsRunning)
			{
				if (!UEditorPythonScriptingLibrary::GetKeepPythonScriptAlive())
				{
					RequestExit();
					return;
				}
			}

			// if we are here the editor is ready.
			if (!bIsRunning && GWorld && GEngine && GEditor && DeltaTime > 0 && GLog)
			{
				if (!ScriptAndArgs.IsEmpty())
				{
					// check if the AssetRegistryModule is ready
					if (!IAssetRegistry::GetChecked().IsLoadingAssets())
					{
						bIsRunning = true;

						FPythonCommandEx PythonCommand;
						PythonCommand.Flags |= EPythonCommandFlags::Unattended; // Prevent all dialog modal from showing up
						PythonCommand.Command = ScriptAndArgs;
						if (!IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand))
						{
							if (bErrorsAreFatal)
							{
								UE_LOGF(LogEditorPythonExecuter, Fatal, "Python script executed with errors");
							}
							else
							{
								UE_LOGF(LogEditorPythonExecuter, Error, "Python script executed with errors");
							}
						}
					}
				}
				else
				{
					RequestExit();
				}
			}
		}

		virtual TStatId GetStatId() const override
		{
			return TStatId();
		}

	private:
		void RequestExit()
		{
			bExitRequested = true;
			DestroyNotification();
			if (GEngine)
			{
				GEngine->HandleDeferCommand(TEXT("QUIT_EDITOR"), *GLog); // Defer close the editor
			}
		}

		bool IsExitRequested() const
		{
			return bExitRequested || IsEngineExitRequested();
		}

		FString ScriptAndArgs;
		bool bErrorsAreFatal = false;

		bool bIsRunning = false;
		bool bExitRequested = false;
	};
}

void FEditorPythonExecuter::OnStartupModule()
{
	const TCHAR* Match = TEXT("-ExecutePythonScript=");
	const TCHAR* Found = FCString::Strifind(FCommandLine::Get(), Match, true);
	if (!Found)
	{
		return;
	}

	// The code needs to manage spaces and quotes. The scripts pathname and the script arguments are passed to the 'PY'
	// command which are passed to Python plugin for execution. Below shows how to quotes the script pathname
	// and the scripts arguments when they contain spaces.
	// +-----------------------------------------------------------------------------------+---------------------------------------------------+
	// | Command Line Parameters                                                           | Resulting "PY" command                            |
	// +-----------------------------------------------------------------------------------+---------------------------------------------------+
	// | -ExecutePythonScript=script.py                                                    | PY script.py                                      |
	// | -ExecutePythonScript="script.py"                                                  | PY script.py                                      |
	// | -ExecutePythonScript="C:/With Space/with space.py"                                | PY C:/With Space/with space.py                    |
	// | -ExecutePythonScript="script.py arg1"                                             | PY script.py arg1                                 |
	// | -ExecutePythonScript="script.py arg1 \\\"args with space\\\""                     | PY script.py arg1 "args with space"               |
	// | -ExecutePythonScript="C:\With Space\with space.py \\\"arg with space\\\""         | PY C:/With Space/with space.py "arg with space"   | NOTE: The Python plugin parses up the ".py" and manages the spaces in the pathname.
	// | -ExecutePythonScript="\\\"C:/With Space/with space.py\\\" \\\"arg with space\\\"" | PY "C:/With Space/with space.py" "arg with space" |
	// +-----------------------------------------------------------------------------------+---------------------------------------------------+

	int32 MatchLen = FCString::Strlen(Match);
	const TCHAR* ScriptAndArgsBegin = Found + MatchLen;
	FString ScriptAndArgs;

	// If the value passed with '-ExecutePythonScript=' is not quoted, use spaces as delimiter.
	if (*ScriptAndArgsBegin != TEXT('"'))
	{
		FParse::Token(ScriptAndArgsBegin, ScriptAndArgs, false);
	}
	else // The value is quoted.
	{
		FParse::QuotedString(ScriptAndArgsBegin, ScriptAndArgs);
	}

	if (!ScriptAndArgs.IsEmpty())
	{
		if (!GIsEditor)
		{
			UE_LOGF(LogEditorPythonExecuter, Error, "-ExecutePythonScript cannot be used outside of the editor.");
		}
		else if (IsRunningCommandlet())
		{
			UE_LOGF(LogEditorPythonExecuter, Error, "-ExecutePythonScript cannot be used by a commandlet. Use -run=PythonScript instead?");
		}
		else
		{
			// If -ExecutePythonScript has been specified then we can assume the user wanted Python support enabled
			IPythonScriptPlugin::Get()->ForceEnablePythonAtRuntime();

			IPythonScriptPlugin::Get()->RegisterOnPythonConfigured(FSimpleDelegate::CreateLambda([ScriptAndArgs = MoveTemp(ScriptAndArgs), bScriptErrorsAreFatal = FParse::Param(FCommandLine::Get(), TEXT("ScriptErrorsAreFatal"))]() mutable
			{
				if (IPythonScriptPlugin::Get()->IsPythonAvailable())
				{
					IPythonScriptPlugin::Get()->RegisterOnPythonInitialized(FSimpleDelegate::CreateLambda([ScriptAndArgs = MoveTemp(ScriptAndArgs), bScriptErrorsAreFatal]() mutable
					{
						InternalEditorPythonRunner::CreateNotification(ScriptAndArgs);
						InternalEditorPythonRunner::Executer = MakeUnique<InternalEditorPythonRunner::FExecuterTickable>(MoveTemp(ScriptAndArgs), bScriptErrorsAreFatal);
					}));
				}
				else
				{
					UE_LOGF(LogEditorPythonExecuter, Error, "-ExecutePythonScript cannot be used when Python support is disabled.");
				}
			}));
		}
	}
}

void FEditorPythonExecuter::OnShutdownModule()
{
	InternalEditorPythonRunner::DestroyNotification();
	InternalEditorPythonRunner::Executer.Reset();
}

#undef LOCTEXT_NAMESPACE

#endif	// WITH_EDITOR
