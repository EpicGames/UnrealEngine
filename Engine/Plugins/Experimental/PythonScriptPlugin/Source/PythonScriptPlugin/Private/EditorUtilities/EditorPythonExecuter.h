// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

/**
 * Class that will execute a Python script once the editor is ready.
 * This is not a commandlet because the editor feature needs to be enabled and running. The script may use the editor functionalities.
 * This could not be replaced with "ExecCmds=". Again need to wait for some editor functionalities to be completely loaded.
 * ie. EnterpriseTest ExecutePythonScript="c:\my_script.py"
 */
class FEditorPythonExecuter
{
public:
	static void OnStartupModule();
	static void OnShutdownModule();
};

#endif	// WITH_EDITOR
