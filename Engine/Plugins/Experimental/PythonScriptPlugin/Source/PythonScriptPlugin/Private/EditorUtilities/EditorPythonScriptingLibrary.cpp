// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorPythonScriptingLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorPythonScriptingLibrary)

#define LOCTEXT_NAMESPACE "EditorPythonScriptingLibrary"

bool UEditorPythonScriptingLibrary::bKeepPythonScriptAlive = false;


void UEditorPythonScriptingLibrary::SetKeepPythonScriptAlive(const bool bNewKeepAlive)
{
	bKeepPythonScriptAlive = bNewKeepAlive;
}

bool UEditorPythonScriptingLibrary::GetKeepPythonScriptAlive()
{
	return bKeepPythonScriptAlive;
}


#undef LOCTEXT_NAMESPACE // "EditorPythonScriptingLibrary"
