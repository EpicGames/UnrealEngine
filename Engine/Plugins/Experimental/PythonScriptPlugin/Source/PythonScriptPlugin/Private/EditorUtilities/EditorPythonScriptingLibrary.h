// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "EditorPythonScriptingLibrary.generated.h"


/**
 * Utility class for Python scripting functionality.
 */
UCLASS(meta=(ScriptName="EditorPythonScripting"))
class UEditorPythonScriptingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

private:
	static bool bKeepPythonScriptAlive;

public:
	/**
	 * Sets the bKeepPythonScriptAlive flag.
	 * 
	 * If this is false (default), it will close the editor during the next tick (when executing a Python script in the editor-environment using the UnrealEditor-Cmd commandline tool).
	 * If this is true, it will not close the editor by itself, and you will have to close it manually, either by setting this value to false again, or by calling a function like unreal.SystemLibrary.quit_editor(). 
	 * 
	 * @param bNewKeepAlive The new value of the bKeepPythonScriptAlive flag.
	 * @return The result of the users decision, true=Ok, false=Cancel, or false if running in unattended mode.
	*/
	UFUNCTION(BlueprintCallable, DisplayName = "Set Keep Python Script Alive", Category = "Editor Scripting | Python")
	static void SetKeepPythonScriptAlive(const bool bNewKeepAlive);

	/**
	 * Returns the value of the bKeepPythonScriptAlive flag.
	 * 
	 * If this is false (default), it will close the editor during the next tick (when executing a Python script in the editor-environment using the UnrealEditor-Cmd commandline tool).
	 * If this is true, it will not close the editor by itself, and you will have to close it manually, either by setting this value to false again, or by calling a function like unreal.SystemLibrary.quit_editor(). 
	 * 
	 * @return The current value of the bKeepPythonScriptAlive flag.
	*/
	UFUNCTION(BlueprintPure, DisplayName = "Get Keep Python Script Alive", Category = "Editor Scripting | Python")
	static bool GetKeepPythonScriptAlive();
};
