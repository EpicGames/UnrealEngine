// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Stats/Stats.h"

#if WITH_PYTHON

THIRD_PARTY_INCLUDES_START
PRAGMA_DISABLE_REGISTER_WARNINGS
#include "Python.h"
#include "structmember.h"
PRAGMA_ENABLE_REGISTER_WARNINGS
THIRD_PARTY_INCLUDES_END

DECLARE_STATS_GROUP(TEXT("Python"), STATGROUP_Python, STATCAT_Advanced);

#endif	// WITH_PYTHON
