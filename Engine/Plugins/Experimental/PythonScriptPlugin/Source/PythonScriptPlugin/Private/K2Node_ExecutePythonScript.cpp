// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_ExecutePythonScript.h"
#include "PyGenUtil.h"
#include "PythonScriptLibrary.h"

#include "UObject/Package.h"
#include "KismetCompiler.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_MakeArray.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ExecutePythonScript)

#define LOCTEXT_NAMESPACE "K2Node_ExecutePythonScript"

namespace ExecutePythonScriptUtil
{

const FName PythonScriptPinName = "PythonScript";
const FName PythonInputsPinName = "PythonInputs";
const FName PythonOutputsPinName = "PythonOutputs";

FString PythonizePinName(const FName InPinName)
{
#if WITH_PYTHON
	return PyGenUtil::PythonizePropertyName(InPinName.ToString(), PyGenUtil::EPythonizeNameCase::Lower);
#else
	return InPinName.ToString();
#endif
}

FString PythonizePinName(const UEdGraphPin* InPin)
{
	return PythonizePinName(InPin->PinName);
}

}

UK2Node_ExecutePythonScript::UK2Node_ExecutePythonScript()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UPythonScriptLibrary, ExecutePythonScript), UPythonScriptLibrary::StaticClass());
}

void UK2Node_ExecutePythonScript::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Hide the argument name pins
	// Note: Have to do this manually as the "HidePin" meta-data only lets you hide a single pin :(
	UEdGraphPin* PythonInputsPin = FindPinChecked(ExecutePythonScriptUtil::PythonInputsPinName, EGPD_Input);
	PythonInputsPin->bHidden = true;
	UEdGraphPin* PythonOutputsPin = FindPinChecked(ExecutePythonScriptUtil::PythonOutputsPinName, EGPD_Input);
	PythonOutputsPin->bHidden = true;

	// Give the Python script pin a default value
	UEdGraphPin* PythonScriptPin = FindPinChecked(ExecutePythonScriptUtil::PythonScriptPinName, EGPD_Input);
	PythonScriptPin->AutogeneratedDefaultValue = TEXT("print('Hello World!')");
	PythonScriptPin->DefaultValue = PythonScriptPin->AutogeneratedDefaultValue;

	// Add user-defined pins
	auto AddUserDefinedPins = [this](const TArray<FName>& InPinNames, EEdGraphPinDirection InPinDirection)
	{
		for (const FName& PinName : InPinNames)
		{
			UEdGraphPin* ArgPin = CreatePin(InPinDirection, UEdGraphSchema_K2::PC_Wildcard, PinName);
			ArgPin->PinFriendlyName = FText::AsCultureInvariant(ExecutePythonScriptUtil::PythonizePinName(ArgPin));
		}
	};
	AddUserDefinedPins(Inputs, EGPD_Input);
	AddUserDefinedPins(Outputs, EGPD_Output);
}

void UK2Node_ExecutePythonScript::PostReconstructNode()
{
	Super::PostReconstructNode();

	auto SynchronizeUserDefinedPins = [this](const TArray<FName>& InPinNames, EEdGraphPinDirection InPinDirection)
	{
		for (const FName& PinName : InPinNames)
		{
			UEdGraphPin* ArgPin = FindArgumentPinChecked(PinName, InPinDirection);
			SynchronizeArgumentPinTypeImpl(ArgPin);
		}
	};
	SynchronizeUserDefinedPins(Inputs, EGPD_Input);
	SynchronizeUserDefinedPins(Outputs, EGPD_Output);
}

void UK2Node_ExecutePythonScript::SynchronizeArgumentPinType(UEdGraphPin* Pin)
{
	auto SynchronizeUserDefinedPin = [this, Pin](const TArray<FName>& InPinNames, EEdGraphPinDirection InPinDirection)
	{
		if (Pin->Direction == InPinDirection && InPinNames.Contains(Pin->PinName))
		{
			// Try and find the argument pin and make sure we get the same result as the pin we were asked to update
			// If not we may have a duplicate pin name with another non-argument pin
			UEdGraphPin* ArgPin = FindArgumentPinChecked(Pin->PinName, InPinDirection);
			if (ArgPin == Pin)
			{
				SynchronizeArgumentPinTypeImpl(Pin);
			}
		}
	};
	SynchronizeUserDefinedPin(Inputs, EGPD_Input);
	SynchronizeUserDefinedPin(Outputs, EGPD_Output);
}

void UK2Node_ExecutePythonScript::SynchronizeArgumentPinTypeImpl(UEdGraphPin* Pin)
{
	FEdGraphPinType NewPinType;
	if (Pin->LinkedTo.Num() > 0)
	{
		NewPinType = Pin->LinkedTo[0]->PinType;
	}
	else
	{
		NewPinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	}

	if (Pin->PinType != NewPinType)
	{
		Pin->PinType = NewPinType;

		GetGraph()->NotifyNodeChanged(this);

		UBlueprint* Blueprint = GetBlueprint();
		if (!Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

UEdGraphPin* UK2Node_ExecutePythonScript::FindArgumentPin(const FName PinName, EEdGraphPinDirection PinDirection)
{
	for (int32 PinIndex = Pins.Num() - 1; PinIndex >= 0; --PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		if ((PinDirection == EGPD_MAX || PinDirection == Pin->Direction) && Pin->PinName == PinName)
		{
			return Pin;
		}
	}
	return nullptr;
}

UEdGraphPin* UK2Node_ExecutePythonScript::FindArgumentPinChecked(const FName PinName, EEdGraphPinDirection PinDirection)
{
	UEdGraphPin* Pin = FindArgumentPin(PinName, PinDirection);
	check(Pin);
	return Pin;
}

void UK2Node_ExecutePythonScript::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property  ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_ExecutePythonScript, Inputs) || PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_ExecutePythonScript, Outputs))
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node_ExecutePythonScript::PinConnectionListChanged(UEdGraphPin* Pin)
{
	// Potentially update an argument pin type
	SynchronizeArgumentPinType(Pin);
}

void UK2Node_ExecutePythonScript::PinTypeChanged(UEdGraphPin* Pin)
{
	// Potentially update an argument pin type
	SynchronizeArgumentPinType(Pin);

	Super::PinTypeChanged(Pin);
}

FText UK2Node_ExecutePythonScript::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	if ((Inputs.Contains(Pin->PinName) || Outputs.Contains(Pin->PinName)) && !Pin->PinFriendlyName.IsEmpty())
	{
		return Pin->PinFriendlyName;
	}
	return Super::GetPinDisplayName(Pin);
}

void UK2Node_ExecutePythonScript::EarlyValidation(FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	TSet<FString> AllPinNames;
	auto ValidatePinArray = [this, &AllPinNames, &MessageLog](const TArray<FName>& InPinNames)
	{
		for (const FName& PinName : InPinNames)
		{
			const FString PythonizedPinName = ExecutePythonScriptUtil::PythonizePinName(PinName);

			if (PinName == UEdGraphSchema_K2::PN_Execute ||
				PinName == UEdGraphSchema_K2::PN_Then ||
				PinName == UEdGraphSchema_K2::PN_ReturnValue ||
				PinName == ExecutePythonScriptUtil::PythonScriptPinName ||
				PinName == ExecutePythonScriptUtil::PythonInputsPinName ||
				PinName == ExecutePythonScriptUtil::PythonOutputsPinName
				)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("InvalidPinName_RestrictedName", "Pin '{0}' ({1}) on @@ is using a restricted name."), FText::AsCultureInvariant(PinName.ToString()), FText::AsCultureInvariant(PythonizedPinName)).ToString(), this);
			}

			if (PythonizedPinName.Len() == 0)
			{
				MessageLog.Error(*LOCTEXT("InvalidPinName_EmptyName", "Empty pin name found on @@").ToString(), this);
			}
			else
			{
#if WITH_PYTHON
				FText NameValidationError;
				if (!PyGenUtil::IsValidName(PythonizedPinName, &NameValidationError))
				{
					MessageLog.Error(*FText::Format(LOCTEXT("InvalidPinName_GenValidationError", "Pin '{0}' ({1}) on @@ failed name validation: {2}."), FText::AsCultureInvariant(PinName.ToString()), FText::AsCultureInvariant(PythonizedPinName), NameValidationError).ToString(), this);
				}
#endif // WITH_PYTHON

				bool bAlreadyUsed = false;
				AllPinNames.Add(PythonizedPinName, &bAlreadyUsed);

				if (bAlreadyUsed)
				{
					MessageLog.Error(*FText::Format(LOCTEXT("InvalidPinName_DuplicateName", "Pin '{0}' ({1}) on @@ has the same name as another pin when exposed to Python."), FText::AsCultureInvariant(PinName.ToString()), FText::AsCultureInvariant(PythonizedPinName)).ToString(), this);
				}
			}
		}
	};

	ValidatePinArray(Inputs);
	ValidatePinArray(Outputs);
}

void UK2Node_ExecutePythonScript::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	auto MakeArgumentNamesNode = [this, &CompilerContext, SourceGraph](const TArray<FName>& ArgNames, UEdGraphPin* TargetPin)
	{
		// Create a "Make Array" node to compile the list of arguments into an array for the Format function being called
		UK2Node_MakeArray* MakeArrayNode = CompilerContext.SpawnIntermediateNode<UK2Node_MakeArray>(this, SourceGraph);
		MakeArrayNode->NumInputs = ArgNames.Num();
		MakeArrayNode->AllocateDefaultPins();

		// Connect the output of the "Make Array" pin to the destination input pin
		// PinConnectionListChanged will set the "Make Array" node's type, only works if one pin is connected
		UEdGraphPin* ArrayOut = MakeArrayNode->GetOutputPin();
		ArrayOut->MakeLinkTo(TargetPin);
		MakeArrayNode->PinConnectionListChanged(ArrayOut);

		// For each argument we need to make a literal string node
		int32 ArgIndex = 0;
		for (const FName& ArgName : ArgNames)
		{
			static const FName MakeLiteralStringFunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, MakeLiteralString);

			// Create the "Make Literal String" node
			UK2Node_CallFunction* MakeLiteralStringNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			MakeLiteralStringNode->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(MakeLiteralStringFunctionName));
			MakeLiteralStringNode->AllocateDefaultPins();

			// Set the literal value to the Pythonized argument name
			UEdGraphPin* MakeLiteralIntValuePin = MakeLiteralStringNode->FindPinChecked(TEXT("Value"), EGPD_Input);
			MakeLiteralIntValuePin->DefaultValue = ExecutePythonScriptUtil::PythonizePinName(ArgName);

			// Find the input pin on the "Make Array" node by index and link it to the literal string
			UEdGraphPin* ArrayIn = MakeArrayNode->FindPinChecked(FString::Printf(TEXT("[%d]"), ArgIndex++));
			MakeLiteralStringNode->GetReturnValuePin()->MakeLinkTo(ArrayIn);
		}

		return MakeArrayNode;
	};

	// Create and connect the argument name nodes
	MakeArgumentNamesNode(Inputs, FindPinChecked(ExecutePythonScriptUtil::PythonInputsPinName, EGPD_Input));
	MakeArgumentNamesNode(Outputs, FindPinChecked(ExecutePythonScriptUtil::PythonOutputsPinName, EGPD_Input));

	Super::ExpandNode(CompilerContext, SourceGraph);
}

bool UK2Node_ExecutePythonScript::CanPasteHere(const UEdGraph* TargetGraph) const
{
	bool bCanPaste = Super::CanPasteHere(TargetGraph);
	if (bCanPaste)
	{
		bCanPaste &= FBlueprintEditorUtils::IsEditorUtilityBlueprint(FBlueprintEditorUtils::FindBlueprintForGraphChecked(TargetGraph));
	}
	return bCanPaste;
}

bool UK2Node_ExecutePythonScript::IsActionFilteredOut(const FBlueprintActionFilter& Filter)
{
	bool bIsFilteredOut = Super::IsActionFilteredOut(Filter);
	if (!bIsFilteredOut)
	{
		for (UEdGraph* TargetGraph : Filter.Context.Graphs)
		{
			bIsFilteredOut |= !CanPasteHere(TargetGraph);
		}
	}
	return bIsFilteredOut;
}

void UK2Node_ExecutePythonScript::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

#undef LOCTEXT_NAMESPACE

