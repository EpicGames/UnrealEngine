// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ExecutePythonScript.generated.h"

class UEdGraph;
class UEdGraphPin;
class FBlueprintActionDatabaseRegistrar;

UCLASS()
class UK2Node_ExecutePythonScript : public UK2Node_CallFunction
{
	GENERATED_BODY()
	
	UK2Node_ExecutePythonScript();
	
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual void PostReconstructNode() override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual bool IsActionFilteredOut(const class FBlueprintActionFilter& Filter) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	//~ End UK2Node Interface.

protected:
	//~ UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface

private:
	/** Synchronize the type of the given argument pin with the type its connected to, or reset it to a wildcard pin if there's no connection */
	void SynchronizeArgumentPinType(UEdGraphPin* Pin);
	void SynchronizeArgumentPinTypeImpl(UEdGraphPin* Pin);

	/** Try and find a named argument pin - this is the same as FindPin except it searches the array from the end as that's where the argument pins will be */
	UEdGraphPin* FindArgumentPin(const FName PinName, EEdGraphPinDirection PinDirection = EGPD_MAX);
	UEdGraphPin* FindArgumentPinChecked(const FName PinName, EEdGraphPinDirection PinDirection = EGPD_MAX);

	/** User-defined input pins */
	UPROPERTY(EditAnywhere, Category="Arguments")
	TArray<FName> Inputs;

	/** User-defined output pins */
	UPROPERTY(EditAnywhere, Category="Arguments")
	TArray<FName> Outputs;
};
