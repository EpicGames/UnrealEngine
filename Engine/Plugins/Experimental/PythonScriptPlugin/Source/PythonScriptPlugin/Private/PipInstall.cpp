// Copyright Epic Games, Inc. All Rights Reserved.

#include "PipInstall.h"
#include "PipRunnable.h"

#include "PyGIL.h"
#include "PyUtil.h"
#include "PythonScriptPluginSettings.h"
#include "Dom/JsonObject.h"
#include "HAL/PlatformFileManager.h"
#include "Async/Async.h"
#include "Async/Mutex.h"
#include "Async/ParallelFor.h"
#include "Misc/CommandLine.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "PipInstall"

IPipInstall& IPipInstall::Get()
{
	static FPipInstall Instance;
	return Instance;
}


bool FPipInstall::InitPipInstall()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::InitPipInstaller);

#if WITH_PYTHON
	// Init Pip installer for python dependencies (if any)
	if (IsInitialized())
	{
		return true;
	}
	
	CheckInvalidPipEnv();

	// Generate the input listing files of plugins with python dependencies and the listing of all requirements (installed or not)
	TArray<TSharedRef<IPlugin>> PythonPlugins;
	WritePluginsListing(PythonPlugins);

	for (const TSharedRef<IPlugin>& PyPlugin : PythonPlugins)
	{
		// TODO: Get site-package dir listing from PyUtil instead of duplicating
		// Remove leftover __pycache__ folders from plugins that use pip, but previously used packaged dependencies
		const FString LibDir = PyPlugin->GetContentDir() / TEXT("Python") / TEXT("Lib");
		CheckRemoveOrphanedPackages(LibDir / TEXT("site-packages"));
		CheckRemoveOrphanedPackages(LibDir / FPlatformMisc::GetUBTPlatform() / TEXT("site-packages"));
	}

	TArray<FString> ReqInLines;
	TArray<FString> ExtraUrls;
	const FString ReqsInFile = WritePluginDependencies(PythonPlugins, ReqInLines, ExtraUrls);

	ClearCachedInstallRequirements();
	if (ReqInLines.IsEmpty())
	{
		UE_LOGF(LogPython, Display, "No pip-enabled plugins with python dependencies found, skipping");
		// Remove outdated parsed dependency files if there's nothing to install
		RemoveParsedDependencyFiles();
		return true;
	}

	// Some dependencies may need installing
	FFeedbackContext* Context = GWarn;
	if (!SetupPipEnv(Context))
	{
		return false;
	}

	if (!CacheDetectInstallDeps(Context))
	{
		return false;
	}

	bInitialized.Store(true);
	return true;
#else
	return false;
#endif // WITH_PYTHON
}

bool FPipInstall::LaunchPipInstall(bool RunAsync, TSharedPtr<ICmdProgressNotifier> CmdProgressNotifier, FSimpleDelegate OnCompleted)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::LaunchPipInstallAsync);

#if WITH_PYTHON
	if (!IsInitialized())
	{
		// Must explicitly initialize pip installer before use
		return false;
	}
	
	if (CountCachedInstallRequirements() <= 0)
	{
		// Nothing to install
		return true;
	}

	FFeedbackContext* Context = GWarn;

	// Run install of all python dependencies for enabled plugins
	return LaunchPipInstallImpl(RunAsync, CmdProgressNotifier, Context, MoveTemp(OnCompleted));
#else
	return false;
#endif
}

bool FPipInstall::IsInstalling()
{
#if WITH_PYTHON
	return IsInitialized() && IsBackgroundInstalling();
#else
	return false;
#endif //WITH_PYTHON
}

int32 FPipInstall::GetNumPackagesToInstall()
{
#if WITH_PYTHON
	if (!IsInitialized())
	{
		// TODO: Possibly use TOptional instead
		return -1;
	}
	return CountCachedInstallRequirements();
#else
	return -1;
#endif // WITH_PYTHON
}


bool FPipInstall::GetPackageInstallList(TArray<FString>& PyPackages)
{
	PyPackages.Reset();
#if WITH_PYTHON
	if (!IsInitialized())
	{
		return false;
	}

	{
		FScopeLock ScopeLock(&GuardCacheCS);
		PyPackages = CachedInstallRequirements;
	}
	
	return true;
#else
	return false;
#endif // WITH_PYTHON
}

bool FPipInstall::RegisterPipSitePackagesPath() const
{
#if WITH_PYTHON
	const FString PipSitePackagePath = GetPipSitePackagesPath();
	{
		FPyScopedGIL GIL;
		PyUtil::AddSitePackagesPath(PipSitePackagePath);
	}
	return true;
#else
	return false;
#endif //WITH_PYTHON
}


FPipInstall::FPipInstall()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::FPipInstall);

#if WITH_PYTHON
	// TODO: Verify this is necessary
	ensure(IsInGameThread());
	
	bInitialized.Store(false);
	BackgroundInstallRunnable = nullptr;

	// Default install path: <ProjectDir>/PipInstall
	PipInstallPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir() / TEXT("PipInstall"));

	// Check for UE_PIPINSTALL_PATH install path override
	const FString EnvInstallPath = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_PIPINSTALL_PATH"));
	if (!EnvInstallPath.IsEmpty())
	{
		FText ErrReason;
		if (FPaths::ValidatePath(EnvInstallPath, &ErrReason))
		{
			PipInstallPath = FPaths::ConvertRelativePathToFull(EnvInstallPath);
		}
		else
		{
			UE_LOGF(LogPython, Warning, "UE_PIPINSTALL_PATH: Invalid path specified: %ls", *ErrReason.ToString());
		}
	}
	
	VenvInterp = GetVenvInterpreter(PipInstallPath);
#endif //WITH_PYTHON
}


#if WITH_PYTHON

// In order to keep editor startup time fast, check directly for this utils version (make sure to match with wheel version in PythonScriptPlugin/Content/Python/Lib/wheels)
// NOTE: This version must also be changed in PipInstallMode.cs in order to support UBT functionality
const FString FPipInstall::PipInstallUtilsVer = TEXT("0.1.5");

const FString FPipInstall::PluginsListingFilename = TEXT("pyreqs_plugins.list");
const FString FPipInstall::PluginsSitePackageFilename = TEXT("plugin_site_package.pth");
const FString FPipInstall::RequirementsInputFilename = TEXT("merged_requirements.in");
const FString FPipInstall::ExtraUrlsFilename = TEXT("extra_urls.txt");
const FString FPipInstall::ParsedRequirementsFilename = TEXT("merged_requirements.txt");

bool FPipInstall::IsInitialized() const
{
	return bInitialized.Load();
}

bool FPipInstall::IsBackgroundInstalling() const
{
	return BackgroundInstallRunnable.IsValid() && BackgroundInstallRunnable->IsRunning();
}

FString FPipInstall::WritePluginsListing(TArray<TSharedRef<IPlugin>>& OutPythonPlugins) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::WritePluginsListing);

	OutPythonPlugins.Empty();

	// List of plugins with pip dependencies
	TArray<FString> PipPluginPaths;
	for ( const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetEnabledPlugins() )
	{
		const FPluginDescriptor& PluginDesc = Plugin->GetDescriptor();
		if (PluginDesc.CachedJson->HasTypedField(TEXT("PythonRequirements"), EJson::Array))
		{
			const FString PluginDescFile = FPaths::ConvertRelativePathToFull(Plugin->GetDescriptorFileName());
			PipPluginPaths.Add(PluginDescFile);
			OutPythonPlugins.Add(Plugin);
		}
	}

	// Create list of plugins that may require pip install dependencies
	const FString PyPluginsListingFile = PipInstallPath / PluginsListingFilename;
	FFileHelper::SaveStringArrayToFile(PipPluginPaths, *PyPluginsListingFile);

	// Create .pth file in site-packages dir to account for plugins with packaged dependencies
	WriteSitePackagePthFile();

    return PyPluginsListingFile;
}

FString FPipInstall::WritePluginDependencies(const TArray<TSharedRef<IPlugin>>& PythonPlugins, TArray<FString>& OutRequirements, TArray<FString>& OutExtraUrls) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::WritePluginDependencies);

	OutRequirements.Empty();
	OutExtraUrls.Empty();

	for (const TSharedRef<IPlugin>& Plugin : PythonPlugins)
	{
		const FPluginDescriptor& PluginDesc = Plugin->GetDescriptor();
		for (const TSharedPtr<FJsonValue>& JsonVal : PluginDesc.CachedJson->GetArrayField(TEXT("PythonRequirements")))
		{
			const TSharedPtr<FJsonObject>& JsonObj = JsonVal->AsObject();
			if (!CheckCompatiblePlatform(JsonObj, FPlatformMisc::GetUBTPlatform()))
			{
				continue;
			}

			if (!CheckCompatibleArch(JsonObj, PyUtil::GetArchString()))
			{
				continue;
			}

			const TArray<TSharedPtr<FJsonValue>>* PyReqs;
			if (JsonObj->TryGetArrayField(TEXT("Requirements"), PyReqs))
			{
				for (const TSharedPtr<FJsonValue>& JsonReqVal : *PyReqs)
				{
					OutRequirements.Add(JsonReqVal->AsString());
				}
			}

			const TArray<TSharedPtr<FJsonValue>>* PyUrls;
			if (JsonObj->TryGetArrayField(TEXT("ExtraIndexUrls"), PyUrls))
			{
				for (const TSharedPtr<FJsonValue>& JsonUrlVal : *PyUrls)
				{
					OutExtraUrls.Add(JsonUrlVal->AsString());
				}
			}
		}
	}

	const FString MergedReqsFile = FPaths::ConvertRelativePathToFull(PipInstallPath / RequirementsInputFilename);
	const FString ExtraUrlsFile = FPaths::ConvertRelativePathToFull(PipInstallPath / ExtraUrlsFilename);

	FFileHelper::SaveStringArrayToFile(OutRequirements, *MergedReqsFile);
	FFileHelper::SaveStringArrayToFile(OutExtraUrls, *ExtraUrlsFile);

	return MergedReqsFile;
}

class FCheckOrphanDirVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	FCheckOrphanDirVisitor()
	: IPlatformFile::FDirectoryVisitor()
	, bOrphan(true)
	{}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDir) override
	{
		if (!bIsDir)
		{
			bOrphan = false;
			return true;
		}

		//// Short circuit recursion if we don't allow deleting sub-hierarchies and this traversal level is already non-orphan
		//if (!bAllowDeleteSubdirs && !bOrphan)
		//{
		//	return true;
		//}

		// Always treat __pycache__ dir as orphan but don't directly delete them unless full parent is also orphan (nothing but empty or __pycache__ dirs)
		const FString DirPath(FilenameOrDirectory);
		if (DirPath.EndsWith(TEXT("__pycache__")))
		{
			return true;
		}

		FCheckOrphanDirVisitor SubDirVisit;
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		bool res = PlatformFile.IterateDirectory(FilenameOrDirectory, SubDirVisit);

		bOrphan = bOrphan && SubDirVisit.bOrphan;
		if (SubDirVisit.bOrphan) //-V1051
		{
			Orphans.Add(FilenameOrDirectory);
		}
		//else if (bAllowDeleteSubdirs)
		//{
		//	Orphans.Append(SubDirVisit.Orphans);
		//}

		return res;
	}

public:
	bool bOrphan;
	TArray<FString> Orphans;
};

// Remove orphan path hierarchies (hierarchies with only __pycache__ or empty dirs)
// Only runs for <PluginDir>/Content/Python/Lib/* subdirectories for plugins with
// Pip PythonRequirements uplugin section
void FPipInstall::CheckRemoveOrphanedPackages(const FString& SitePackagesPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::CheckRemoveOrphanedPackages);

	if (!FPaths::DirectoryExists(SitePackagesPath))
	{
		return;
	}
	
	// NOTE: FCheckOrphanDirVisitor should only return top-level orphan hierarchies for removal (all or nothing)
	FCheckOrphanDirVisitor DirVisit;
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.IterateDirectory(*SitePackagesPath, DirVisit))
	{
		return;
	}

	if (DirVisit.bOrphan)
	{
		// Remove the entire site-packages dir if everything beneath is orphaned
		UE_LOGF(LogPython, Log, "PipInstall found orphan plugin site-package directory: %ls (removing)", *SitePackagesPath);
		PlatformFile.DeleteDirectoryRecursively(*SitePackagesPath);
	}
	else
	{
		// Only remove specifically orphaned subdirs if there are some valid hierarchies in site-packages
		for (const FString& OrphanDir : DirVisit.Orphans)
		{
			UE_LOGF(LogPython, Log, "PipInstall found orphan plugin site-package directory: %ls (removing)", *OrphanDir);
			PlatformFile.DeleteDirectoryRecursively(*OrphanDir);
		}
	}
}

void FPipInstall::CheckInvalidPipEnv() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::CheckInvalidPipEnv);

	if (!FPaths::DirectoryExists(PipInstallPath))
	{
		return;
	}

	// If not a venv directory don't delete in case offline packages were added before editor run
	FString VenvConfig = PipInstallPath / TEXT("pyvenv.cfg");
	if (!FPaths::FileExists(VenvConfig))
	{
		return;
	}

	const FString VenvVersion = ParseVenvVersion();
	if (VenvVersion == TEXT(PY_VERSION))
	{
		return;
	}

	UE_LOGF(LogPython, Display, "Engine python version (%ls) incompatible with venv (%ls), recreating...", TEXT(PY_VERSION), *VenvVersion);

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.DeleteDirectoryRecursively(*PipInstallPath);
}

bool FPipInstall::SetupPipEnv(FFeedbackContext* Context, bool bForceRebuild /* = false */) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::SetupPipEnv);

	const FString EngineInterp = PyUtil::GetInterpreterExecutablePath();
#ifdef PYTHON_CHECK_SYSEXEC
	// HACK: Set this compiler variable to check what sys.executable python subprocesses get (should match python executable unreal was built against)
	RunPythonCmd(LOCTEXT("PipInstall.DebugInterpWeirdness", "Check Python sys.executable..."), EngineInterp, TEXT("-c \"import sys; print(f'sys.executable: {sys.executable}')\""), Context);
#endif //PYTHON_CHECK_SYSEXEC

	if (!bForceRebuild && FPaths::FileExists(VenvInterp))
	{
		return SetupPipInstallUtils(Context);
	}

	if (bForceRebuild)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.DeleteDirectoryRecursively(*PipInstallPath);
		
		// Regen the initial files needed here (it's a waste of time, but this is a rare scenario)
		TArray<TSharedRef<IPlugin>> PythonPlugins;
		WritePluginsListing(PythonPlugins);

		TArray<FString> ReqInLines;
		TArray<FString> ExtraUrls;
		const FString ReqsInFile = WritePluginDependencies(PythonPlugins, ReqInLines, ExtraUrls);
	}

	const FString VenvCmd = FString::Printf(TEXT("-m venv \"%s\""), *FPaths::ConvertRelativePathToFull(PipInstallPath));
	int32 Res = RunPythonCmd(EngineInterp, VenvCmd, Context);
	if (Res != 0)
	{
		UE_LOGF(LogPython, Error, "Unable to create pip install environment (%d)", Res);
		return false;
	}

	return SetupPipInstallUtils(Context);
}

void FPipInstall::RemoveParsedDependencyFiles() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::RemoveParsedDependencyFiles);

	const FString ParsedReqsFile = PipInstallPath / ParsedRequirementsFilename;
	if (FPaths::FileExists(ParsedReqsFile))
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.DeleteFile(*ParsedReqsFile);
	}
}

bool FPipInstall::ParsePluginDependencies(const FString& MergedInRequirementsFile, FFeedbackContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::ParsePluginDependencies);

	const FString ParsedReqsFile = PipInstallPath / ParsedRequirementsFilename;

	// NOTE: Hashes are all-or-nothing so if we are ignoring, just remove them all with the parser
	// TODO: Handle this per-plugin
	FString DisableHashes = TEXT("");
	if (!GetDefault<UPythonScriptPluginSettings>()->bPipStrictHashCheck)
	{
		DisableHashes = TEXT("--disable-hashes");
	}

	const FString Cmd = FString::Printf(TEXT("-m ue_parse_plugin_reqs %s -vv \"%s\" \"%s\""), *DisableHashes, *MergedInRequirementsFile, *ParsedReqsFile);
	return (RunPythonCmd(VenvInterp, Cmd, Context) == 0);
}

bool FPipInstall::LaunchPipInstallImpl(bool RunAsync, TSharedPtr<ICmdProgressNotifier> CmdProgressNotifier, FFeedbackContext* Context, FSimpleDelegate OnCompleted)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::LaunchPipInstallImpl);

	if (IsBackgroundInstalling())
	{
		return false;
	}

	const FString ParsedReqsFile = GetParsedRequirementsPath();
	const FString ExtraUrlsFile = GetExtraUrlsPath();

	if (!FPaths::FileExists(ParsedReqsFile))
	{
		UE_LOGF(LogPython, Warning, "PipInstaller background task already running");
		return true;
	}

	int ReqCount = CountCachedInstallRequirements();
	if ( ReqCount == 0 )
	{
		return true;
	}

	TArray<FString> ExtraUrls;
	if (FPaths::FileExists(ExtraUrlsFile))
	{
		FFileHelper::LoadFileToStringArray(ExtraUrls, *ExtraUrlsFile);
	}

	const FString Cmd = SetupPipInstallCmd(ParsedReqsFile, ExtraUrls);

	TSharedPtr<ICmdProgressParser> ProgressParser = nullptr;
	if (CmdProgressNotifier.IsValid())
	{
		ProgressParser = MakeShared<FPipProgressParser>(ReqCount, CmdProgressNotifier.ToSharedRef());
	}

	if (RunAsync)
	{
		return RunPipCmdAsync(VenvInterp, Cmd, Context, ProgressParser, MoveTemp(OnCompleted));
	}

	return RunPipCmdSync(VenvInterp, Cmd, Context, ProgressParser, MoveTemp(OnCompleted));
}

FString FPipInstall::GetPipInstallPath() const
{
	return PipInstallPath;
}

FString FPipInstall::GetPipSitePackagesPath() const
{
	const FString VenvPath = GetPipInstallPath();
#if PLATFORM_WINDOWS
	return VenvPath / TEXT("Lib") / TEXT("site-packages");
#elif PLATFORM_MAC || PLATFORM_LINUX
	return VenvPath / TEXT("lib") / FString::Printf(TEXT("python%d.%d"), PY_MAJOR_VERSION, PY_MINOR_VERSION) / TEXT("site-packages");
#else
	static_assert(false, "Python not supported on this platform!");
#endif
}

bool FPipInstall::CacheDetectInstallDeps(FFeedbackContext* Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::CacheDetectInstallDeps);
	
	const FString ReqsInFile = GetInputRequirementsPath();
	if (!ParsePluginDependencies(ReqsInFile, Context))
	{
		return false;
	}
	
	return UpdateCachedInstallRequirements();
}


int32 FPipInstall::CountCachedInstallRequirements()
{
	FScopeLock ScopeLock(&GuardCacheCS);
	return CachedInstallRequirements.Num();
}

void FPipInstall::ClearCachedInstallRequirements()
{
	FScopeLock ScopeLock(&GuardCacheCS);
	CachedInstallRequirements.Empty();
}


bool FPipInstall::UpdateCachedInstallRequirements()
{
	FScopeLock ScopeLock(&GuardCacheCS);
	CachedInstallRequirements.Empty();
	
	const FString ParsedReqsFile = GetParsedRequirementsPath();
	if (!FPaths::FileExists(ParsedReqsFile))
	{
		return false;
	}

	TArray<FString> ParsedReqLines;
	if ( !FFileHelper::LoadFileToStringArray(ParsedReqLines, *ParsedReqsFile) )
	{
		return false;
	}

	for (const FStringView Line : ParsedReqLines)
	{
		bool bCommentLine = Line.TrimStart().StartsWith(TCHAR('#'));
		if (!bCommentLine && !Line.Contains(TEXT("# [pkg:check]")))
		{
			CachedInstallRequirements.Add(FString(Line));
		}
	}

	return true;
}

FString FPipInstall::GetInputRequirementsPath() const
{
	return FPaths::ConvertRelativePathToFull(PipInstallPath / RequirementsInputFilename);
}

FString FPipInstall::GetParsedRequirementsPath() const
{
	return FPaths::ConvertRelativePathToFull(PipInstallPath / ParsedRequirementsFilename);
}

FString FPipInstall::GetExtraUrlsPath() const
{
	return FPaths::ConvertRelativePathToFull(PipInstallPath / ExtraUrlsFilename);
}

void FPipInstall::WriteSitePackagePthFile() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::WriteSitePackagePthFile);

	// Write all paths from script-plugin
	// TODO: Should we directly use PyUtil::GetSystemPaths instead?
	// TArray<FString> PluginSitePackagePaths = PyUtil::GetSystemPaths();

	// List of enabled plugins' site-packages folders
	TArray<FString> PluginSitePackagePaths;
	UE::FMutex Mutex;

	TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetEnabledPlugins();
	ParallelFor(Plugins.Num(),
		[&PluginSitePackagePaths, &Plugins, &Mutex](int32 Index)
		{
			TSharedRef<IPlugin> Plugin = Plugins[Index];
			const FString PythonContentPath = FPaths::ConvertRelativePathToFull(Plugin->GetContentDir() / TEXT("Python"));
			
			TArray<TTuple<bool,FString>> SitePackagePathChecks;
			SitePackagePathChecks.Reserve(PyUtil::GetSitePackageSubdirs().Num());

			// Write platform/general site-packages paths per-plugin to .pth file to account for packaged python dependencies during pip install
			bool AnySitePackages = false;
			for (const FString& SitePkgDir : PyUtil::GetSitePackageSubdirs())
			{
				FString ChkPath = PythonContentPath / SitePkgDir;
				bool bPathExists = FPaths::DirectoryExists(ChkPath);
				
				SitePackagePathChecks.Emplace(bPathExists, MoveTemp(ChkPath));
				AnySitePackages |= bPathExists;
			}

			if (AnySitePackages)
			{
				UE::TUniqueLock ScopeLock(Mutex);
				for (auto& Check : SitePackagePathChecks)
				{
					if (Check.Get<0>())
					{
						PluginSitePackagePaths.Emplace(MoveTemp(Check.Get<1>()));
					}
				}
			}
		}
	);

	// Additional paths
	for (const FDirectoryPath& AdditionalPath : GetDefault<UPythonScriptPluginSettings>()->AdditionalPaths)
	{
		FString AddPath = FPaths::ConvertRelativePathToFull(AdditionalPath.Path);
		if (FPaths::DirectoryExists(AddPath))
		{
			PluginSitePackagePaths.Emplace(MoveTemp(AddPath));
		}
	}

	// UE_PYTHONPATH
	{
		TArray<FString> SystemEnvPaths;
		FPlatformMisc::GetEnvironmentVariable(TEXT("UE_PYTHONPATH")).ParseIntoArray(SystemEnvPaths, FPlatformMisc::GetPathVarDelimiter());
		for (FString& SystemEnvPath : SystemEnvPaths)
		{
			if (FPaths::DirectoryExists(SystemEnvPath))
			{
				PluginSitePackagePaths.Emplace(MoveTemp(SystemEnvPath));
			}
		}
	}

	// Make sure the order in the file is deterministic
	PluginSitePackagePaths.Sort();

	// Create .pth file in PipInstall/Lib/site-packages to account for plugins with packaged dependencies
	const FString PyPluginsSitePackageFile = FPaths::ConvertRelativePathToFull(GetPipSitePackagesPath() / PluginsSitePackageFilename);
	FFileHelper::SaveStringArrayToFile(PluginSitePackagePaths, *PyPluginsSitePackageFile, FFileHelper::EEncodingOptions::ForceUTF8);
}


bool FPipInstall::SetupPipInstallUtils(FFeedbackContext* Context) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::SetupPipInstallUtils);

	if (CheckPipInstallUtils(Context))
	{
		return true;
	}

	const FString PythonScriptDir = GetPythonScriptPluginPath();
	if (PythonScriptDir.IsEmpty())
	{
		return false;
	}

	const FString PipWheelsDir = FPaths::ConvertRelativePathToFull(PythonScriptDir / TEXT("Content/Python/Lib/wheels"));
	const FString InstallRequirements = FPaths::ConvertRelativePathToFull(PythonScriptDir / TEXT("Content/Python/PipInstallUtils/requirements.txt"));

	const FString PipInstallReq = TEXT("ue-pipinstall-utils==") + PipInstallUtilsVer;
	const FString Cmd = FString::Printf(TEXT("-m pip install --upgrade --no-index --find-links \"%s\" -r \"%s\" %s"), *PipWheelsDir, *InstallRequirements, *PipInstallReq);

	return (RunPythonCmd(VenvInterp, Cmd, Context) == 0);
}


bool FPipInstall::CheckPipInstallUtils(FFeedbackContext* Context) const
{
	// Verify that correct version of pip install utils is already available
	const FString Cmd = FString::Printf(TEXT("-c \"import pkg_resources;dist=pkg_resources.working_set.find(pkg_resources.Requirement.parse('ue-pipinstall-utils'));exit(dist.version!='%s' if dist is not None else 1)\""), *PipInstallUtilsVer);
	return (RunPythonCmd(VenvInterp, Cmd, Context) == 0);
}

FString FPipInstall::SetupPipInstallCmd(const FString& ParsedReqsFile, const TArray<FString>& ExtraUrls) const
{
	const UPythonScriptPluginSettings* ScriptSettings = GetDefault<UPythonScriptPluginSettings>();

	FString Cmd = TEXT("-m pip install --disable-pip-version-check --only-binary=:all:");

	// Force require hashes in requirements lines by default
	if (ScriptSettings->bPipStrictHashCheck)
	{
		Cmd += TEXT(" --require-hashes");
	}

	if (ScriptSettings->bOfflineOnly)
	{
		Cmd += TEXT(" --no-index");
	}
	else if (!ScriptSettings->OverrideIndexURL.IsEmpty())
	{
		Cmd += TEXT(" --index-url ") + ScriptSettings->OverrideIndexURL;
	}
	else if (!ExtraUrls.IsEmpty())
	{
		for (const FString& Url: ExtraUrls)
		{
			Cmd += TEXT(" --extra-index-url ") + Url;
		}
	}

	if (!ScriptSettings->ExtraInstallArgs.IsEmpty())
	{
		Cmd += " " + ScriptSettings->ExtraInstallArgs;
	}
	 
	Cmd += " -r \"" + ParsedReqsFile + "\"";
	
	return Cmd;
}

int32 FPipInstall::RunPythonCmd(const FString& PythonInterp, const FString& Cmd, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::RunPythonCmd);

	UE_LOGF(LogPython, Log, "Running python command: python %ls", *Cmd);

	int32 OutResult = 0;
	if (!FLoggedSubprocessSync::Run(OutResult, FPaths::ConvertRelativePathToFull(PythonInterp), Cmd, Context, CmdParser))
	{
		UE_LOGF(LogPython, Error, "Unable to create python process");
		return -1;
	}

	return OutResult;
}

bool FPipInstall::RunPipCmdAsync(const FString& PythonInterp, const FString& Cmd, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser, FSimpleDelegate OnCompleted)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPipInstall::RunPythonCmd);
	
	UE_LOGF(LogPython, Log, "Running background pip install: python %ls", *Cmd);
	BackgroundInstallRunnable = MakeShared<FLoggedSubprocessThread>(FPaths::ConvertRelativePathToFull(PythonInterp), Cmd, Context, CmdParser);
	BackgroundInstallRunnable->OnCompleted().BindLambda([this, OnCompleted = MoveTemp(OnCompleted)](int32 ReturnCode)
	{
		if (ReturnCode == 0)
		{
			CacheDetectInstallDeps();

			if (OnCompleted.IsBound())
			{
				AsyncTask(ENamedThreads::GameThread, [OnCompleted]()
				{
					OnCompleted.Execute();
				});
			}
		}
	});
	
	return BackgroundInstallRunnable->Launch();
}

bool FPipInstall::RunPipCmdSync(const FString& PythonInterp, const FString& Cmd, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser, FSimpleDelegate OnCompleted)
{
	int Res = RunPythonCmd(VenvInterp, Cmd, Context, CmdParser);
	if (Res == 0)
	{
		// TODO: Update regardless of success?
		bool bCached = CacheDetectInstallDeps();
		OnCompleted.ExecuteIfBound();
		return bCached;
	}

	return false;
}


FString FPipInstall::GetPythonScriptPluginPath()
{
	TSharedPtr<IPlugin> PythonPlugin = IPluginManager::Get().FindPlugin("PythonScriptPlugin");
	if (!PythonPlugin)
	{
		return TEXT("");
	}

	return PythonPlugin->GetBaseDir();
}

FString FPipInstall::ParseVenvVersion() const
{
	FString VenvConfig = PipInstallPath / TEXT("pyvenv.cfg");
	if (!FPaths::FileExists(VenvConfig))
	{
		return TEXT("");
	}

	TArray<FString> ConfigLines;
	if (!FFileHelper::LoadFileToStringArray(ConfigLines, *VenvConfig))
	{
		return TEXT("");
	}

	for (FStringView Line : ConfigLines)
	{
		FStringView ChkLine = Line.TrimStartAndEnd();
		if (!ChkLine.StartsWith(TEXT("version =")))
		{
			continue;
		}

		FStringView Version = ChkLine.RightChop(9).TrimStart();
		return FString(Version);
	}

	return TEXT("");
}

FString FPipInstall::GetVenvInterpreter(const FString& InstallPath)
{
#if PLATFORM_WINDOWS
	return InstallPath / TEXT("Scripts/python.exe");
#elif PLATFORM_MAC || PLATFORM_LINUX
	return InstallPath / TEXT("bin/python3");
#else
	static_assert(false, "Python not supported on this platform!");
#endif
}

bool FPipInstall::CheckCompatiblePlatform(const TSharedPtr<FJsonObject>& JsonObject, const FString& PlatformName)
{
	FString JsonPlatform;
	return !JsonObject->TryGetStringField(TEXT("Platform"), JsonPlatform) || JsonPlatform.Equals(TEXT("All"), ESearchCase::IgnoreCase) || JsonPlatform.Equals(PlatformName, ESearchCase::IgnoreCase);
}

bool FPipInstall::CheckCompatibleArch(const TSharedPtr<FJsonObject>& JsonObject, const FString& ArchName)
{
	FString JsonArch;
	return !JsonObject->TryGetStringField(TEXT("Architecture"), JsonArch) || JsonArch.Equals(TEXT("All"), ESearchCase::IgnoreCase) || JsonArch.Equals(ArchName, ESearchCase::IgnoreCase);
}

#endif //WITH_PYTHON

#undef LOCTEXT_NAMESPACE
