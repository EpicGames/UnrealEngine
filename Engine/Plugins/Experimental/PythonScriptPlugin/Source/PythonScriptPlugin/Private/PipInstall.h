// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPipInstall.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "Interfaces/IPluginManager.h"
#include "Templates/SharedPointer.h"

class FJsonObject;
class FFeedbackContext;

// Forward declares from PipRunnable.h
struct ICmdProgressParser;
class FLoggedSubprocessThread;

// Static singleton implementation of IPipInstall
class FPipInstall : public IPipInstall
{
public:
	static FPipInstall& Get();

	// IPipInstall Interface Implementation
	virtual bool InitPipInstall() override;
	// Run pip to install all missing dependencies for enabled plugins
	virtual bool LaunchPipInstall(bool RunAsync, TSharedPtr<ICmdProgressNotifier> CmdProgressNotifier = nullptr, FSimpleDelegate OnCompleted = FSimpleDelegate()) override;
	// Check if a background install is still running
	virtual bool IsInstalling() override;
	
	// Get number of missing python packages to install
	virtual int32 GetNumPackagesToInstall() override;
	// Get list of python packages to install
	virtual bool GetPackageInstallList(TArray<FString>& PyPackages) override;
	
	// Register the site-packages path with embedded python env
	virtual bool RegisterPipSitePackagesPath() const override;

private:
	friend class IPipInstall;
	FPipInstall();

#if WITH_PYTHON
	
private:
	bool IsInitialized() const;
	bool IsBackgroundInstalling() const;

	void CheckRemoveOrphanedPackages(const FString& SitePackagesPath);
	void CheckInvalidPipEnv() const;

	FString WritePluginsListing(TArray<TSharedRef<IPlugin>>& OutPythonPlugins) const;
	FString WritePluginDependencies(const TArray<TSharedRef<IPlugin>>& PythonPlugins, TArray<FString>& OutRequirements, TArray<FString>& OutExtraUrls) const;

	bool SetupPipEnv(FFeedbackContext* Context, bool bForceRebuild = false) const;
	void RemoveParsedDependencyFiles() const;
	bool ParsePluginDependencies(const FString& MergedInRequirementsFile, FFeedbackContext* Context) const;
	bool LaunchPipInstallImpl(bool RunAsync, TSharedPtr<ICmdProgressNotifier> CmdProgressNotifier, FFeedbackContext* Context, FSimpleDelegate OnCompleted);

	FString GetPipInstallPath() const;
	FString GetPipSitePackagesPath() const;

	bool CacheDetectInstallDeps(FFeedbackContext* Context = nullptr);
	
	int32 CountCachedInstallRequirements();
	void ClearCachedInstallRequirements();
	bool UpdateCachedInstallRequirements();

private:

	FString GetInputRequirementsPath() const;
	FString GetParsedRequirementsPath() const;
	FString GetExtraUrlsPath() const;
	
	void WriteSitePackagePthFile() const;
	bool SetupPipInstallUtils(FFeedbackContext* Context) const;
	bool CheckPipInstallUtils(FFeedbackContext* Context) const;

	FString ParseVenvVersion() const;
	FString SetupPipInstallCmd(const FString& ParsedReqsFile, const TArray<FString>& ExtraUrls) const;
	bool RunPipCmdAsync(const FString& PythonInterp, const FString& Cmd, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser, FSimpleDelegate OnCompleted = FSimpleDelegate());
	bool RunPipCmdSync(const FString& PythonInterp, const FString& Cmd, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser, FSimpleDelegate OnCompleted = FSimpleDelegate());

	static int32 RunPythonCmd(const FString& PythonInterp, const FString& Cmd, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser = nullptr);

	static FString GetPythonScriptPluginPath();
	static FString GetVenvInterpreter(const FString& InstallPath);

	static bool CheckCompatibleArch(const TSharedPtr<FJsonObject>& JsontObject, const FString& ArchName);
	static bool CheckCompatiblePlatform(const TSharedPtr<FJsonObject>& JsonObject, const FString& PlatformName);

	TAtomic<bool> bInitialized;
	FCriticalSection GuardCacheCS;
	TSharedPtr<FLoggedSubprocessThread> BackgroundInstallRunnable;
	
	TArray<FString> CachedInstallRequirements;

	FString PipInstallPath;
	FString VenvInterp;

	static const FString PipInstallUtilsVer;
	static const FString PluginsListingFilename;
	static const FString PluginsSitePackageFilename;
	static const FString RequirementsInputFilename;
	static const FString ExtraUrlsFilename;
	static const FString ParsedRequirementsFilename;
	
#endif //WITH_PYTHON
};
