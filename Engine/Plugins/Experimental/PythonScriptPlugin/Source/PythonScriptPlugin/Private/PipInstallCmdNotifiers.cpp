// Copyright Epic Games, Inc. All Rights Reserved.

#include "PipInstallCmdNotifiers.h"
#include "PyUtil.h"
#include "Internationalization/Text.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/SlowTask.h"

#define LOCTEXT_NAMESPACE "PipInstall"

// FSlowTaskNotifier Implementation
FSlowTaskNotifier::FSlowTaskNotifier(float GuessSteps, const FText& Description)
	: SlowTask(MakePimpl<FSlowTask>(GuessSteps, Description, true, *GWarn))
	, TotalWork(FMath::Max(GuessSteps, 1.0f))
	, WorkDone(SlowTask->CompletedWork)
{
	SlowTask->Initialize();
	SlowTask->Visibility = ESlowTaskVisibility::Important;
}

FSlowTaskNotifier::~FSlowTaskNotifier()
{
	SlowTask->Destroy();
}

void FSlowTaskNotifier::UpdateProgress(float UpdateWorkDone, float UpdateTotalWork, const FText& Status)
{
	// Exponentially approach 100% if work goes above estimate
	float NextDone = TotalWork * UpdateWorkDone / UpdateTotalWork;
	float NextStep = FMath::Max(NextDone - WorkDone, 0.0f);
	float ProgLeft = FMath::Max(TotalWork - WorkDone, 0.0f);

	// Clamp to 90% of remaining progress
	float NextWork = FMath::Clamp(NextStep, 0.0f, 0.9*ProgLeft);

	// TODO: Pass in specific requirements to update status lines more accurately
	SlowTask->EnterProgressFrame(NextWork);
	WorkDone += NextWork;
}
	
void FSlowTaskNotifier::Completed(bool bSuccess)
{}


//FAsyncTaskCmdNotifier Implementation
FAsyncTaskCmdNotifier::FAsyncTaskCmdNotifier(float GuessSteps, const FText& Description)
	: TotalWork(FMath::Max(GuessSteps, 1.0f)), WorkDone(0.0f)
{
	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bIsHeadless = false;
	NotificationConfig.TitleText = Description;
	NotificationConfig.LogCategory = &LogPython;
	NotificationConfig.bCanCancel.Set(false);
	NotificationConfig.bKeepOpenOnSuccess.Set(true);
	NotificationConfig.bKeepOpenOnFailure.Set(true);

	AsyncNotifier = MakePimpl<FAsyncTaskNotification>(NotificationConfig);
	AsyncNotifier->SetNotificationState(FAsyncNotificationStateData(Description, FText::GetEmpty(), EAsyncTaskNotificationState::Pending));
}

void FAsyncTaskCmdNotifier::UpdateProgress(float UpdateWorkDone, float UpdateTotalWork, const FText& Status)
{
	float PctDone = (float)UpdateWorkDone / UpdateTotalWork;
	AsyncNotifier->SetProgressText(FText::Format(LOCTEXT("AsyncTaskCmdNotifier.StatusFmt", "{0} [{1}]"), Status, FText::AsPercent(PctDone)));
}

void FAsyncTaskCmdNotifier::Completed(bool bSuccess)
{
	AsyncNotifier->SetComplete(bSuccess);
}

#undef LOCTEXT_NAMESPACE
