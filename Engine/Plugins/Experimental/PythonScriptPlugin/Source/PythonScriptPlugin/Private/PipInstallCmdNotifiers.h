// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPipInstall.h"
#include "Templates/PimplPtr.h"

class FAsyncTaskNotification;
struct FSlowTask;
class FText;

// Slowtask-based notifier for updating command progress (ICmdProgressNotifier)
class FSlowTaskNotifier : public ICmdProgressNotifier
{
public:
	FSlowTaskNotifier(float GuessSteps, const FText& Description);
	virtual ~FSlowTaskNotifier() override;
	
	// ICmdProgressNotifier methods
	virtual void UpdateProgress(float UpdateWorkDone, float UpdateTotalWork, const FText& Status) override;
	virtual void Completed(bool bSuccess) override;

private:
	TPimplPtr<FSlowTask> SlowTask;

	float TotalWork;
	float WorkDone;
};


// Asyntask notifier for updating background install commands
class FAsyncTaskCmdNotifier : public ICmdProgressNotifier
{
public:
	FAsyncTaskCmdNotifier(float GuessSteps, const FText& Description);
	
	// ICmdProgressNotifier methods
	virtual void UpdateProgress(float UpdateWorkDone, float UpdateTotalWork, const FText& Status) override;
	virtual void Completed(bool bSuccess) override;

private:
	TPimplPtr<FAsyncTaskNotification> AsyncNotifier;

	float TotalWork;
	float WorkDone;
};

