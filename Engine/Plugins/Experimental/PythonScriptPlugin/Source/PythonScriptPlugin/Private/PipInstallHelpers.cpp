// Copyright Epic Games, Inc. All Rights Reserved.

#include "PipInstallHelpers.h"
#include "IPipInstall.h"
#include "IPythonScriptPlugin.h"
#include "PipInstallLauncher.h"
#include "PipInstallCmdNotifiers.h"
#include "PyUtil.h"

#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Text.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "PipInstall"

class SPipInstallDialogWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPipInstallDialogWidget)
		: _PipInstallPtr(nullptr)
		, _AllowBackground(true)
		, _DialogWindow(nullptr)
		{}

		SLATE_ARGUMENT(IPipInstall*, PipInstallPtr)
		SLATE_ARGUMENT(bool, AllowBackground)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, DialogWindow)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& Args)
	{
		PipInstallPtr = Args._PipInstallPtr;
		DialogWindow = Args._DialogWindow;
		AllowBackground = Args._AllowBackground;

		InitInsallList();

		FText InstallButtonTooltip;
		FText CancelButtonTooltip;
		if (AllowBackground)
		{
			InstallButtonTooltip = LOCTEXT("PipInstallUI.InstallBG.ToolTip", "Click to begin background package installation");
			CancelButtonTooltip = LOCTEXT("PipInstallUI.CancelBG.ToolTip", "Click to begin background package installation");
		}
		else
		{
			InstallButtonTooltip = LOCTEXT("PipInstallUI.InstallFG.ToolTip", "Click to begin package installation");
			CancelButtonTooltip = LOCTEXT("PipInstallUI.CancelFG.ToolTip", "Packages must be installed to continue");
		}
		
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SNew(STextBlock)
				.Text(FText::FormatOrdered(LOCTEXT("PipInstallPackages.Title", "Packages to Install ({0}): "), PipInstallPtr->GetNumPackagesToInstall()))
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.MaxHeight(200.0)
				.Padding(2)
			[
				SAssignNew(PackageInstallList, SListView<TSharedPtr<FString>>)
				.SelectionMode(ESelectionMode::None)
				.ListItemsSource(&InstallPackages)
				.OnGenerateRow(this, &SPipInstallDialogWidget::OnGenerateListRow)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
			[
				SNew(SUniformGridPanel)
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("PipInstallUI.Install", "Install Packages"))
						.ToolTipText(InstallButtonTooltip)
						.OnClicked(this, &SPipInstallDialogWidget::OnInstallClicked)
				]
				+ SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(LOCTEXT("PipInstallUI.Cancel", "Cancel"))
						.IsEnabled(AllowBackground && !PipInstallPtr->IsInstalling())
						.ToolTipText(CancelButtonTooltip)
						.OnClicked(this, &SPipInstallDialogWidget::OnCancelClicked)
				]
			]
		];
	}

	//~ Begin SWidget interface
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape && AllowBackground)
		{
			return OnCancelClicked();
		}
		else if ( InKeyEvent.GetKey() == EKeys::Enter  )
		{
			return OnInstallClicked();
		}

		return FReply::Unhandled();
	}
	//~ End SWidget interface

	bool ShouldStartInstall()
	{
		return StartInstall;
	}
	
private:
	void InitInsallList()
	{
		TArray<FString> Packages;
		PipInstallPtr->GetPackageInstallList(Packages);
		for ( const FString& Package : Packages )
		{
			// Remove hashes/comments, etc from the pip requirements lines keep only "package==ver" portion
			FStringView PackageView = Package;
			PackageView.TrimStartInline();
			int32 CutIdx;
			if (PackageView.FindChar(TEXT(' '), CutIdx))
			{
				PackageView.LeftInline(CutIdx);
			}

			InstallPackages.Add(MakeShared<FString>(PackageView));
		}
	}

	TSharedRef<ITableRow> OnGenerateListRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
			[
				SNew(SOverlay)
				.Visibility(EVisibility::SelfHitTestInvisible)

				+SOverlay::Slot()
				.Padding(0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FLinearColor::Black)
				]
				+SOverlay::Slot()
				.Padding(0.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(*InItem.Get()))
				]	
			];
	}

	FReply OnInstallClicked()
	{
		StartInstall = true;
		if (DialogWindow.IsValid())
		{
			DialogWindow.Pin()->RequestDestroyWindow();
		}
		
		return FReply::Handled();
	}

	FReply OnCancelClicked()
	{
		StartInstall = false;
		if (DialogWindow.IsValid())
		{
			DialogWindow.Pin()->RequestDestroyWindow();
		}
		
		return FReply::Handled();
	}
	
private:
	bool StartInstall = false;
	
	bool AllowBackground = true;
	// Main settings object
	IPipInstall* PipInstallPtr = nullptr;
	// Main window
	TWeakPtr<SWindow> DialogWindow;
	
	// List view for packages needing install
	TSharedPtr<SListView<TSharedPtr<FString>>> PackageInstallList;

	// List of packages to install
	TArray<TSharedPtr<FString>> InstallPackages;
};

bool ShowPackageInstallDialogModal(IPipInstall& PipInstall, bool bAllowBackground)
{
	TSharedPtr<SWindow> ParentWindow = nullptr;
	if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
	{
		IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
		ParentWindow = MainFrame.GetParentWindow();
	}
	
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("PipInstallDialog.Title", "Python Dependencies Install"))
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::PrimaryWorkArea);

	TSharedPtr<SPipInstallDialogWidget> PipInstallDialog;
	Window->SetContent(
		SAssignNew(PipInstallDialog, SPipInstallDialogWidget)
			.PipInstallPtr(&PipInstall)
			.AllowBackground(bAllowBackground)
			.DialogWindow(Window)
	);

	FSlateApplication::Get().AddModalWindow(Window, ParentWindow, /*bSlowTaskWindow =*/false);

	return PipInstallDialog->ShouldStartInstall();
}

int FPipInstallHelper::GetNumPackagesToInstall()
{
	check(IsInGameThread());
	
	IPipInstall& PipInstall = IPipInstall::Get();
	if (!PipInstall.InitPipInstall())
	{
		UE_LOGF(LogPython, Error, "Unable to initialize Pip Install");
		return -1;
	}

	return PipInstall.GetNumPackagesToInstall();
}


EPipInstallDialogResult FPipInstallHelper::ShowPipInstallDialog(bool bAllowBackgroundInstall, FSimpleDelegate OnCompleted)
{
	check(IsInGameThread());
	
	IPipInstall& PipInstall = IPipInstall::Get();
	if (!PipInstall.InitPipInstall())
	{
		UE_LOGF(LogPython, Error, "Unable to initialize Pip Install");
		return EPipInstallDialogResult::Error;
	}

	//if (PipInstall.IsInstalling())
	//{
	//	return EPipInstallDialogResult::BackgroundInstall;
	//}
	
	bool bClickedInstall = ShowPackageInstallDialogModal(PipInstall, bAllowBackgroundInstall);

	// Check we actually need to install anything
	if (PipInstall.GetNumPackagesToInstall() == 0)
	{
		return EPipInstallDialogResult::Finished;
	}
	
	// Install must be run immediately (game-thread) if background disallowed
	if (!bAllowBackgroundInstall)
	{
		return (PipInstallLauncher::StartSync(PipInstall, MoveTemp(OnCompleted)) ? EPipInstallDialogResult::Finished : EPipInstallDialogResult::Error); 
	}
	else if (PipInstall.IsInstalling())
	{
		return EPipInstallDialogResult::BackgroundInstall;
	}
	else if (bClickedInstall)
	{
		return (PipInstallLauncher::StartAsync(PipInstall, MoveTemp(OnCompleted)) ? EPipInstallDialogResult::BackgroundInstall : EPipInstallDialogResult::Error);
	}
	
	return EPipInstallDialogResult::Canceled;
}

bool FPipInstallHelper::LaunchHeadlessPipInstall()
{
	check(IsInGameThread());

	IPipInstall& PipInstall = IPipInstall::Get();
	if (!PipInstall.InitPipInstall())
	{
		UE_LOGF(LogPython, Error, "Unable to initialize Pip Install");
		return false;
	}

	return PipInstallLauncher::StartSync(PipInstall); 
}

void FPythonScriptInitHelper::InitPython(FSimpleDelegate OnInitialized, FSimpleDelegate OnPythonUnavailable)
{
	IPythonScriptPlugin::Get()->ForceEnablePythonAtRuntime();
	IPythonScriptPlugin::Get()->RegisterOnPythonConfigured(FSimpleDelegate::CreateLambda([OnInitialized = MoveTemp(OnInitialized), OnPythonUnavailable = MoveTemp(OnPythonUnavailable)]() mutable
	{
		if (IPythonScriptPlugin::Get()->IsPythonAvailable())
		{
			IPythonScriptPlugin::Get()->RegisterOnPythonInitialized(MoveTemp(OnInitialized));
		}
		else
		{
			OnPythonUnavailable.ExecuteIfBound();
		}
	}));
}

void FPythonScriptInitHelper::InitPythonAndPipInstall(FSimpleDelegate OnPipInstalled, FSimpleDelegate OnPythonUnavailable)
{
	InitPython(FSimpleDelegate::CreateLambda([OnPipInstalled = MoveTemp(OnPipInstalled)]()
	{
		// Run deferred pip install through UI interface if necessary
		int NumPackages = FPipInstallHelper::GetNumPackagesToInstall();
		if (NumPackages > 0)
		{
			FPipInstallHelper::ShowPipInstallDialog(true, OnPipInstalled);
		}
		else
		{
			OnPipInstalled.ExecuteIfBound();
		}
	}), MoveTemp(OnPythonUnavailable));
}


namespace PipInstallLauncher
{
	bool StartSync(IPipInstall& PipInstall, FSimpleDelegate OnCompleted)
	{
		TSharedPtr<ICmdProgressNotifier> CmdNotifier = MakeShared<FSlowTaskNotifier>(PipInstall.GetNumPackagesToInstall(), LOCTEXT("PipInstall.FGInstallText", "Installing Python Dependencies..."));
		return PipInstall.LaunchPipInstall(false, CmdNotifier, MoveTemp(OnCompleted));
	}

	bool StartAsync(IPipInstall& PipInstall, FSimpleDelegate OnCompleted)
	{
		TSharedPtr<ICmdProgressNotifier> CmdProgressNotifier = MakeShared<FAsyncTaskCmdNotifier>(PipInstall.GetNumPackagesToInstall(), LOCTEXT("PipInstall.BGInstallText", "Installing Python Dependencies..."));
		return PipInstall.LaunchPipInstall(true, CmdProgressNotifier, MoveTemp(OnCompleted));
	}
}

#undef LOCTEXT_NAMESPACE
