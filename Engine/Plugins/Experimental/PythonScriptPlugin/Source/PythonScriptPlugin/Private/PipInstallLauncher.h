// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPipInstall.h"

namespace PipInstallLauncher
{
	bool StartSync(IPipInstall& PipInstall, FSimpleDelegate OnCompleted = FSimpleDelegate());
	bool StartAsync(IPipInstall& PipInstall, FSimpleDelegate OnCompleted = FSimpleDelegate());
}
