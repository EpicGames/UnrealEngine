// Copyright Epic Games, Inc. All Rights Reserved.

#include "PipRunnable.h"

#include "PyUtil.h"
#include "Misc/FeedbackContext.h"

#define LOCTEXT_NAMESPACE "PipInstall"

// FPipProgressParser Implementation
const TArray<FString> FPipProgressParser::MatchStatusStrs = {TEXT("Requirement"), TEXT("Downloading"), TEXT("Collecting"), TEXT("Using"), TEXT("Installing")};
const TMap<FString,FString> FPipProgressParser::LogReplaceStrs = {{TEXT("Installing collected packages:"), TEXT("Installing collected python package dependencies:")}};

FPipProgressParser::FPipProgressParser(int GuessRequirementsCount, TSharedRef<ICmdProgressNotifier> InCmdNotifier)
	: RequirementsDone(0.0f)
	, RequirementsCount(FMath::Max(2.0f*GuessRequirementsCount,1.0f))
	, CmdNotifier(InCmdNotifier)
{}

float FPipProgressParser::GetTotalWork()
{
	return RequirementsCount;
}

float FPipProgressParser::GetWorkDone()
{
	return RequirementsDone;
}

bool FPipProgressParser::UpdateStatus(const FString& ChkLine)
{
	FString TrimLine = ChkLine.TrimStartAndEnd();
	// Just log if it's not a status update line
	if (!CheckUpdateMatch(TrimLine))
	{
		return false;
	}

	FText Status = FText::FromString(ReplaceUpdateStrs(TrimLine));
	CmdNotifier->UpdateProgress(RequirementsDone, RequirementsCount, Status);

	RequirementsDone += 1;
	RequirementsCount = FMath::Max(RequirementsCount, RequirementsDone + 1);

	// Note: Return true instead to not log status matched lines
	return false;
}

void FPipProgressParser::NotifyCompleted(bool bSuccess)
{
	CmdNotifier->Completed(bSuccess);
}


bool FPipProgressParser::CheckUpdateMatch(const FString& Line)
{
	for (const FString& ChkMatch : MatchStatusStrs)
	{
		if (Line.StartsWith(ChkMatch))
		{
			return true;
		}
	}

	return false;
}

FString FPipProgressParser::ReplaceUpdateStrs(const FString& Line)
{
	FString RepLine = Line;
	for (const TPair<FString, FString>& ReplaceMap : LogReplaceStrs)
	{
		RepLine = RepLine.Replace(*ReplaceMap.Key, *ReplaceMap.Value, ESearchCase::CaseSensitive);
	}

	return RepLine;
}

// Implementation of a tickable class for running a subprocess with status updates via ICmdProgressParser interface
class FLoggedSubprocess
{
public:
	FLoggedSubprocess(const FString& URI, const FString& Params, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser)
		: Context(Context), CmdParser(CmdParser)
	{
		bLogOutput = (Context != nullptr);

		// Create a stdout pipe for the child process
		StdOutPipeRead = nullptr;
		StdOutPipeWrite = nullptr;
		verify(FPlatformProcess::CreatePipe(StdOutPipeRead, StdOutPipeWrite));

		ProcExitCode = -1;
		ProcessHandle = FPlatformProcess::CreateProc(*URI, *Params, false, true, true, nullptr, 0, nullptr, StdOutPipeWrite, nullptr);
	}

	~FLoggedSubprocess()
	{
		FPlatformProcess::ClosePipe(StdOutPipeRead, StdOutPipeWrite);
	}

	bool IsValid() const
	{
		return ProcessHandle.IsValid();
	}

	int32 GetExitCode() const
	{
		return ProcExitCode;
	}

	bool TickSubprocess()
	{
		bool bProcessFinished = FPlatformProcess::GetProcReturnCode(ProcessHandle, &ProcExitCode);
		ParseOutput(FPlatformProcess::ReadPipe(StdOutPipeRead));

		return !bProcessFinished;
	}

	void KillSubprocess()
	{
		FPlatformProcess::TerminateProc(ProcessHandle, true);

		// Make sure to read the final bytes from the pipe before closing
		ParseOutput(FPlatformProcess::ReadPipe(StdOutPipeRead));

		// Indicate failure in the return code
		ProcExitCode = -1;
	}

private:
	void ParseOutput(const FString& NewOutput)
	{
		BufferedText += NewOutput;

		int32 EndOfLineIdx;
		while (BufferedText.FindChar(TEXT('\n'), EndOfLineIdx))
		{
			FString Line = BufferedText.Left(EndOfLineIdx);
			Line.RemoveFromEnd(TEXT("\r"), ESearchCase::CaseSensitive);

			// Always log if no output parser, also log if UpdateStatus returns false
			bool bOnlyStatusUpdate = CmdParser.IsValid() && CmdParser->UpdateStatus(Line);
			if ( bLogOutput && !bOnlyStatusUpdate)
			{
				Context->Log(LogPython.GetCategoryName(), ELogVerbosity::Log, Line);
			}

			BufferedText.MidInline(EndOfLineIdx + 1, MAX_int32, EAllowShrinking::No);
		}
	}
	
	FFeedbackContext* Context;
	TSharedPtr<ICmdProgressParser> CmdParser;

	bool bLogOutput;
	int32 ProcExitCode;
	FProcHandle ProcessHandle;
	void* StdOutPipeRead = nullptr;
	void* StdOutPipeWrite = nullptr;

	FString BufferedText;
};


bool FLoggedSubprocessSync::Run(int32& OutExitCode, const FString& URI, const FString& Params, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser)
{
	FLoggedSubprocess Subproc(URI, Params, Context, CmdParser);
	if (!Subproc.IsValid())
	{
		return false;
	}

	while (Subproc.TickSubprocess())
	{
		FPlatformProcess::Sleep(0.01f);
	}

	OutExitCode = Subproc.GetExitCode();
	return true;
}

// FLoggedSubprocessThread implementation
FLoggedSubprocessThread::FLoggedSubprocessThread(const FString& URI, const FString& Params, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser)
	: Process(URI, Params, true, true)
	, Context(Context)
	, CmdParser(CmdParser)
	, bLogOutput(Context != nullptr)
{
	Process.OnOutput().BindRaw(this, &FLoggedSubprocessThread::ParseOutput);
	Process.OnCompleted().BindRaw(this, &FLoggedSubprocessThread::HandleCompleted);
}

void FLoggedSubprocessThread::ParseOutput(FString StdoutLine) const
{
	bool bOnlyStatusUpdate = CmdParser.IsValid() && CmdParser->UpdateStatus(StdoutLine);
	if ( bLogOutput && !bOnlyStatusUpdate)
	{
		Context->Log(LogPython.GetCategoryName(), ELogVerbosity::Log, StdoutLine);
	}
}

void FLoggedSubprocessThread::HandleCompleted(int32 ReturnCode) const
{
	if (CmdParser.IsValid())
	{
		CmdParser->NotifyCompleted(ReturnCode==0);
	}
	
	OnProcCompleted.ExecuteIfBound(ReturnCode);
}


#undef LOCTEXT_NAMESPACE
