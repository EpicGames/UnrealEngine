// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPipInstall.h"
#include "Containers/UnrealString.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/MonitoredProcess.h"
#include "Templates/SharedPointer.h"

struct FSlowTask;

// Simple interface for parsing cmd output to update slowtask progress
// Similar to FFeedbackContextMarkup, but supports arbitrary line parsing
struct ICmdProgressParser
{
    virtual ~ICmdProgressParser()= default;
	// Get a total work estimate
	virtual float GetTotalWork() = 0;
    virtual float GetWorkDone() = 0;
	// Parse line and update status/progress (return true to eat the output and not log)
	virtual bool UpdateStatus(const FString& ChkLine) = 0;
	virtual void NotifyCompleted(bool bSuccess) = 0;
};

// Pip progress parser implemenation of ICmdProgressParser
class FPipProgressParser : public ICmdProgressParser
{
public:
	FPipProgressParser(int GuessRequirementsCount, TSharedRef<ICmdProgressNotifier> InCmdNotifier);
    // ICmdProgressParser methods
	virtual float GetTotalWork() override;
    virtual float GetWorkDone() override;
	virtual bool UpdateStatus(const FString& ChkLine) override;
	virtual void NotifyCompleted(bool bSuccess) override;

private:
	static bool CheckUpdateMatch(const FString& Line);
	static FString ReplaceUpdateStrs(const FString& Line);

	float RequirementsDone;
	float RequirementsCount;
    TSharedRef<ICmdProgressNotifier> CmdNotifier;

	static const TArray<FString> MatchStatusStrs;
	static const TMap<FString,FString> LogReplaceStrs;
};

class FLoggedSubprocessSync
{
public:
	static bool Run(int32& OutExitCode, const FString& URI, const FString& Params, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser);
};

// Threaded subprocess runner (converted to wrapped FMonitoredProcess)
DECLARE_DELEGATE_OneParam(FOnSubprocessThreadCompleted, int32)
class FLoggedSubprocessThread
{
public:
	FLoggedSubprocessThread(const FString& URI, const FString& Params, FFeedbackContext* Context, TSharedPtr<ICmdProgressParser> CmdParser);

	bool Launch() {return Process.Launch();}
	bool IsRunning() {return Process.Update();}
	
	FOnSubprocessThreadCompleted& OnCompleted() {return OnProcCompleted;}

private:
	void ParseOutput(FString StdoutLine) const;
	void HandleCompleted(int32 ReturnCode) const;
	
	FOnSubprocessThreadCompleted OnProcCompleted;
	
	FMonitoredProcess Process;
	FFeedbackContext* Context;
	TSharedPtr<ICmdProgressParser> CmdParser;
	const bool bLogOutput;
};
