// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyConstant.h"
#include "PyUtil.h"
#include "PyPtr.h"

#if WITH_PYTHON

bool InitializePyConstant()
{
	if (!ensure(PyType_Ready(&PyConstantDescrType) == 0))
	{
		PyUtil::LogPythonError();
		UE_LOGF(LogPython, Error, "PyType_Ready(&PyConstantDescrType) failed! Python will be disabled.")
		return false;
	}
	return true;
}

bool FPyConstantDef::AddConstantToType(FPyConstantDef* InConstant, PyTypeObject* InType)
{
	return AddConstantToDict(InConstant, InType->tp_dict);
}

bool FPyConstantDef::AddConstantsToType(FPyConstantDef* InConstants, PyTypeObject* InType)
{
	return AddConstantsToDict(InConstants, InType->tp_dict);
}

bool FPyConstantDef::AddConstantToModule(FPyConstantDef* InConstant, PyObject* InModule)
{
	if (!PyModule_Check(InModule))
	{
		return false;
	}

	PyObject* ModuleDict = PyModule_GetDict(InModule);
	return AddConstantToDict(InConstant, ModuleDict);
}

bool FPyConstantDef::AddConstantsToModule(FPyConstantDef* InConstants, PyObject* InModule)
{
	if (!PyModule_Check(InModule))
	{
		return false;
	}
	
	PyObject* ModuleDict = PyModule_GetDict(InModule);
	return AddConstantsToDict(InConstants, ModuleDict);
}

bool FPyConstantDef::AddConstantToDict(FPyConstantDef* InConstant, PyObject* InDict)
{
	FPyObjectPtr PyConstantDescr = FPyObjectPtr::StealReference((PyObject*)FPyConstantDescrObject::New(InConstant));
	if (!PyConstantDescr)
	{
		return false;
	}

	if (PyDict_SetItemString(InDict, InConstant->ConstantName, PyConstantDescr) != 0)
	{
		return false;
	}

	return true;
}

bool FPyConstantDef::AddConstantsToDict(FPyConstantDef* InConstants, PyObject* InDict)
{
	if (!PyDict_Check(InDict))
	{
		return false;
	}

	for (FPyConstantDef* ConstantDef = InConstants; ConstantDef->ConstantName; ++ConstantDef)
	{
		if (!AddConstantToDict(ConstantDef, InDict))
		{
			return false;
		}
	}

	return true;
}

FPyConstantDescrObject* FPyConstantDescrObject::New(const FPyConstantDef* InConstantDef)
{
	FPyConstantDescrObject* Self = (FPyConstantDescrObject*)PyConstantDescrType.tp_alloc(&PyConstantDescrType, 0);
	if (Self)
	{
		Self->ConstantName = PyUnicode_FromString(InConstantDef->ConstantName);
		Self->ConstantDef = InConstantDef;
	}
	return Self;
}

void FPyConstantDescrObject::Free(FPyConstantDescrObject* InSelf)
{
	Py_XDECREF(InSelf->ConstantName);
	InSelf->ConstantName = nullptr;

	InSelf->ConstantDef = nullptr;

	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

PyTypeObject InitializePyConstantDescrType()
{
	struct FFuncs
	{
		static void Dealloc(FPyConstantDescrObject* InSelf)
		{
			FPyConstantDescrObject::Free(InSelf);
		}

		static PyObject* Str(FPyConstantDescrObject* InSelf)
		{
			return PyUnicode_FromString("<built-in constant value>");
		}

		static PyObject* DescrGet(FPyConstantDescrObject* InSelf, PyObject* InObj, PyObject* InType)
		{
			return InSelf->ConstantDef->ConstantGetter((PyTypeObject*)InType, InSelf->ConstantDef->ConstantContext);
		}

		static int DescrSet(FPyConstantDescrObject* InSelf, PyObject* InObj, PyObject* InValue)
		{
			PyErr_SetString(PyExc_Exception, "Constant values are read-only");
			return -1;
		}
	};

	struct FGetSets
	{
		static PyObject* GetDoc(FPyConstantDescrObject* InSelf, void* InClosure)
		{
			if (InSelf->ConstantDef->ConstantDoc)
			{
				return PyUnicode_FromString(InSelf->ConstantDef->ConstantDoc);
			}

			Py_RETURN_NONE;
		}
	};

	static PyMemberDef PyMembers[] = {
		{ PyCStrCast("__name__"), T_OBJECT, STRUCT_OFFSET(FPyConstantDescrObject, ConstantName), READONLY, nullptr },
		{ nullptr, 0, 0, 0, nullptr }
	};

	static PyGetSetDef PyGetSets[] = {
		{ PyCStrCast("__doc__"), (getter)&FGetSets::GetDoc, nullptr, nullptr, nullptr },
		{ nullptr, nullptr, nullptr, nullptr, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"constant_value", /* tp_name */
		sizeof(FPyConstantDescrObject), /* tp_basicsize */
	};

	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_descr_get = (descrgetfunc)&FFuncs::DescrGet;
	PyType.tp_descr_set = (descrsetfunc)&FFuncs::DescrSet;
	PyType.tp_getattro = (getattrofunc)&PyObject_GenericGetAttr;

	PyType.tp_members = PyMembers;
	PyType.tp_getset = PyGetSets;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;

	return PyType;
}

PyTypeObject PyConstantDescrType = InitializePyConstantDescrType();

#endif	// WITH_PYTHON
