// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "CoreMinimal.h"

#if WITH_PYTHON

/** Python type for FPyConstantDescrObject */
extern PyTypeObject PyConstantDescrType;

/** Initialize the PyConstant types */
bool InitializePyConstant();

/** Function pointer used to convert a constant value pointer into a Python object */
typedef PyObject*(*PyConstantGetter)(PyTypeObject*, const void*);

/**
 * Definition for a constant value
 * This takes a pointer to some static or otherwise persistent data, along with a function used to convert this data into a Python object when needed
 * Compared to a template, this avoids variance in the Python-type which would require a new Python-type to be registered for each instantiation
 * Compared to storing a PyObject*, this avoids returning an instance that could be mutated and affect the constant value
 */
struct FPyConstantDef
{
	/** A pointer to the constant context */
	const void* ConstantContext;

	/** Function pointer used to convert a constant data into a Python object */
	PyConstantGetter ConstantGetter;

	/** The name of the constant value */
	const char* ConstantName;

	/** The doc string of the constant value (may be null) */
	const char* ConstantDoc;

	/** Add a singular constant to the given type */
	static bool AddConstantToType(FPyConstantDef* InConstant, PyTypeObject* InType);

	/** Add the given null-terminated table of constants to the given type */
	static bool AddConstantsToType(FPyConstantDef* InConstants, PyTypeObject* InType);

	/** Add a singular constant to the given module */
	static bool AddConstantToModule(FPyConstantDef* InConstant, PyObject* InModule);

	/** Add the given null-terminated table of constants to the given module */
	static bool AddConstantsToModule(FPyConstantDef* InConstants, PyObject* InModule);

	/** Add a singular constant to the given dict */
	static bool AddConstantToDict(FPyConstantDef* InConstant, PyObject* InDict);

	/** Add the given null-terminated table of constants to the given dict */
	static bool AddConstantsToDict(FPyConstantDef* InConstants, PyObject* InDict);
};

/**
 * Python object for the descriptor of an constant value
 */
struct FPyConstantDescrObject
{
	/** Common Python Object */
	PyObject_HEAD

	/** Name of the constant being described */
	PyObject* ConstantName;

	/** Pointer to the definition of this constant */
	const FPyConstantDef* ConstantDef;

	/** New an instance */
	static FPyConstantDescrObject* New(const FPyConstantDef* InConstantDef);

	/** Free this instance */
	static void Free(FPyConstantDescrObject* InSelf);
};

#endif	// WITH_PYTHON
