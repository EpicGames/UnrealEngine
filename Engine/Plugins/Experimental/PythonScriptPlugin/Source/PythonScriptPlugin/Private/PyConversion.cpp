// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyConversion.h"
#include "PyUtil.h"
#include "PyGenUtil.h"
#include "PyPtr.h"

#include "PyWrapperObject.h"
#include "PyWrapperStruct.h"
#include "PyWrapperEnum.h"
#include "PyWrapperDelegate.h"
#include "PyWrapperName.h"
#include "PyWrapperText.h"
#include "PyWrapperArray.h"
#include "PyWrapperFixedArray.h"
#include "PyWrapperSet.h"
#include "PyWrapperMap.h"
#include "PyWrapperFieldPath.h"
#include "PyWrapperTypeRegistry.h"

#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPortFlags.h"

#if WITH_PYTHON

#define PYCONVERSION_RETURN(RESULT, ERROR_CTX, ERROR_MSG)							\
	{																				\
		const FPyConversionResult PyConversionReturnResult_Internal = (RESULT);		\
		if (!PyConversionReturnResult_Internal)										\
		{																			\
			if (SetErrorState == ESetErrorState::Yes)								\
			{																		\
				PyUtil::SetPythonError(PyExc_TypeError, (ERROR_CTX), (ERROR_MSG));	\
			}																		\
			else																	\
			{																		\
				PyErr_Clear();														\
			}																		\
		}																			\
		return PyConversionReturnResult_Internal;									\
	}

namespace PyConversion
{

namespace Internal
{

FPyConversionResult NativizeStructInstance(PyObject* PyObj, UScriptStruct* StructType, void* StructInstance, const ESetErrorState SetErrorState)
{
	FPyConversionResult Result = FPyConversionResult::Failure();

	PyTypeObject* PyStructType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(StructType);
	FPyWrapperStructPtr PyStruct = FPyWrapperStructPtr::StealReference(FPyWrapperStruct::CastPyObject(PyObj, PyStructType, &Result));
	if (PyStruct && ensureAlways(PyStruct->ScriptStruct->IsChildOf(StructType)))
	{
		StructType->CopyScriptStruct(StructInstance, PyStruct->StructInstance);
	}

	PYCONVERSION_RETURN(Result, TEXT("NativizeStructInstance"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(PyStructType)));
}

FPyConversionResult PythonizeStructInstance(UScriptStruct* StructType, const void* StructInstance, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	OutPyObj = (PyObject*)FPyWrapperStructFactory::Get().CreateInstance(StructType, (void*)StructInstance, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
	return FPyConversionResult::Success();
}

template <typename T>
FPyConversionResult NativizeSigned(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState, const TCHAR* InErrorType)
{
	// Booleans subclass integer, so exclude those explicitly
	if (!PyBool_Check(PyObj))
	{
		if (PyLong_Check(PyObj))
		{
			OutVal = PyLong_AsLongLong(PyObj);
			return FPyConversionResult::Success();
		}

		if (PyFloat_Check(PyObj))
		{
			OutVal = PyFloat_AsDouble(PyObj);
			return FPyConversionResult::SuccessWithCoercion();
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s'"), *PyUtil::GetFriendlyTypename(PyObj), InErrorType));
}

template <typename T>
FPyConversionResult NativizeUnsigned(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState, const TCHAR* InErrorType)
{
	// Booleans subclass integer, so exclude those explicitly
	if (!PyBool_Check(PyObj))
	{
		if (PyLong_Check(PyObj))
		{
			OutVal = PyLong_AsUnsignedLongLong(PyObj);
			return FPyConversionResult::Success();
		}

		if (PyFloat_Check(PyObj))
		{
			OutVal = PyFloat_AsDouble(PyObj);
			return FPyConversionResult::SuccessWithCoercion();
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s'"), *PyUtil::GetFriendlyTypename(PyObj), InErrorType));
}

template <typename T>
FPyConversionResult NativizeReal(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState, const TCHAR* InErrorType)
{
	// Booleans subclass integer, so exclude those explicitly
	if (!PyBool_Check(PyObj))
	{
		if (PyLong_Check(PyObj))
		{
			OutVal = PyLong_AsDouble(PyObj);
			return FPyConversionResult::SuccessWithCoercion();
		}

		if (PyFloat_Check(PyObj))
		{
			OutVal = PyFloat_AsDouble(PyObj);
			return FPyConversionResult::Success();
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s'"), *PyUtil::GetFriendlyTypename(PyObj), InErrorType));
}

template <typename T>
FPyConversionResult PythonizeSigned(const T Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState, const TCHAR* InErrorType)
{
	OutPyObj = PyLong_FromLongLong(Val);
	return FPyConversionResult::Success();
}

template <typename T>
FPyConversionResult PythonizeUnsigned(const T Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState, const TCHAR* InErrorType)
{
	OutPyObj = PyLong_FromUnsignedLongLong(Val);
	return FPyConversionResult::Success();
}

template <typename T>
FPyConversionResult PythonizeReal(const T Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState, const TCHAR* InErrorType)
{
	OutPyObj = PyFloat_FromDouble(Val);
	return FPyConversionResult::Success();
}

} // namespace Internal

FPyConversionResult Nativize(PyObject* PyObj, bool& OutVal, const ESetErrorState SetErrorState)
{
	if (PyObj == Py_True)
	{
		OutVal = true;
		return FPyConversionResult::Success();
	}

	if (PyObj == Py_False)
	{
		OutVal = false;
		return FPyConversionResult::Success();
	}

	if (PyObj == Py_None)
	{
		OutVal = false;
		return FPyConversionResult::Success();
	}

	if (PyLong_Check(PyObj))
	{
		OutVal = PyLong_AsLongLong(PyObj) != 0;
		return FPyConversionResult::SuccessWithCoercion();
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'bool'"), *PyUtil::GetFriendlyTypename(PyObj)));
}

FPyConversionResult Pythonize(const bool Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	if (Val)
	{
		Py_INCREF(Py_True);
		OutPyObj = Py_True;
	}
	else
	{
		Py_INCREF(Py_False);
		OutPyObj = Py_False;
	}

	return FPyConversionResult::Success();
}

FPyConversionResult Nativize(PyObject* PyObj, int8& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeSigned(PyObj, OutVal, SetErrorState, TEXT("int8"));
}

FPyConversionResult Pythonize(const int8 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeSigned(Val, OutPyObj, SetErrorState, TEXT("int8"));
}

FPyConversionResult Nativize(PyObject* PyObj, uint8& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeUnsigned(PyObj, OutVal, SetErrorState, TEXT("uint8"));
}

FPyConversionResult Pythonize(const uint8 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeUnsigned(Val, OutPyObj, SetErrorState, TEXT("uint8"));
}

FPyConversionResult Nativize(PyObject* PyObj, int16& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeSigned(PyObj, OutVal, SetErrorState, TEXT("int16"));
}

FPyConversionResult Pythonize(const int16 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeSigned(Val, OutPyObj, SetErrorState, TEXT("int16"));
}

FPyConversionResult Nativize(PyObject* PyObj, uint16& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeUnsigned(PyObj, OutVal, SetErrorState, TEXT("uint16"));
}

FPyConversionResult Pythonize(const uint16 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeUnsigned(Val, OutPyObj, SetErrorState, TEXT("uint16"));
}

FPyConversionResult Nativize(PyObject* PyObj, int32& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeSigned(PyObj, OutVal, SetErrorState, TEXT("int32"));
}

FPyConversionResult Pythonize(const int32 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeSigned(Val, OutPyObj, SetErrorState, TEXT("int32"));
}

FPyConversionResult Nativize(PyObject* PyObj, uint32& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeUnsigned(PyObj, OutVal, SetErrorState, TEXT("uint32"));
}

FPyConversionResult Pythonize(const uint32 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeUnsigned(Val, OutPyObj, SetErrorState, TEXT("uint32"));
}

FPyConversionResult Nativize(PyObject* PyObj, int64& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeSigned(PyObj, OutVal, SetErrorState, TEXT("int64"));
}

FPyConversionResult Pythonize(const int64 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeSigned(Val, OutPyObj, SetErrorState, TEXT("int64"));
}

FPyConversionResult Nativize(PyObject* PyObj, uint64& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeUnsigned(PyObj, OutVal, SetErrorState, TEXT("uint64"));
}

FPyConversionResult Pythonize(const uint64 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeUnsigned(Val, OutPyObj, SetErrorState, TEXT("uint64"));
}

FPyConversionResult Nativize(PyObject* PyObj, float& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeReal(PyObj, OutVal, SetErrorState, TEXT("float"));
}

FPyConversionResult Pythonize(const float Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeReal(Val, OutPyObj, SetErrorState, TEXT("float"));
}

FPyConversionResult Nativize(PyObject* PyObj, double& OutVal, const ESetErrorState SetErrorState)
{
	return Internal::NativizeReal(PyObj, OutVal, SetErrorState, TEXT("double"));
}

FPyConversionResult Pythonize(const double Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return Internal::PythonizeReal(Val, OutPyObj, SetErrorState, TEXT("double"));
}

FPyConversionResult Nativize(PyObject* PyObj, FString& OutVal, const ESetErrorState SetErrorState)
{
	if (PyUnicode_Check(PyObj))
	{
		if (const char* PyUtf8Buffer = PyUnicode_AsUTF8(PyObj))
		{
			OutVal = UTF8_TO_TCHAR(PyUtf8Buffer);
			return FPyConversionResult::Success();
		}
	}

	if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperNameType) == 1)
	{
		FPyWrapperName* PyWrappedName = (FPyWrapperName*)PyObj;
		OutVal = PyWrappedName->Value.ToString();
		return FPyConversionResult::SuccessWithCoercion();
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'String'"), *PyUtil::GetFriendlyTypename(PyObj)));
}

FPyConversionResult Pythonize(const FString& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	OutPyObj = PyUnicode_FromString(TCHAR_TO_UTF8(*Val));
	return FPyConversionResult::Success();
}

FPyConversionResult Nativize(PyObject* PyObj, FName& OutVal, const ESetErrorState SetErrorState)
{
	if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperNameType) == 1)
	{
		FPyWrapperName* PyWrappedName = (FPyWrapperName*)PyObj;
		OutVal = PyWrappedName->Value;
		return FPyConversionResult::Success();
	}

	FString NameStr;
	if (Nativize(PyObj, NameStr, ESetErrorState::No))
	{
		if (NameStr.Len() >= NAME_SIZE)
		{
			PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'Name' (lenth: %d, max length: %d)"), *PyUtil::GetFriendlyTypename(PyObj), NameStr.Len(), NAME_SIZE - 1));
		}

		OutVal = *NameStr;
		return FPyConversionResult::SuccessWithCoercion();
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'Name'"), *PyUtil::GetFriendlyTypename(PyObj)));
}

FPyConversionResult Pythonize(const FName& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	OutPyObj = (PyObject*)FPyWrapperNameFactory::Get().CreateInstance(Val);
	return FPyConversionResult::Success();
}

FPyConversionResult Nativize(PyObject* PyObj, FText& OutVal, const ESetErrorState SetErrorState)
{
	if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperTextType) == 1)
	{
		FPyWrapperText* PyWrappedText = (FPyWrapperText*)PyObj;
		OutVal = PyWrappedText->Value;
		return FPyConversionResult::Success();
	}

	FString TextStr;
	if (Nativize(PyObj, TextStr, ESetErrorState::No))
	{
		OutVal = FText::AsCultureInvariant(TextStr);
		return FPyConversionResult::SuccessWithCoercion();
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'Text'"), *PyUtil::GetFriendlyTypename(PyObj)));
}

FPyConversionResult Pythonize(const FText& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	OutPyObj = (PyObject*)FPyWrapperTextFactory::Get().CreateInstance(Val);
	return FPyConversionResult::Success();
}

FPyConversionResult Nativize(PyObject* PyObj, FFieldPath& OutVal, const ESetErrorState SetErrorState)
{
	if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperFieldPathType) == 1)
	{
		FPyWrapperFieldPath* PyWrappedFieldPath = (FPyWrapperFieldPath*)PyObj;
		OutVal = PyWrappedFieldPath->Value;
		return FPyConversionResult::Success();
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'FieldPath'"), *PyUtil::GetFriendlyTypename(PyObj)));
}

FPyConversionResult Pythonize(const FFieldPath& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	OutPyObj = (PyObject*)FPyWrapperFieldPathFactory::Get().CreateInstance(Val);
	return FPyConversionResult::Success();
}


FPyConversionResult Nativize(PyObject* PyObj, void*& OutVal, const ESetErrorState SetErrorState)
{
	// CObject was removed in Python 3.2
#if !(PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 2)
	if (PyCObject_Check(PyObj))
	{
		OutVal = PyCObject_AsVoidPtr(PyObj);
		return FPyConversionResult::Success();
	}
#endif	// !(PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 2)

	// Capsule was added in Python 2.7 and 3.1
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 1
	if (PyCapsule_CheckExact(PyObj))
	{
		OutVal = PyCapsule_GetPointer(PyObj, PyCapsule_GetName(PyObj));
		return FPyConversionResult::Success();
	}
#endif	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 1

	if (PyObj == Py_None)
	{
		OutVal = nullptr;
		return FPyConversionResult::Success();
	}

	{
		uint64 PtrValue = 0;
		if (PyConversion::Nativize(PyObj, PtrValue, ESetErrorState::No))
		{
			OutVal = (void*)PtrValue;
			return FPyConversionResult::SuccessWithCoercion();
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("Nativize"), *FString::Printf(TEXT("Cannot nativize '%s' as 'void*'"), *PyUtil::GetFriendlyTypename(PyObj)));
}

FPyConversionResult Pythonize(void* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	if (Val)
	{
		// Use Capsule for Python 3.1+, and CObject for older versions
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 1
		OutPyObj = PyCapsule_New(Val, nullptr, nullptr);
#else	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 1
		OutPyObj = PyCObject_FromVoidPtr(Val, nullptr);
#endif	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 1
	}
	else
	{
		Py_INCREF(Py_None);
		OutPyObj = Py_None;
	}

	return FPyConversionResult::Success();
}

FPyConversionResult Nativize(PyObject* PyObj, UObject*& OutVal, const ESetErrorState SetErrorState)
{
	return NativizeObject(PyObj, OutVal, UObject::StaticClass(), SetErrorState);
}

FPyConversionResult Pythonize(UObject* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return PythonizeObject(Val, OutPyObj, SetErrorState);
}

FPyConversionResult NativizeObject(PyObject* PyObj, UObject*& OutVal, UClass* ExpectedType, const ESetErrorState SetErrorState)
{
	auto IsObjectExpectedType = [ExpectedType](const UObject* ObjectInstance) -> bool
	{
		if (!ExpectedType)
		{
			return true;
		}

		return ExpectedType->HasAnyClassFlags(CLASS_Interface)
			? ObjectInstance->GetClass()->ImplementsInterface(ExpectedType)
			: ObjectInstance->IsA(ExpectedType);
	};

	if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperObjectType) == 1)
	{
		FPyWrapperObject* PyWrappedObj = (FPyWrapperObject*)PyObj;
		if (FPyWrapperObject::ValidateInternalState(PyWrappedObj) && IsObjectExpectedType(PyWrappedObj->ObjectInstance))
		{
			OutVal = PyWrappedObj->ObjectInstance;
			return FPyConversionResult::Success();
		}
	}

	if (PyObj == Py_None)
	{
		OutVal = nullptr;
		return FPyConversionResult::Success();
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("NativizeObject"), *FString::Printf(TEXT("Cannot nativize '%s' as 'Object' (allowed Class type: '%s')"), *PyUtil::GetFriendlyTypename(PyObj), ExpectedType ? *ExpectedType->GetName() : TEXT("<any>")));
}

FPyConversionResult PythonizeObject(UObject* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	if (Val)
	{
		OutPyObj = (PyObject*)FPyWrapperObjectFactory::Get().CreateInstance(Val);
	}
	else
	{
		Py_INCREF(Py_None);
		OutPyObj = Py_None;
	}

	return FPyConversionResult::Success();
}

PyObject* PythonizeObject(UObject* Val, const ESetErrorState SetErrorState)
{
	PyObject* Obj = nullptr;
	PythonizeObject(Val, Obj, SetErrorState);
	return Obj;
}

FPyConversionResult NativizeClass(PyObject* PyObj, UClass*& OutVal, UClass* ExpectedType, const ESetErrorState SetErrorState)
{
	UClass* Class = nullptr;

	if (PyType_Check(PyObj) && PyType_IsSubtype((PyTypeObject*)PyObj, &PyWrapperObjectType))
	{
		Class = FPyWrapperObjectMetaData::GetClass((PyTypeObject*)PyObj);
	}

	if (Class || NativizeObject(PyObj, (UObject*&)Class, UClass::StaticClass(), SetErrorState))
	{
		if (!Class || !ExpectedType || Class->IsChildOf(ExpectedType))
		{
			OutVal = Class;
			return FPyConversionResult::Success();
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("NativizeClass"), *FString::Printf(TEXT("Cannot nativize '%s' as 'Class' (allowed Class type: '%s')"), *PyUtil::GetFriendlyTypename(PyObj), ExpectedType ? *ExpectedType->GetName() : TEXT("<any>")));
}

FPyConversionResult PythonizeClass(UClass* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return PythonizeObject(Val, OutPyObj, SetErrorState);
}

PyObject* PythonizeClass(UClass* Val, const ESetErrorState SetErrorState)
{
	PyObject* Obj = nullptr;
	PythonizeClass(Val, Obj, SetErrorState);
	return Obj;
}

FPyConversionResult NativizeStruct(PyObject* PyObj, UScriptStruct*& OutVal, UScriptStruct* ExpectedType, const ESetErrorState SetErrorState)
{
	UScriptStruct* Struct = nullptr;

	if (PyType_Check(PyObj) && PyType_IsSubtype((PyTypeObject*)PyObj, &PyWrapperStructType))
	{
		Struct = FPyWrapperStructMetaData::GetStruct((PyTypeObject*)PyObj);
	}

	if (Struct || NativizeObject(PyObj, (UObject*&)Struct, UScriptStruct::StaticClass(), SetErrorState))
	{
		if (!Struct || !ExpectedType || Struct->IsChildOf(ExpectedType))
		{
			OutVal = Struct;
			return FPyConversionResult::Success();
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("NativizeStruct"), *FString::Printf(TEXT("Cannot nativize '%s' as 'Struct' (allowed Struct type: '%s')"), *PyUtil::GetFriendlyTypename(PyObj), ExpectedType ? *ExpectedType->GetName() : TEXT("<any>")));
}

FPyConversionResult PythonizeStruct(UScriptStruct* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	return PythonizeObject(Val, OutPyObj, SetErrorState);
}

PyObject* PythonizeStruct(UScriptStruct* Val, const ESetErrorState SetErrorState)
{
	PyObject* Obj = nullptr;
	PythonizeStruct(Val, Obj, SetErrorState);
	return Obj;
}

void EnsureWrappedBlueprintEnumType(const UEnum* EnumType)
{
	if (PyGenUtil::IsBlueprintGeneratedEnum(EnumType))
	{
		FPyWrapperTypeRegistry& PyWrapperTypeRegistry = FPyWrapperTypeRegistry::Get();

		FPyWrapperTypeRegistry::FGeneratedWrappedTypeReferences GeneratedWrappedTypeReferences;
		TSet<FName> DirtyModules;

		PyWrapperTypeRegistry.GenerateWrappedTypeForObject(EnumType, GeneratedWrappedTypeReferences, DirtyModules, EPyTypeGenerationFlags::IncludeBlueprintGeneratedTypes);

		PyWrapperTypeRegistry.GenerateWrappedTypesForReferences(GeneratedWrappedTypeReferences, DirtyModules);
		PyWrapperTypeRegistry.NotifyModulesDirtied(DirtyModules);
	}
}

FPyConversionResult NativizeEnumEntry(PyObject* PyObj, const UEnum* EnumType, int64& OutVal, const ESetErrorState SetErrorState)
{
	FPyConversionResult Result = FPyConversionResult::Failure();

	EnsureWrappedBlueprintEnumType(EnumType);

	PyTypeObject* PyEnumType = FPyWrapperTypeRegistry::Get().GetWrappedEnumType(EnumType);
	FPyWrapperEnumPtr PyEnum = FPyWrapperEnumPtr::StealReference(FPyWrapperEnum::CastPyObject(PyObj, PyEnumType, &Result));
	if (PyEnum)
	{
		OutVal = FPyWrapperEnum::GetEnumEntryValue(PyEnum);
	}

	PYCONVERSION_RETURN(Result, TEXT("NativizeEnumEntry"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(PyEnumType)));
}

FPyConversionResult PythonizeEnumEntry(const int64 Val, const UEnum* EnumType, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
{
	EnsureWrappedBlueprintEnumType(EnumType);

	PyTypeObject* PyEnumType = FPyWrapperTypeRegistry::Get().GetWrappedEnumType(EnumType);
	if (const FPyWrapperEnumMetaData* PyEnumMetaData = FPyWrapperEnumMetaData::GetMetaData(PyEnumType))
	{
		// Find an enum entry using this value
		for (FPyWrapperEnum* PyEnumEntry : PyEnumMetaData->EnumEntries)
		{
			const int64 EnumEntryVal = FPyWrapperEnum::GetEnumEntryValue(PyEnumEntry);
			if (EnumEntryVal == Val)
			{
				Py_INCREF(PyEnumEntry);
				OutPyObj = (PyObject*)PyEnumEntry;
				return FPyConversionResult::Success();
			}
		}
	}

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("PythonizeEnumEntry"), *FString::Printf(TEXT("Cannot pythonize '%" INT64_FMT "' (int64) as '%s'"), Val, *PyUtil::GetFriendlyTypename(PyEnumType)));
}

PyObject* PythonizeEnumEntry(const int64 Val, const UEnum* EnumType, const ESetErrorState SetErrorState)
{
	PyObject* Obj = nullptr;
	PythonizeEnumEntry(Val, EnumType, Obj, SetErrorState);
	return Obj;
}

FPyConversionResult NativizeProperty(PyObject* PyObj, const FProperty* Prop, void* ValueAddr, const TConstArrayView<void*>& InArchetypeInstValueAddrs, const FPropertyAccessChangeNotify* InChangeNotify, const ESetErrorState SetErrorState)
{
#define PYCONVERSION_PROPERTY_RETURN(RESULT) \
	PYCONVERSION_RETURN(RESULT, TEXT("NativizeProperty"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s' (%s)"), *PyUtil::GetFriendlyTypename(PyObj), *Prop->GetName(), *Prop->GetClass()->GetName()))

	if (Prop->ArrayDim > 1)
	{
		FPyWrapperFixedArrayPtr PyFixedArray = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::CastPyObject(PyObj, &PyWrapperFixedArrayType, Prop));
		if (PyFixedArray)
		{
			EmitPropertyChangeNotifications(InChangeNotify, /*bIdenticalValue*/false, [&]()
			{
				const int32 ArrSize = FMath::Min(Prop->ArrayDim, PyFixedArray->ArrayProp->ArrayDim);
				for (int32 ArrIndex = 0; ArrIndex < ArrSize; ++ArrIndex)
				{
					for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
					{
						Prop->CopySingleValue(static_cast<uint8*>(ArchInstValueAddr) + (Prop->GetElementSize() * ArrIndex), FPyWrapperFixedArray::GetItemPtr(PyFixedArray, ArrIndex));
					}
					Prop->CopySingleValue(static_cast<uint8*>(ValueAddr) + (Prop->GetElementSize() * ArrIndex), FPyWrapperFixedArray::GetItemPtr(PyFixedArray, ArrIndex));
				}
			});
			return FPyConversionResult::Success();
		}

		PYCONVERSION_PROPERTY_RETURN(FPyConversionResult::Failure());
	}

	return NativizeProperty_Direct(PyObj, Prop, ValueAddr, InArchetypeInstValueAddrs, InChangeNotify, SetErrorState);

#undef PYCONVERSION_PROPERTY_RETURN
}

FPyConversionResult PythonizeProperty(const FProperty* Prop, const void* ValueAddr, PyObject*& OutPyObj, const EPyConversionMethod ConversionMethod, PyObject* OwnerPyObj, const ESetErrorState SetErrorState)
{
	if (Prop->ArrayDim > 1)
	{
		OutPyObj = (PyObject*)FPyWrapperFixedArrayFactory::Get().CreateInstance((void*)ValueAddr, Prop, FPyWrapperOwnerContext(OwnerPyObj, OwnerPyObj ? Prop : nullptr), ConversionMethod);
		return FPyConversionResult::Success();
	}

	return PythonizeProperty_Direct(Prop, ValueAddr, OutPyObj, ConversionMethod, OwnerPyObj, SetErrorState);
}

FPyConversionResult NativizeProperty_Direct(PyObject* PyObj, const FProperty* Prop, void* ValueAddr, const TConstArrayView<void*>& InArchetypeInstValueAddrs, const FPropertyAccessChangeNotify* InChangeNotify, const ESetErrorState SetErrorState)
{
#define PYCONVERSION_PROPERTY_RETURN(RESULT) \
	PYCONVERSION_RETURN(RESULT, TEXT("NativizeProperty"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s' (%s)"), *PyUtil::GetFriendlyTypename(PyObj), *Prop->GetName(), *Prop->GetClass()->GetName()))

#define NATIVIZE_SETTER_PROPERTY(PROPTYPE)											\
	if (const PROPTYPE* CastProp = CastField<PROPTYPE>(Prop))								\
	{																				\
		PROPTYPE::TCppType NewValue;												\
		const FPyConversionResult Result = Nativize(PyObj, NewValue, SetErrorState);\
		if (Result)																	\
		{																			\
			auto OldValue = CastProp->GetPropertyValue(ValueAddr);					\
			EmitPropertyChangeNotifications(InChangeNotify,							\
				OldValue == NewValue, [&]()											\
			{																		\
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)			\
				{																	\
					CastProp->SetPropertyValue(ArchInstValueAddr, NewValue);		\
				}																	\
				CastProp->SetPropertyValue(ValueAddr, NewValue);					\
			});																		\
		}																			\
		PYCONVERSION_PROPERTY_RETURN(Result);										\
	}

#define NATIVIZE_INLINE_PROPERTY(PROPTYPE)											\
	if (const PROPTYPE* CastProp = CastField<PROPTYPE>(Prop))								\
	{																				\
		PROPTYPE::TCppType NewValue;												\
		const FPyConversionResult Result = Nativize(PyObj, NewValue, SetErrorState);\
		if (Result)																	\
		{																			\
			auto* ValuePtr = static_cast<PROPTYPE::TCppType*>(ValueAddr);			\
			EmitPropertyChangeNotifications(InChangeNotify,							\
				CastProp->Identical(ValuePtr, &NewValue, PPF_None), [&]()			\
			{																		\
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)			\
				{																	\
					auto* ArchInstValuePtr = static_cast<PROPTYPE::TCppType*>(ArchInstValueAddr);	\
					*ArchInstValuePtr = NewValue;									\
				}																	\
				*ValuePtr = MoveTemp(NewValue);										\
			});																		\
		}																			\
		PYCONVERSION_PROPERTY_RETURN(Result);										\
	}

	NATIVIZE_SETTER_PROPERTY(FBoolProperty);
	NATIVIZE_INLINE_PROPERTY(FInt8Property);
	NATIVIZE_INLINE_PROPERTY(FInt16Property);
	NATIVIZE_INLINE_PROPERTY(FUInt16Property);
	NATIVIZE_INLINE_PROPERTY(FIntProperty);
	NATIVIZE_INLINE_PROPERTY(FUInt32Property);
	NATIVIZE_INLINE_PROPERTY(FInt64Property);
	NATIVIZE_INLINE_PROPERTY(FUInt64Property);
	NATIVIZE_INLINE_PROPERTY(FFloatProperty);
	NATIVIZE_INLINE_PROPERTY(FDoubleProperty);
	NATIVIZE_INLINE_PROPERTY(FStrProperty);
	NATIVIZE_INLINE_PROPERTY(FNameProperty);
	NATIVIZE_INLINE_PROPERTY(FTextProperty);
	NATIVIZE_INLINE_PROPERTY(FFieldPathProperty);

	if (const FByteProperty* CastProp = CastField<FByteProperty>(Prop))
	{
		uint8 NewValue = 0;
		FPyConversionResult Result = FPyConversionResult::Failure();

		if (CastProp->Enum)
		{
			int64 EnumVal = 0;
			Result = NativizeEnumEntry(PyObj, CastProp->Enum, EnumVal, SetErrorState);
			if (Result.GetState() == EPyConversionResultState::SuccessWithCoercion)
			{
				// Don't allow implicit conversion on enum properties
				Result.SetState(EPyConversionResultState::Failure);
			}
			if (Result)
			{
				NewValue = (uint8)EnumVal;
			}
		}
		else
		{
			Result = Nativize(PyObj, NewValue, SetErrorState);
		}

		if (Result)
		{
			auto* ValuePtr = static_cast<uint8*>(ValueAddr);
			EmitPropertyChangeNotifications(InChangeNotify, *ValuePtr == NewValue, [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					auto* InstValuePtr = static_cast<uint8*>(ArchInstValueAddr);
					*InstValuePtr = NewValue;
				}
				*ValuePtr = NewValue;
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FEnumProperty* CastProp = CastField<FEnumProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();

		FNumericProperty* EnumInternalProp = CastProp->GetUnderlyingProperty();
		if (EnumInternalProp)
		{
			int64 NewValue = 0;

			Result = NativizeEnumEntry(PyObj, CastProp->GetEnum(), NewValue, SetErrorState);
			if (Result.GetState() == EPyConversionResultState::SuccessWithCoercion)
			{
				// Don't allow implicit conversion on enum properties
				Result.SetState(EPyConversionResultState::Failure);
			}

			if (Result)
			{
				const int64 OldValue = EnumInternalProp->GetSignedIntPropertyValue(ValueAddr);
				EmitPropertyChangeNotifications(InChangeNotify, OldValue == NewValue, [&]()
				{
					for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
					{
						EnumInternalProp->SetIntPropertyValue(ArchInstValueAddr, NewValue);
					}
					EnumInternalProp->SetIntPropertyValue(ValueAddr, NewValue);
				});
			}
		}

		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FClassProperty* CastProp = CastField<FClassProperty>(Prop))
	{
		UClass* NewValue = nullptr;
		const FPyConversionResult Result = NativizeClass(PyObj, NewValue, CastProp->MetaClass, SetErrorState);
		if (Result)
		{
			UObject* OldValue = CastProp->GetObjectPropertyValue(ValueAddr);
			EmitPropertyChangeNotifications(InChangeNotify, OldValue == NewValue, [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->SetPropertyValue(ArchInstValueAddr, NewValue);
				}
				CastProp->SetObjectPropertyValue(ValueAddr, NewValue);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FSoftClassProperty* CastProp = CastField<FSoftClassProperty>(Prop))
	{
		UClass* NewValue = nullptr;
		const FPyConversionResult Result = NativizeClass(PyObj, NewValue, CastProp->MetaClass, SetErrorState);
		if (Result)
		{
			UObject* OldValue = CastProp->GetObjectPropertyValue(ValueAddr);
			EmitPropertyChangeNotifications(InChangeNotify, OldValue == NewValue, [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->SetObjectPropertyValue(ArchInstValueAddr, NewValue);
				}
				CastProp->SetObjectPropertyValue(ValueAddr, NewValue);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FObjectPropertyBase* CastProp = CastField<FObjectPropertyBase>(Prop))
	{
		UObject* NewValue = nullptr;
		const FPyConversionResult Result = NativizeObject(PyObj, NewValue, CastProp->PropertyClass, SetErrorState);
		if (Result)
		{
			UObject* OldValue = CastProp->GetObjectPropertyValue(ValueAddr);
			EmitPropertyChangeNotifications(InChangeNotify, OldValue == NewValue, [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->SetObjectPropertyValue(ArchInstValueAddr, NewValue);
				}
				CastProp->SetObjectPropertyValue(ValueAddr, NewValue);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FInterfaceProperty* CastProp = CastField<FInterfaceProperty>(Prop))
	{
		UObject* NewValue = nullptr;
		const FPyConversionResult Result = NativizeObject(PyObj, NewValue, CastProp->InterfaceClass, SetErrorState);
		if (Result)
		{
			UObject* OldValue = CastProp->GetPropertyValue(ValueAddr).GetObject();
			EmitPropertyChangeNotifications(InChangeNotify, OldValue == NewValue, [&]()
			{
				const FScriptInterface ScriptInterface = FScriptInterface(NewValue, NewValue ? NewValue->GetInterfaceAddress(CastProp->InterfaceClass) : nullptr);
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->SetPropertyValue(ArchInstValueAddr, ScriptInterface);
				}
				CastProp->SetPropertyValue(ValueAddr, ScriptInterface);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FStructProperty* CastProp = CastField<FStructProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();
		PyTypeObject* PyStructType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(CastProp->Struct);
		FPyWrapperStructPtr PyStruct = FPyWrapperStructPtr::StealReference(FPyWrapperStruct::CastPyObject(PyObj, PyStructType, &Result));
		if (PyStruct && ensureAlways(PyStruct->ScriptStruct->IsChildOf(CastProp->Struct)))
		{
			EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, PyStruct->StructInstance, PPF_None), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->Struct->CopyScriptStruct(ArchInstValueAddr, PyStruct->StructInstance);
				}
				CastProp->Struct->CopyScriptStruct(ValueAddr, PyStruct->StructInstance);
			});
		}
		else
		{
			// Extra function scope as we don't want PYCONVERSION_RETURN to early return and skip the PYCONVERSION_PROPERTY_RETURN below
			[&]()
			{
				PYCONVERSION_RETURN(Result, TEXT("NativizeStructInstance"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(PyStructType)));
			}();
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FDelegateProperty* CastProp = CastField<FDelegateProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();
		PyTypeObject* PyDelegateType = FPyWrapperTypeRegistry::Get().GetWrappedDelegateType(CastProp->SignatureFunction);
		FPyWrapperDelegatePtr PyDelegate = FPyWrapperDelegatePtr::StealReference(FPyWrapperDelegate::CastPyObject(PyObj, PyDelegateType, &Result));
		if (PyDelegate)
		{
			EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, PyDelegate->DelegateInstance, PPF_None), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->SetPropertyValue(ArchInstValueAddr, *PyDelegate->DelegateInstance);
				}
				CastProp->SetPropertyValue(ValueAddr, *PyDelegate->DelegateInstance);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FMulticastDelegateProperty* CastProp = CastField<FMulticastDelegateProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();
		PyTypeObject* PyDelegateType = FPyWrapperTypeRegistry::Get().GetWrappedDelegateType(CastProp->SignatureFunction);
		FPyWrapperMulticastDelegatePtr PyDelegate = FPyWrapperMulticastDelegatePtr::StealReference(FPyWrapperMulticastDelegate::CastPyObject(PyObj, PyDelegateType, &Result));
		if (PyDelegate)
		{
			EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, PyDelegate->DelegateInstance, PPF_None), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->SetMulticastDelegate(ArchInstValueAddr, *PyDelegate->DelegateInstance);
				}
				CastProp->SetMulticastDelegate(ValueAddr, *PyDelegate->DelegateInstance);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FArrayProperty* CastProp = CastField<FArrayProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();
		FPyWrapperArrayPtr PyArray = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::CastPyObject(PyObj, &PyWrapperArrayType, CastProp->Inner, &Result));
		if (PyArray)
		{
			EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, PyArray->ArrayInstance, PPF_None), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->CopySingleValue(ArchInstValueAddr, PyArray->ArrayInstance);
				}
				CastProp->CopySingleValue(ValueAddr, PyArray->ArrayInstance);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FSetProperty* CastProp = CastField<FSetProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();
		FPyWrapperSetPtr PySet = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(PyObj, &PyWrapperSetType, CastProp->ElementProp, &Result));
		if (PySet)
		{
			EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, PySet->SetInstance, PPF_None), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->CopySingleValue(ArchInstValueAddr, PySet->SetInstance);
				}
				CastProp->CopySingleValue(ValueAddr, PySet->SetInstance);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FMapProperty* CastProp = CastField<FMapProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();
		FPyWrapperMapPtr PyMap = FPyWrapperMapPtr::StealReference(FPyWrapperMap::CastPyObject(PyObj, &PyWrapperMapType, CastProp->KeyProp, CastProp->ValueProp, &Result));
		if (PyMap)
		{
			EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, PyMap->MapInstance, PPF_None), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->CopySingleValue(ArchInstValueAddr, PyMap->MapInstance);
				}
				CastProp->CopySingleValue(ValueAddr, PyMap->MapInstance);
			});
		}
		PYCONVERSION_PROPERTY_RETURN(Result);
	}

	if (const FOptionalProperty* CastProp = CastField<FOptionalProperty>(Prop))
	{
		FPyConversionResult Result = FPyConversionResult::Failure();

		if (PyObj == Py_None)
		{
			EmitPropertyChangeNotifications(InChangeNotify, !CastProp->IsSet(ValueAddr), [&]()
			{
				for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
				{
					CastProp->MarkUnset(ArchInstValueAddr);
				}
				CastProp->MarkUnset(ValueAddr);
			});

			Result = FPyConversionResult::Success();
		}
		else
		{
			PyUtil::FPropValueOnScope NewValue(PyUtil::FConstPropOnScope::ExternalReference(CastProp));
			Result = NativizeProperty_Direct(PyObj, CastProp->GetValueProperty(), CastProp->MarkSetAndGetInitializedValuePointerToReplace(NewValue.GetValue()), {}, nullptr, SetErrorState);

			if (Result)
			{
				EmitPropertyChangeNotifications(InChangeNotify, CastProp->Identical(ValueAddr, NewValue.GetValue(), PPF_None), [&]()
				{
					for (void* ArchInstValueAddr : InArchetypeInstValueAddrs)
					{
						CastProp->CopySingleValue(ArchInstValueAddr, NewValue.GetValue());
					}
					CastProp->CopySingleValue(ValueAddr, NewValue.GetValue());
				});
			}
		}

		PYCONVERSION_PROPERTY_RETURN(Result);
	}

#undef NATIVIZE_SETTER_PROPERTY
#undef NATIVIZE_INLINE_PROPERTY

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("NativizeProperty"), *FString::Printf(TEXT("Cannot nativize '%s' as '%s' (%s). %s conversion not implemented!"), *PyUtil::GetFriendlyTypename(PyObj), *Prop->GetName(), *Prop->GetClass()->GetName(), *Prop->GetClass()->GetName()));

#undef PYCONVERSION_PROPERTY_RETURN
}

FPyConversionResult PythonizeProperty_Direct(const FProperty* Prop, const void* ValueAddr, PyObject*& OutPyObj, const EPyConversionMethod ConversionMethod, PyObject* OwnerPyObj, const ESetErrorState SetErrorState)
{
#define PYCONVERSION_PROPERTY_RETURN(RESULT) \
	PYCONVERSION_RETURN(RESULT, TEXT("PythonizeProperty"), *FString::Printf(TEXT("Cannot pythonize '%s' (%s)"), *Prop->GetName(), *Prop->GetClass()->GetName()))

	const FPyWrapperOwnerContext OwnerContext(OwnerPyObj, OwnerPyObj ? Prop : nullptr);
	OwnerContext.AssertValidConversionMethod(ConversionMethod);

#define PYTHONIZE_GETTER_PROPERTY(PROPTYPE)											\
	if (const PROPTYPE* CastProp = CastField<PROPTYPE>(Prop))										\
	{																				\
		auto&& Value = CastProp->GetPropertyValue(ValueAddr);						\
		PYCONVERSION_PROPERTY_RETURN(Pythonize(Value, OutPyObj, SetErrorState));	\
	}

	PYTHONIZE_GETTER_PROPERTY(FBoolProperty);
	PYTHONIZE_GETTER_PROPERTY(FInt8Property);
	PYTHONIZE_GETTER_PROPERTY(FInt16Property);
	PYTHONIZE_GETTER_PROPERTY(FUInt16Property);
	PYTHONIZE_GETTER_PROPERTY(FIntProperty);
	PYTHONIZE_GETTER_PROPERTY(FUInt32Property);
	PYTHONIZE_GETTER_PROPERTY(FInt64Property);
	PYTHONIZE_GETTER_PROPERTY(FUInt64Property);
	PYTHONIZE_GETTER_PROPERTY(FFloatProperty);
	PYTHONIZE_GETTER_PROPERTY(FDoubleProperty);
	PYTHONIZE_GETTER_PROPERTY(FStrProperty);
	PYTHONIZE_GETTER_PROPERTY(FNameProperty);
	PYTHONIZE_GETTER_PROPERTY(FTextProperty);
	PYTHONIZE_GETTER_PROPERTY(FFieldPathProperty);

	if (const FByteProperty* CastProp = CastField<FByteProperty>(Prop))
	{
		const uint8 Value = CastProp->GetPropertyValue(ValueAddr);
		if (CastProp->Enum)
		{
			PYCONVERSION_PROPERTY_RETURN(PythonizeEnumEntry((int64)Value, CastProp->Enum, OutPyObj, SetErrorState));
		}
		else
		{
			PYCONVERSION_PROPERTY_RETURN(Pythonize(Value, OutPyObj, SetErrorState));
		}
	}

	if (const FEnumProperty* CastProp = CastField<FEnumProperty>(Prop))
	{
		FNumericProperty* EnumInternalProp = CastProp->GetUnderlyingProperty();
		PYCONVERSION_PROPERTY_RETURN(PythonizeEnumEntry(EnumInternalProp ? EnumInternalProp->GetSignedIntPropertyValue(ValueAddr) : 0, CastProp->GetEnum(), OutPyObj, SetErrorState));
	}

	if (const FClassProperty* CastProp = CastField<FClassProperty>(Prop))
	{
		UClass* Value = Cast<UClass>(CastProp->LoadObjectPropertyValue(ValueAddr));
		PYCONVERSION_PROPERTY_RETURN(PythonizeClass(Value, OutPyObj, SetErrorState));
	}

	if (const FSoftClassProperty* CastProp = CastField<FSoftClassProperty>(Prop))
	{
		UClass* Value = Cast<UClass>(CastProp->LoadObjectPropertyValue(ValueAddr));
		PYCONVERSION_PROPERTY_RETURN(PythonizeClass(Value, OutPyObj, SetErrorState));
	}

	if (const FObjectPropertyBase* CastProp = CastField<FObjectPropertyBase>(Prop))
	{
		UObject* Value = CastProp->LoadObjectPropertyValue(ValueAddr);
		PYCONVERSION_PROPERTY_RETURN(Pythonize(Value, OutPyObj, SetErrorState));
	}

	if (const FInterfaceProperty* CastProp = CastField<FInterfaceProperty>(Prop))
	{
		UObject* Value = CastProp->GetPropertyValue(ValueAddr).GetObject();
		if (Value)
		{
			OutPyObj = (PyObject*)FPyWrapperObjectFactory::Get().CreateInstance(CastProp->InterfaceClass, Value);
		}
		else
		{
			Py_INCREF(Py_None);
			OutPyObj = Py_None;
		}
		return FPyConversionResult::Success();
	}

	if (const FStructProperty* CastProp = CastField<FStructProperty>(Prop))
	{
		OutPyObj = (PyObject*)FPyWrapperStructFactory::Get().CreateInstance(CastProp->Struct, (void*)ValueAddr, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}

	if (const FDelegateProperty* CastProp = CastField<FDelegateProperty>(Prop))
	{
		const FScriptDelegate* Value = CastProp->GetPropertyValuePtr(ValueAddr);
		OutPyObj = (PyObject*)FPyWrapperDelegateFactory::Get().CreateInstance(CastProp->SignatureFunction, (FScriptDelegate*)Value, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}

	if (const FMulticastDelegateProperty* CastProp = CastField<FMulticastDelegateProperty>(Prop))
	{
		const FMulticastScriptDelegate* Value = CastProp->GetMulticastDelegate(ValueAddr);
		OutPyObj = (PyObject*)FPyWrapperMulticastDelegateFactory::Get().CreateInstance(CastProp->SignatureFunction, (FMulticastScriptDelegate*)Value, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}

	// TODO: We're just not handling sparse delegates at this time
	/*if (const FMulticastSparseDelegateProperty* CastProp = CastField<FMulticastSparseDelegateProperty>(Prop))
	{
		const FMulticastScriptDelegate* Value = CastProp->GetMulticastDelegate(ValueAddr);
		OutPyObj = (PyObject*)FPyWrapperMulticastDelegateFactory::Get().CreateInstance(CastProp->SignatureFunction, (FMulticastScriptDelegate*)Value, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}*/

	if (const FArrayProperty* CastProp = CastField<FArrayProperty>(Prop))
	{
		OutPyObj = (PyObject*)FPyWrapperArrayFactory::Get().CreateInstance((void*)ValueAddr, CastProp, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}

	if (const FSetProperty* CastProp = CastField<FSetProperty>(Prop))
	{
		OutPyObj = (PyObject*)FPyWrapperSetFactory::Get().CreateInstance((void*)ValueAddr, CastProp, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}

	if (const FMapProperty* CastProp = CastField<FMapProperty>(Prop))
	{
		OutPyObj = (PyObject*)FPyWrapperMapFactory::Get().CreateInstance((void*)ValueAddr, CastProp, OwnerContext, ConversionMethod);
		return FPyConversionResult::Success();
	}

	if (const FOptionalProperty* CastProp = CastField<FOptionalProperty>(Prop))
	{
		if (CastProp->IsSet(ValueAddr))
		{
			PYCONVERSION_PROPERTY_RETURN(PythonizeProperty_Direct(CastProp->GetValueProperty(), CastProp->GetValuePointerForRead(ValueAddr), OutPyObj, ConversionMethod, OwnerPyObj, SetErrorState));
		}
		else
		{
			Py_INCREF(Py_None);
			OutPyObj = Py_None;
			return FPyConversionResult::Success();
		}
	}

#undef PYTHONIZE_GETTER_PROPERTY

	PYCONVERSION_RETURN(FPyConversionResult::Failure(), TEXT("PythonizeProperty"), *FString::Printf(TEXT("Cannot pythonize '%s' (%s). %s conversion not implemented!"), *Prop->GetName(), *Prop->GetClass()->GetName(), *Prop->GetClass()->GetName()));

#undef PYCONVERSION_PROPERTY_RETURN
}

FPyConversionResult NativizeProperty_InContainer(PyObject* PyObj, const FProperty* Prop, void* BaseAddr, const int32 ArrayIndex, const TConstArrayView<void*>& InArchetypeInstBaseAddrs, const FPropertyAccessChangeNotify* InChangeNotify, const ESetErrorState SetErrorState)
{
	TArray<void*> ArchetypeInstValueAddrs;
	ArchetypeInstValueAddrs.Reserve(InArchetypeInstBaseAddrs.Num());
	for (void* ContainerAddr : InArchetypeInstBaseAddrs)
	{
		ArchetypeInstValueAddrs.Add(Prop->ContainerPtrToValuePtr<void>(ContainerAddr, ArrayIndex));
	}

	check(ArrayIndex < Prop->ArrayDim);
	return NativizeProperty(PyObj, Prop, Prop->ContainerPtrToValuePtr<void>(BaseAddr, ArrayIndex), ArchetypeInstValueAddrs, InChangeNotify, SetErrorState);
}

FPyConversionResult PythonizeProperty_InContainer(const FProperty* Prop, const void* BaseAddr, const int32 ArrayIndex, PyObject*& OutPyObj, const EPyConversionMethod ConversionMethod, PyObject* OwnerPyObj, const ESetErrorState SetErrorState)
{
	check(ArrayIndex < Prop->ArrayDim);
	return PythonizeProperty(Prop, Prop->ContainerPtrToValuePtr<void>(BaseAddr, ArrayIndex), OutPyObj, ConversionMethod, OwnerPyObj, SetErrorState);
}

void EmitPropertyChangeNotifications(const FPropertyAccessChangeNotify* InChangeNotify, const bool bIdenticalValue, const TFunctionRef<void()>& InDoChangeFunc)
{
	Py_BEGIN_ALLOW_THREADS
	PropertyAccessUtil::EmitPreChangeNotify(InChangeNotify, bIdenticalValue);
	if (!bIdenticalValue)
	{
		InDoChangeFunc();
	}
	PropertyAccessUtil::EmitPostChangeNotify(InChangeNotify, bIdenticalValue);
	Py_END_ALLOW_THREADS
}

}

#undef PYCONVERSION_RETURN

#endif	// WITH_PYTHON
