// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyConversionMethod.h"
#include "PyConversionResult.h"
#include "PyWrapperOwnerContext.h"
#include "UObject/Class.h"
#include "Templates/EnableIf.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/PointerIsConvertibleFromTo.h"

#if WITH_PYTHON

/**
 * Conversion between native and Python types.
 * @note These functions may set error state when using ESetErrorState::Yes.
 */
namespace PyConversion
{
	enum class ESetErrorState : uint8 { No, Yes };

	/** bool overload */
	FPyConversionResult Nativize(PyObject* PyObj, bool& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const bool Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** int8 overload */
	FPyConversionResult Nativize(PyObject* PyObj, int8& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const int8 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** uint8 overload */
	FPyConversionResult Nativize(PyObject* PyObj, uint8& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const uint8 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** int16 overload */
	FPyConversionResult Nativize(PyObject* PyObj, int16& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const int16 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** uint16 overload */
	FPyConversionResult Nativize(PyObject* PyObj, uint16& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const uint16 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** int32 overload */
	FPyConversionResult Nativize(PyObject* PyObj, int32& OutVal, const ESetErrorState ErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const int32 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** uint32 overload */
	FPyConversionResult Nativize(PyObject* PyObj, uint32& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const uint32 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** int64 overload */
	FPyConversionResult Nativize(PyObject* PyObj, int64& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const int64 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** uint64 overload */
	FPyConversionResult Nativize(PyObject* PyObj, uint64& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const uint64 Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** float overload */
	FPyConversionResult Nativize(PyObject* PyObj, float& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const float Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** double overload */
	FPyConversionResult Nativize(PyObject* PyObj, double& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const double Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** FString overload */
	FPyConversionResult Nativize(PyObject* PyObj, FString& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const FString& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** FName overload */
	FPyConversionResult Nativize(PyObject* PyObj, FName& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const FName& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** FText overload */
	FPyConversionResult Nativize(PyObject* PyObj, FText& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const FText& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** FFieldPath overload */
	FPyConversionResult Nativize(PyObject* PyObj, FFieldPath& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(const FFieldPath& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** void* overload */
	FPyConversionResult Nativize(PyObject* PyObj, void*& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(void* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** UObject overload */
	FPyConversionResult Nativize(PyObject* PyObj, UObject*& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult Pythonize(UObject* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** Conversion for object types, including optional type checking */
	FPyConversionResult NativizeObject(PyObject* PyObj, UObject*& OutVal, UClass* ExpectedType, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeObject(UObject* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	PyObject* PythonizeObject(UObject* Val, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** Conversion for class types, including optional type checking */
	FPyConversionResult NativizeClass(PyObject* PyObj, UClass*& OutVal, UClass* ExpectedType, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeClass(UClass* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	PyObject* PythonizeClass(UClass* Val, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** Conversion for struct types, including optional type checking */
	FPyConversionResult NativizeStruct(PyObject* PyObj, UScriptStruct*& OutVal, UScriptStruct* ExpectedType, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeStruct(UScriptStruct* Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	PyObject* PythonizeStruct(UScriptStruct* Val, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** Conversion for enum entries */
	FPyConversionResult NativizeEnumEntry(PyObject* PyObj, const UEnum* EnumType, int64& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeEnumEntry(const int64 Val, const UEnum* EnumType, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	PyObject* PythonizeEnumEntry(const int64 Val, const UEnum* EnumType, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	namespace Internal
	{
		/** Internal version of NativizeStructInstance/PythonizeStructInstance that work on the type-erased data */
		FPyConversionResult NativizeStructInstance(PyObject* PyObj, UScriptStruct* StructType, void* StructInstance, const ESetErrorState SetErrorState);
		FPyConversionResult PythonizeStructInstance(UScriptStruct* StructType, const void* StructInstance, PyObject*& OutPyObj, const ESetErrorState SetErrorState);

		/** Dummy catch-all for type conversions that aren't yet implemented */
		template <typename T, typename Spec = void>
		struct FTypeConv
		{
			static FPyConversionResult Nativize(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState)
			{
				ensureAlwaysMsgf(false, TEXT("Nativize not implemented for type"));
				return FPyConversionResult::Failure();
			}
			
			static FPyConversionResult Pythonize(const T& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
			{
				ensureAlwaysMsgf(false, TEXT("Pythonize not implemented for type"));
				return FPyConversionResult::Failure();
			}
		};

		/** Override the catch-all for UObject types */
		template <typename T>
		struct FTypeConv<T, typename TEnableIf<TPointerIsConvertibleFromTo<typename TRemovePointer<T>::Type, UObject>::Value>::Type>
		{
			static FPyConversionResult Nativize(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState)
			{
				return PyConversion::NativizeObject(PyObj, (UObject*&)OutVal, TRemovePointer<T>::Type::StaticClass(), SetErrorState);
			}

			static FPyConversionResult Pythonize(const T& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState)
			{
				return PyConversion::PythonizeObject((UObject*)Val, OutPyObj, SetErrorState);
			}
		};
	}

	/**
	 * Generic version of Nativize used if there is no matching overload.
	 * Used to allow conversion from object and struct types that don't match a specific override (see FTypeConv).
	 */
	template <typename T>
	FPyConversionResult Nativize(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		return Internal::FTypeConv<T>::Nativize(PyObj, OutVal, SetErrorState);
	}

	/**
	 * Generic version of Pythonize used if there is no matching overload.
	 * Used to allow conversion from object and struct types that don't match a specific override (see FTypeConv).
	 */
	template <typename T>
	FPyConversionResult Pythonize(const T& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		return Internal::FTypeConv<T>::Pythonize(Val, OutPyObj, SetErrorState);
	}

	/** Generic version of Pythonize that returns a PyObject rather than a bool */
	template <typename T>
	PyObject* Pythonize(const T& Val, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		PyObject* Obj = nullptr;
		Pythonize(Val, Obj, SetErrorState);
		return Obj;
	}

	/** Conversion for known struct types */
	template <typename T>
	FPyConversionResult NativizeStructInstance(PyObject* PyObj, T& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		return Internal::NativizeStructInstance(PyObj, TBaseStructure<T>::Get(), &OutVal, SetErrorState);
	}

	/** Conversion for known struct types */
	template <typename T>
	FPyConversionResult PythonizeStructInstance(const T& Val, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		return Internal::PythonizeStructInstance(TBaseStructure<T>::Get(), &Val, OutPyObj, SetErrorState);
	}

	/** Conversion for known struct types that returns a PyObject rather than a bool */
	template <typename T>
	PyObject* PythonizeStructInstance(const T& Val, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		PyObject* Obj = nullptr;
		Internal::PythonizeStructInstance(TBaseStructure<T>::Get(), &Val, Obj, SetErrorState);
		return Obj;
	}

	/** Conversion for known enum types */
	template <typename T>
	FPyConversionResult NativizeEnumEntry(PyObject* PyObj, const UEnum* EnumType, T& OutVal, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		int64 OutTmpVal = 0;
		FPyConversionResult Result = NativizeEnumEntry(PyObj, EnumType, OutTmpVal, SetErrorState);
		if (Result)
		{
			OutVal = (T)OutTmpVal;
		}
		return Result;
	}

	/** Conversion for known enum types */
	template <typename T>
	FPyConversionResult PythonizeEnumEntry(const T& Val, const UEnum* EnumType, PyObject*& OutPyObj, const ESetErrorState SetErrorState = ESetErrorState::Yes)
	{
		const int64 TmpVal = (int64)Val;
		return PythonizeEnumEntry(TmpVal, EnumType, OutPyObj, SetErrorState);
	}

	/** Conversion for property instances (including fixed arrays) - ValueAddr should point to the property data */
	FPyConversionResult NativizeProperty(PyObject* PyObj, const FProperty* Prop, void* ValueAddr, const TConstArrayView<void*>& ArchetypeInstValueAddrs = TArray<void*>(), const FPropertyAccessChangeNotify* InChangeNotify = nullptr, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeProperty(const FProperty* Prop, const void* ValueAddr, PyObject*& OutPyObj, const EPyConversionMethod ConversionMethod = EPyConversionMethod::Copy, PyObject* OwnerPyObj = nullptr, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** Conversion for single property instances - ValueAddr should point to the property data */
	FPyConversionResult NativizeProperty_Direct(PyObject* PyObj, const FProperty* Prop, void* ValueAddr, const TConstArrayView<void*>& ArchetypeInstValueAddrs = TArray<void*>(), const FPropertyAccessChangeNotify* InChangeNotify = nullptr, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeProperty_Direct(const FProperty* Prop, const void* ValueAddr, PyObject*& OutPyObj, const EPyConversionMethod ConversionMethod = EPyConversionMethod::Copy, PyObject* OwnerPyObj = nullptr, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/** Conversion for property instances within a structure (including fixed arrays) - BaseAddr should point to the structure data */
	FPyConversionResult NativizeProperty_InContainer(PyObject* PyObj, const FProperty* Prop, void* BaseAddr, const int32 ArrayIndex, const TConstArrayView<void*>& ArchetypeInstBaseAddrs = TArray<void*>(), const FPropertyAccessChangeNotify* InChangeNotify = nullptr, const ESetErrorState SetErrorState = ESetErrorState::Yes);
	FPyConversionResult PythonizeProperty_InContainer(const FProperty* Prop, const void* BaseAddr, const int32 ArrayIndex, PyObject*& OutPyObj, const EPyConversionMethod ConversionMethod = EPyConversionMethod::Copy, PyObject* OwnerPyObj = nullptr, const ESetErrorState SetErrorState = ESetErrorState::Yes);

	/**
	 * Helper function used to emit property change notifications as value changes are made
	 * This function should be called when you know the value will actually change (or know you want to emit the notifications for it changing) and will do 
	 * the pre-change notify, invoke the passed delegate to perform the change, then do the post-change notify
	 */
	void EmitPropertyChangeNotifications(const FPropertyAccessChangeNotify* InChangeNotify, const bool bIdenticalValue, const TFunctionRef<void()>& InDoChangeFunc);
}

#endif	// WITH_PYTHON
