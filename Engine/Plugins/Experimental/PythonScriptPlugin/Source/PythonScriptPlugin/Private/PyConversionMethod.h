// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "CoreMinimal.h"

#if WITH_PYTHON

enum class EPyConversionMethod : uint8
{
	/** Copy the value */
	Copy,
	/** Steal the value (or fallback to Copy) */
	Steal,
	/** Reference the value from the given owner (or fallback to Copy) */
	Reference,
};

FORCEINLINE void AssertValidPyConversionOwner(PyObject* InPyOwner, const EPyConversionMethod InMethod)
{
	checkf(InPyOwner || InMethod != EPyConversionMethod::Reference, TEXT("EPyConversionMethod::Reference requires a valid owner object"));
}

#endif	// WITH_PYTHON
