// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if WITH_PYTHON

/** States that can be applied to a Python conversion result */
enum class EPyConversionResultState : uint8
{
	/** Conversion failed */
	Failure,
	/** Conversion succeeded */
	Success,
	/** Conversion succeeded, but type coercion occurred */
	SuccessWithCoercion,
};

/** The result of attempting a Python conversion */
class FPyConversionResult
{
public:
	/** Default constructor */
	FPyConversionResult()
		: State(EPyConversionResultState::Failure)
	{
	}

	/** Construct from a specific state */
	explicit FPyConversionResult(const EPyConversionResultState InState)
		: State(InState)
	{
	}

	/** Factory for a result set to the Failure state */
	FORCEINLINE static FPyConversionResult Failure()
	{
		return FPyConversionResult(EPyConversionResultState::Failure);
	}

	/** Factory for a result set to the Success state */
	FORCEINLINE static FPyConversionResult Success()
	{
		return FPyConversionResult(EPyConversionResultState::Success);
	}

	/** Factory for a result set to the SuccessWithCoercion state */
	FORCEINLINE static FPyConversionResult SuccessWithCoercion()
	{
		return FPyConversionResult(EPyConversionResultState::SuccessWithCoercion);
	}

	/** Is this result in a successful state? */
	explicit operator bool() const
	{
		return Succeeded();
	}

	/** Is this result in a successful state? */
	bool Succeeded() const
	{
		return State != EPyConversionResultState::Failure;
	}

	/** Is this result in a failure state? */
	bool Failed() const
	{
		return State == EPyConversionResultState::Failure;
	}

	/** Get the current result state */
	EPyConversionResultState GetState() const
	{
		return State;
	}

	/** Set the result state */
	void SetState(const EPyConversionResultState InState)
	{
		State = InState;
	}

private:
	/** Current state of this result */
	EPyConversionResultState State;
};

/** Helper function to set the value of an optional conversion result */
FORCEINLINE void SetOptionalPyConversionResult(const FPyConversionResult& InResult, FPyConversionResult* OutResult)
{
	if (OutResult)
	{
		*OutResult = InResult;
	}
}

#endif	// WITH_PYTHON
