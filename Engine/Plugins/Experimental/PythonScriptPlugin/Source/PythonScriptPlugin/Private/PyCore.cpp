// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyCore.h"
#include "PyGIL.h"
#include "PyGenUtil.h"
#include "PyReferenceCollector.h"
#include "PyWrapperTypeRegistry.h"
#include "IPythonScriptPlugin.h"

#include "PyWrapperBase.h"
#include "PyWrapperObject.h"
#include "PyWrapperStruct.h"
#include "PyWrapperEnum.h"
#include "PyWrapperDelegate.h"
#include "PyWrapperName.h"
#include "PyWrapperText.h"
#include "PyWrapperArray.h"
#include "PyWrapperFixedArray.h"
#include "PyWrapperSet.h"
#include "PyWrapperMap.h"
#include "PyWrapperFieldPath.h"
#include "PyWrapperMath.h"

#include "PythonScriptPlugin.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/ObjectRedirector.h"
#include "Misc/SlowTask.h"
#include "Misc/PackageName.h"
#include "Misc/OutputDeviceRedirector.h"

#if WITH_PYTHON

namespace PyCoreUtil
{

TArray<FTSTicker::FDelegateHandle> PythonTickerCallbackHandles;

FPyTickerDelegateHandle* RegisterTickerCallback(const float InDelay, PyObject* InPyCallable)
{
	// We copy the PyCallable into the lambda to keep the Python object alive as long as the delegate is bound
	FTSTicker::FDelegateHandle TickerDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("PythonTicker"), InDelay, [PyCallable = FPyAutoGILPtr(FPyObjectPtr::NewReference(InPyCallable))](const float InDeltaTime) mutable
	{
		if (!PyUtil::CanTickPython())
		{
			return true;
		}

		FPyScopedGIL GIL;

		FPyObjectPtr PyArgs = FPyObjectPtr::StealReference(PyTuple_New(1));
		PyTuple_SetItem(PyArgs, 0, PyConversion::Pythonize(InDeltaTime)); // SetItem steals the reference

		bool bContinue = true;
		FPyObjectPtr Result = FPyObjectPtr::StealReference(PyObject_CallObject(PyCallable.Get().GetPtr(), PyArgs));
		if (Result)
		{
			PyConversion::Nativize(Result, bContinue, PyConversion::ESetErrorState::No);
		}
		else
		{
			PyUtil::LogPythonError();
		}

		return bContinue;
	});

	if (TickerDelegateHandle.IsValid())
	{
		PythonTickerCallbackHandles.Add(TickerDelegateHandle);
	}

	return FPyTickerDelegateHandle::CreateInstance(TickerDelegateHandle);
}

bool UnregisterTickerCallback(PyObject* InCallbackHandle)
{
	FPyTickerDelegateHandlePtr PyTickerDelegateHandle = FPyTickerDelegateHandlePtr::StealReference(FPyTickerDelegateHandle::CastPyObject(InCallbackHandle));
	if (!PyTickerDelegateHandle)
	{
		return false;
	}

	FTSTicker::FDelegateHandle& TickerDelegateHandle = PyTickerDelegateHandle->Value;
	if (TickerDelegateHandle.IsValid())
	{
		PythonTickerCallbackHandles.Remove(TickerDelegateHandle);
		FTSTicker::RemoveTicker(TickerDelegateHandle);
	}

	return true;
}

void ClearTickerCallbacks()
{
	if (PythonTickerCallbackHandles.Num() > 0)
	{
		FTSTicker& CoreTicker = FTSTicker::GetCoreTicker();
		for (const FTSTicker::FDelegateHandle& Handle : PythonTickerCallbackHandles)
		{
			CoreTicker.RemoveTicker(Handle);
		}
		PythonTickerCallbackHandles.Empty();
	}
}

bool ConvertOptionalString(PyObject* InObj, FString& OutString, const TCHAR* InErrorCtxt, const TCHAR* InErrorMsg)
{
	OutString.Reset();
	if (InObj && InObj != Py_None)
	{
		if (!PyConversion::Nativize(InObj, OutString))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, InErrorMsg);
			return false;
		}
	}
	return true;
}

bool ConvertOptionalFunctionFlag(PyObject* InObj, EPyUFunctionDefFlags& OutFlags, const EPyUFunctionDefFlags InTrueFlagBit, const EPyUFunctionDefFlags InFalseFlagBit, const TCHAR* InErrorCtxt, const TCHAR* InErrorMsg)
{
	if (InObj && InObj != Py_None)
	{
		bool bFlagValue = false;
		if (!PyConversion::Nativize(InObj, bFlagValue))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, InErrorMsg);
			return false;
		}
		OutFlags |= (bFlagValue ? InTrueFlagBit : InFalseFlagBit);
	}
	return true;
}

void ApplyMetaData(PyObject* InMetaData, const TFunctionRef<void(const FString&, const FString&)>& InPredicate)
{
	if (PyDict_Check(InMetaData))
	{
		PyObject* MetaDataKey = nullptr;
		PyObject* MetaDataValue = nullptr;
		Py_ssize_t MetaDataIndex = 0;
		while (PyDict_Next(InMetaData, &MetaDataIndex, &MetaDataKey, &MetaDataValue))
		{
			const FString MetaDataKeyStr = PyUtil::PyObjectToUEString(MetaDataKey);
			const FString MetaDataValueStr = PyUtil::PyObjectToUEString(MetaDataValue);
			InPredicate(MetaDataKeyStr, MetaDataValueStr);
		}
	}
}

} // namespace PyCoreUtil

TStrongObjectPtr<UPackage>& GetPythonTypeContainerSingleton()
{
	static TStrongObjectPtr<UPackage> PythonTypeContainer;
	return PythonTypeContainer;
}

UObject* GetPythonTypeContainer()
{
	return GetPythonTypeContainerSingleton().Get();
}

UObjectRedirector* CreatePythonTypeLegacyRedirector(const FString& ShortName, const FTopLevelAssetPath& GeneratedPathName)
{
	UObject* RedirectorOuter = GetPythonTypeContainer();
	UObject* ExistingObject = FindObject<UObject>(RedirectorOuter, *ShortName);
	UObjectRedirector* Redirector = Cast<UObjectRedirector>(ExistingObject);

	// If the generated path name matches the redirector path name, then delete any existing redirector as we're about to create an object with that name
	if (FTopLevelAssetPath(RedirectorOuter->GetFName(), *ShortName) == GeneratedPathName)
	{
		if (Redirector)
		{
			Redirector->DestinationObject = nullptr;
			Redirector->ClearFlags(RF_Public | RF_Standalone);
			Redirector->Rename(*FString::Printf(TEXT("%s_DELETED"), *ShortName), nullptr, REN_DontCreateRedirectors);
		}
		return nullptr;
	}
	
	// Otherwise, create a redirector if one doesn't already exist
	// The caller is responsible for setting DestinationObject
	if (!Redirector && !ExistingObject)
	{
		Redirector = NewObject<UObjectRedirector>(RedirectorOuter, *ShortName, RF_Public | RF_Transient);
	}
	return Redirector;
}


FPyDelegateHandle* FPyDelegateHandle::CreateInstance(const FDelegateHandle& InValue)
{
	FPyDelegateHandlePtr NewInstance = FPyDelegateHandlePtr::StealReference(FPyDelegateHandle::New(&PyDelegateHandleType));
	if (NewInstance)
	{
		if (FPyDelegateHandle::Init(NewInstance, InValue) != 0)
		{
			PyUtil::LogPythonError();
			return nullptr;
		}
	}
	return NewInstance.Release();
}

FPyDelegateHandle* FPyDelegateHandle::CastPyObject(PyObject* InPyObject)
{
	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyDelegateHandleType) == 1)
	{
		Py_INCREF(InPyObject);
		return (FPyDelegateHandle*)InPyObject;
	}

	return nullptr;
}


FPyTickerDelegateHandle* FPyTickerDelegateHandle::CreateInstance(const FTSTicker::FDelegateHandle& InValue)
{
	FPyTickerDelegateHandlePtr NewInstance = FPyTickerDelegateHandlePtr::StealReference(FPyTickerDelegateHandle::New(&PyTickerDelegateHandleType));
	if (NewInstance)
	{
		if (FPyTickerDelegateHandle::Init(NewInstance, InValue) != 0)
		{
			PyUtil::LogPythonError();
			return nullptr;
		}
	}
	return NewInstance.Release();
}

FPyTickerDelegateHandle* FPyTickerDelegateHandle::CastPyObject(PyObject* InPyObject)
{
	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyTickerDelegateHandleType) == 1)
	{
		Py_INCREF(InPyObject);
		return (FPyTickerDelegateHandle*)InPyObject;
	}

	return nullptr;
}


FPyScopedSlowTask* FPyScopedSlowTask::New(PyTypeObject* InType)
{
	FPyScopedSlowTask* Self = (FPyScopedSlowTask*)InType->tp_alloc(InType, 0);
	if (Self)
	{
		Self->SlowTask = nullptr;
	}
	return Self;
}

void FPyScopedSlowTask::Free(FPyScopedSlowTask* InSelf)
{
	Deinit(InSelf);

	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

int FPyScopedSlowTask::Init(FPyScopedSlowTask* InSelf, const float InAmountOfWork, const FText& InDefaultMessage, const bool InEnabled)
{
	Deinit(InSelf);

	InSelf->SlowTask = new FSlowTask(InAmountOfWork, InDefaultMessage, InEnabled);

	return 0;
}

void FPyScopedSlowTask::Deinit(FPyScopedSlowTask* InSelf)
{
	delete InSelf->SlowTask;
	InSelf->SlowTask = nullptr;
}

bool FPyScopedSlowTask::ValidateInternalState(FPyScopedSlowTask* InSelf)
{
	if (!InSelf->SlowTask)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - SlowTask is null!"));
		return false;
	}

	return true;
}


template <typename ObjectType, typename SelfType>
bool PyTypeIterator_PassesFilter(SelfType* InSelf)
{
	FPersistentThreadSafeObjectIterator& Iter = *InSelf->Iterator;
	ObjectType* IterObj = CastChecked<ObjectType>(*Iter);
	return !InSelf->IteratorFilter || IterObj->IsChildOf(InSelf->IteratorFilter);
}


bool FPyClassIterator::PassesFilter(FPyClassIterator* InSelf)
{
	return PyTypeIterator_PassesFilter<UClass, FPyClassIterator>(InSelf);
}

UClass* FPyClassIterator::ExtractFilter(FPyClassIterator* InSelf, PyObject* InPyFilter)
{
	UClass* IterFilter = nullptr;
	if (!PyConversion::NativizeClass(InPyFilter, IterFilter, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(InPyFilter)));
	}
	return IterFilter;
}


bool FPyStructIterator::PassesFilter(FPyStructIterator* InSelf)
{
	return PyTypeIterator_PassesFilter<UScriptStruct, FPyStructIterator>(InSelf);
}

UScriptStruct* FPyStructIterator::ExtractFilter(FPyStructIterator* InSelf, PyObject* InPyFilter)
{
	UScriptStruct* IterFilter = nullptr;
	if (!PyConversion::NativizeStruct(InPyFilter, IterFilter, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Struct'"), *PyUtil::GetFriendlyTypename(InPyFilter)));
	}
	return IterFilter;
}


PyObject* FPyTypeIterator::GetIterValue(FPyTypeIterator* InSelf)
{
	FPersistentThreadSafeObjectIterator& Iter = *InSelf->Iterator;
	UStruct* IterObj = CastChecked<UStruct>(*Iter);

	PyTypeObject* IterType = nullptr;
	if (const UClass* IterClass = Cast<UClass>(IterObj))
	{
		IterType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(IterClass);
	}
	if (const UScriptStruct* IterStruct = Cast<UScriptStruct>(IterObj))
	{
		IterType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(IterStruct);
	}
	check(IterType);

	Py_INCREF(IterType);
	return (PyObject*)IterType;
}

bool FPyTypeIterator::PassesFilter(FPyTypeIterator* InSelf)
{
	if (!PyTypeIterator_PassesFilter<UStruct, FPyTypeIterator>(InSelf))
	{
		return false;
	}

	FPersistentThreadSafeObjectIterator& Iter = *InSelf->Iterator;
	UStruct* IterObj = CastChecked<UStruct>(*Iter);

	if (const UClass* IterClass = Cast<UClass>(IterObj))
	{
		return FPyWrapperTypeRegistry::Get().HasWrappedClassType(IterClass);
	}
	if (const UScriptStruct* IterStruct = Cast<UScriptStruct>(IterObj))
	{
		return FPyWrapperTypeRegistry::Get().HasWrappedStructType(IterStruct);
	}

	return false;
}

UStruct* FPyTypeIterator::ExtractFilter(FPyTypeIterator* InSelf, PyObject* InPyFilter)
{
	UStruct* IterFilter = nullptr;
	if (PyType_Check(InPyFilter))
	{
		if (PyType_IsSubtype((PyTypeObject*)InPyFilter, &PyWrapperObjectType))
		{
			IterFilter = FPyWrapperObjectMetaData::GetClass((PyTypeObject*)InPyFilter);
		}
		else if (PyType_IsSubtype((PyTypeObject*)InPyFilter, &PyWrapperStructType))
		{
			IterFilter = FPyWrapperStructMetaData::GetStruct((PyTypeObject*)InPyFilter);
		}
	}
	else
	{
		PyConversion::NativizeObject(InPyFilter, (UObject*&)IterFilter, nullptr);
	}
	if (!IterFilter || !(IterFilter->IsA<UClass>() || IterFilter->IsA<UScriptStruct>()))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class' or 'Struct'"), *PyUtil::GetFriendlyTypename(InPyFilter)));
	}
	return IterFilter;
}


FPyUValueDef* FPyUValueDef::New(PyTypeObject* InType)
{
	FPyUValueDef* Self = (FPyUValueDef*)InType->tp_alloc(InType, 0);
	if (Self)
	{
		Self->Value = nullptr;
		Self->MetaData = nullptr;
	}
	return Self;
}

void FPyUValueDef::Free(FPyUValueDef* InSelf)
{
	Deinit(InSelf);
	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

int FPyUValueDef::Init(FPyUValueDef* InSelf, PyObject* InValue, PyObject* InMetaData)
{
	Deinit(InSelf);

	Py_INCREF(InValue);
	InSelf->Value = InValue;

	Py_INCREF(InMetaData);
	InSelf->MetaData = InMetaData;

	return 0;
}

void FPyUValueDef::Deinit(FPyUValueDef* InSelf)
{
	Py_XDECREF(InSelf->Value);
	InSelf->Value = nullptr;

	Py_XDECREF(InSelf->MetaData);
	InSelf->MetaData = nullptr;
}

void FPyUValueDef::ApplyMetaData(FPyUValueDef* InSelf, const TFunctionRef<void(const FString&, const FString&)>& InPredicate)
{
	PyCoreUtil::ApplyMetaData(InSelf->MetaData, InPredicate);
}


FPyFPropertyDef* FPyFPropertyDef::New(PyTypeObject* InType)
{
	FPyFPropertyDef* Self = (FPyFPropertyDef*)InType->tp_alloc(InType, 0);
	if (Self)
	{
		Self->PropType = nullptr;
		Self->MetaData = nullptr;
		new(&Self->GetterFuncName) FString();
		new(&Self->SetterFuncName) FString();
	}
	return Self;
}

void FPyFPropertyDef::Free(FPyFPropertyDef* InSelf)
{
	Deinit(InSelf);

	InSelf->GetterFuncName.~FString();
	InSelf->SetterFuncName.~FString();
	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

int FPyFPropertyDef::Init(FPyFPropertyDef* InSelf, PyObject* InPropType, PyObject* InMetaData, FString InGetterFuncName, FString InSetterFuncName)
{
	Deinit(InSelf);

	Py_INCREF(InPropType);
	InSelf->PropType = InPropType;

	Py_INCREF(InMetaData);
	InSelf->MetaData = InMetaData;

	InSelf->GetterFuncName = MoveTemp(InGetterFuncName);
	InSelf->SetterFuncName = MoveTemp(InSetterFuncName);

	return 0;
}

void FPyFPropertyDef::Deinit(FPyFPropertyDef* InSelf)
{
	Py_XDECREF(InSelf->PropType);
	InSelf->PropType = nullptr;

	Py_XDECREF(InSelf->MetaData);
	InSelf->MetaData = nullptr;

	InSelf->GetterFuncName.Reset();
	InSelf->SetterFuncName.Reset();
}

void FPyFPropertyDef::ApplyMetaData(FPyFPropertyDef* InSelf, FProperty* InProp)
{
	PyCoreUtil::ApplyMetaData(InSelf->MetaData, [InProp](const FString& InMetaDataKey, const FString& InMetaDataValue)
	{
		InProp->SetMetaData(*InMetaDataKey, *InMetaDataValue);
	});
}


FPyUFunctionDef* FPyUFunctionDef::New(PyTypeObject* InType)
{
	FPyUFunctionDef* Self = (FPyUFunctionDef*)InType->tp_alloc(InType, 0);
	if (Self)
	{
		Self->Func = nullptr;
		Self->FuncRetType = nullptr;
		Self->FuncParamTypes = nullptr;
		Self->MetaData = nullptr;
		Self->FuncFlags = EPyUFunctionDefFlags::None;
	}
	return Self;
}

void FPyUFunctionDef::Free(FPyUFunctionDef* InSelf)
{
	Deinit(InSelf);

	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

int FPyUFunctionDef::Init(FPyUFunctionDef* InSelf, PyObject* InFunc, PyObject* InFuncRetType, PyObject* InFuncParamTypes, PyObject* InMetaData, EPyUFunctionDefFlags InFuncFlags)
{
	Deinit(InSelf);

	Py_INCREF(InFunc);
	InSelf->Func = InFunc;

	Py_INCREF(InFuncRetType);
	InSelf->FuncRetType = InFuncRetType;

	Py_INCREF(InFuncParamTypes);
	InSelf->FuncParamTypes = InFuncParamTypes;

	Py_INCREF(InMetaData);
	InSelf->MetaData = InMetaData;

	InSelf->FuncFlags = InFuncFlags;

	return 0;
}

void FPyUFunctionDef::Deinit(FPyUFunctionDef* InSelf)
{
	Py_XDECREF(InSelf->Func);
	InSelf->Func = nullptr;

	Py_XDECREF(InSelf->FuncRetType);
	InSelf->FuncRetType = nullptr;

	Py_XDECREF(InSelf->FuncParamTypes);
	InSelf->FuncParamTypes = nullptr;

	Py_XDECREF(InSelf->MetaData);
	InSelf->MetaData = nullptr;

	InSelf->FuncFlags = EPyUFunctionDefFlags::None;
}

void FPyUFunctionDef::ApplyMetaData(FPyUFunctionDef* InSelf, UFunction* InFunc)
{
	PyCoreUtil::ApplyMetaData(InSelf->MetaData, [InFunc](const FString& InMetaDataKey, const FString& InMetaDataValue)
	{
		InFunc->SetMetaData(*InMetaDataKey, *InMetaDataValue);
	});
}


PyTypeObject InitializePyScopedSlowTaskType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyScopedSlowTask::New(InType);
		}

		static void Dealloc(FPyScopedSlowTask* InSelf)
		{
			FPyScopedSlowTask::Free(InSelf);
		}

		static int Init(FPyScopedSlowTask* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyWorkObj = nullptr;
			PyObject* PyDescObj = nullptr;
			PyObject* PyEnabledObj = nullptr;

			static const char *ArgsKwdList[] = { "work", "desc", "enabled", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|OO:call", (char**)ArgsKwdList, &PyWorkObj, &PyDescObj, &PyEnabledObj))
			{
				return -1;
			}

			float Work = 0.0f;
			if (!PyConversion::Nativize(PyWorkObj, Work))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'work' (%s) to 'float'"), *PyUtil::GetFriendlyTypename(PyWorkObj)));
				return -1;
			}

			FText Desc;
			if (PyDescObj && !PyConversion::Nativize(PyDescObj, Desc))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'desc' (%s) to 'Text'"), *PyUtil::GetFriendlyTypename(PyDescObj)));
				return -1;
			}

			bool bEnabled = true;
			if (PyEnabledObj && !PyConversion::Nativize(PyEnabledObj, bEnabled))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'enabled' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyEnabledObj)));
				return -1;
			}

			return FPyScopedSlowTask::Init(InSelf, Work, Desc, bEnabled);
		}
	};

	struct FMethods
	{
		static PyObject* EnterScope(FPyScopedSlowTask* InSelf)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			InSelf->SlowTask->Initialize();

			Py_INCREF(InSelf);
			return (PyObject*)InSelf;
		}

		static PyObject* ExitScope(FPyScopedSlowTask* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}
			
			InSelf->SlowTask->Destroy();

			Py_RETURN_NONE;
		}

		static PyObject* MakeDialogDelayed(FPyScopedSlowTask* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyDelayObj = nullptr;
			PyObject* PyCanCancelObj = nullptr;
			PyObject* PyAllowInPIEObj = nullptr;

			static const char *ArgsKwdList[] = { "delay", "can_cancel", "allow_in_pie", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|OO:make_dialog_delayed", (char**)ArgsKwdList, &PyDelayObj, &PyCanCancelObj, &PyAllowInPIEObj))
			{
				return nullptr;
			}

			float Delay = 0.0f;
			if (!PyConversion::Nativize(PyDelayObj, Delay))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'delay' (%s) to 'float'"), *PyUtil::GetFriendlyTypename(PyDelayObj)));
				return nullptr;
			}

			bool bCanCancel = false;
			if (PyCanCancelObj && !PyConversion::Nativize(PyCanCancelObj, bCanCancel))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'can_cancel' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyCanCancelObj)));
				return nullptr;
			}

			bool bAllowInPIE = false;
			if (PyAllowInPIEObj && !PyConversion::Nativize(PyAllowInPIEObj, bAllowInPIE))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'allow_in_pie' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyAllowInPIEObj)));
				return nullptr;
			}

			InSelf->SlowTask->MakeDialogDelayed(Delay, bCanCancel, bAllowInPIE);

			Py_RETURN_NONE;
		}

		static PyObject* MakeDialog(FPyScopedSlowTask* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyCanCancelObj = nullptr;
			PyObject* PyAllowInPIEObj = nullptr;

			static const char *ArgsKwdList[] = { "can_cancel", "allow_in_pie", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "|OO:make_dialog", (char**)ArgsKwdList, &PyCanCancelObj, &PyAllowInPIEObj))
			{
				return nullptr;
			}

			bool bCanCancel = false;
			if (PyCanCancelObj && !PyConversion::Nativize(PyCanCancelObj, bCanCancel))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'can_cancel' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyCanCancelObj)));
				return nullptr;
			}

			bool bAllowInPIE = false;
			if (PyAllowInPIEObj && !PyConversion::Nativize(PyAllowInPIEObj, bAllowInPIE))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'allow_in_pie' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyAllowInPIEObj)));
				return nullptr;
			}

			InSelf->SlowTask->MakeDialog(bCanCancel, bAllowInPIE);

			Py_RETURN_NONE;
		}

		static PyObject* EnterProgressFrame(FPyScopedSlowTask* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyWorkObj = nullptr;
			PyObject* PyDescObj = nullptr;

			static const char *ArgsKwdList[] = { "work", "desc", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "|OO:enter_progress_frame", (char**)ArgsKwdList, &PyWorkObj, &PyDescObj))
			{
				return nullptr;
			}

			float Work = 1.0f;
			if (PyWorkObj && !PyConversion::Nativize(PyWorkObj, Work))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'work' (%s) to 'float'"), *PyUtil::GetFriendlyTypename(PyWorkObj)));
				return nullptr;
			}

			FText Desc;
			if (PyDescObj && !PyConversion::Nativize(PyDescObj, Desc))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'description' (%s) to 'Text'"), *PyUtil::GetFriendlyTypename(PyDescObj)));
				return nullptr;
			}

			InSelf->SlowTask->EnterProgressFrame(Work, Desc);

			Py_RETURN_NONE;
		}

		static PyObject* ShouldCancel(FPyScopedSlowTask* InSelf)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const bool bShouldCancel = InSelf->SlowTask->ShouldCancel();
			return PyConversion::Pythonize(bShouldCancel);
		}
		
		static PyObject* TickProgress(FPyScopedSlowTask* InSelf)
		{
			if (!FPyScopedSlowTask::ValidateInternalState(InSelf))
			{
				return nullptr;
			}
			
			InSelf->SlowTask->TickProgress();
			
			Py_RETURN_NONE;
		}
	};

	static PyMethodDef PyMethods[] = {
		{ "__enter__", PyCFunctionCast(&FMethods::EnterScope), METH_NOARGS, "__enter__(self) -> ScopedSlowTask -- begin this slow task" },
		{ "__exit__", PyCFunctionCast(&FMethods::ExitScope), METH_VARARGS | METH_KEYWORDS, "__exit__(self, type: Optional[Type[BaseException]], value: Optional[BaseException], traceback: Optional[TracebackType]) -> bool -- end this slow task" },
		{ "make_dialog_delayed", PyCFunctionCast(&FMethods::MakeDialogDelayed), METH_VARARGS | METH_KEYWORDS, "make_dialog_delayed(self, delay: float, can_cancel: bool = False, allow_in_pie: bool = False) -> None -- creates a new dialog for this slow task after the given time delay (in seconds). If the task completes before this time, no dialog will be shown" },
		{ "make_dialog", PyCFunctionCast(&FMethods::MakeDialog), METH_VARARGS | METH_KEYWORDS, "make_dialog(self, can_cancel: bool = False, allow_in_pie: bool = False) -> None -- creates a new dialog for this slow task, if there is currently not one open" },
		{ "enter_progress_frame", PyCFunctionCast(&FMethods::EnterProgressFrame), METH_VARARGS | METH_KEYWORDS, "enter_progress_frame(self, work: float = 1.0, desc: Union[Text, str] = \"\") -> None -- indicate that we are to enter a frame that will take up the specified amount of work (completes any previous frames)" },
		{ "should_cancel", PyCFunctionCast(&FMethods::ShouldCancel), METH_NOARGS, "should_cancel(self) -> bool -- True if the user has requested that the slow task be canceled" },
		{ "tick_progress", PyCFunctionCast(&FMethods::TickProgress), METH_NOARGS, "tick_progress(self) -> None -- Let the UI refresh but doesn't advance progress" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"ScopedSlowTask", /* tp_name */
		sizeof(FPyScopedSlowTask), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type used to create and managed a scoped slow task in Python";

	return PyType;
}


PyTypeObject InitializePyObjectIteratorType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyObjectIterator::New(InType);
		}

		static void Dealloc(FPyObjectIterator* InSelf)
		{
			FPyObjectIterator::Free(InSelf);
		}

		static int Init(FPyObjectIterator* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;

			static const char *ArgsKwdList[] = { "type", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "|O:call", (char**)ArgsKwdList, &PyTypeObj))
			{
				return -1;
			}

			UClass* IterClass = UObject::StaticClass();
			if (PyTypeObj)
			{
				if (PyTypeObj == Py_None)
				{
					PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("'type' cannot be 'None'"));
					return -1;
				}
				if (!PyConversion::NativizeClass(PyTypeObj, IterClass, nullptr))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyTypeObj)));
					return -1;
				}
			}

			return FPyObjectIterator::Init(InSelf, IterClass, nullptr);
		}

		static FPyObjectIterator* GetIter(FPyObjectIterator* InSelf)
		{
			return FPyObjectIterator::GetIter(InSelf);
		}

		static PyObject* IterNext(FPyObjectIterator* InSelf)
		{
			return FPyObjectIterator::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"ObjectIterator", /* tp_name */
		sizeof(FPyObjectIterator), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for iterating Unreal Object instances";

	return PyType;
}


template <typename ObjectType, typename SelfType>
PyTypeObject InitializePyTypeIteratorType(const char* InTypeName, const char* InTypeDoc)
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)SelfType::New(InType);
		}

		static void Dealloc(SelfType* InSelf)
		{
			SelfType::Free(InSelf);
		}

		static int Init(SelfType* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;

			static const char *ArgsKwdList[] = { "type", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:call", (char**)ArgsKwdList, &PyTypeObj))
			{
				return -1;
			}

			ObjectType* IterFilter = SelfType::ExtractFilter(InSelf, PyTypeObj);
			if (!IterFilter)
			{
				return -1;
			}

			return SelfType::Init(InSelf, ObjectType::StaticClass(), IterFilter);
		}

		static SelfType* GetIter(SelfType* InSelf)
		{
			return SelfType::GetIter(InSelf);
		}

		static PyObject* IterNext(SelfType* InSelf)
		{
			return SelfType::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		InTypeName, /* tp_name */
		sizeof(SelfType), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = InTypeDoc;

	return PyType;
}


PyTypeObject InitializePyUValueDefType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyUValueDef::New(InType);
		}

		static void Dealloc(FPyUValueDef* InSelf)
		{
			FPyUValueDef::Free(InSelf);
		}

		static int Init(FPyUValueDef* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyValueObj = nullptr;
			PyObject* PyMetaObj = nullptr;

			static const char *ArgsKwdList[] = { "val", "meta", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:call", (char**)ArgsKwdList, &PyValueObj, &PyMetaObj))
			{
				return -1;
			}

			if (PyValueObj == Py_None)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("'val' cannot be 'None'"));
				return -1;
			}

			return FPyUValueDef::Init(InSelf, PyValueObj, PyMetaObj);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"ValueDef", /* tp_name */
		sizeof(FPyUValueDef), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type used to define constant values from Python";

	return PyType;
}


PyTypeObject InitializePyFPropertyDefType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyFPropertyDef::New(InType);
		}

		static void Dealloc(FPyFPropertyDef* InSelf)
		{
			FPyFPropertyDef::Free(InSelf);
		}

		static int Init(FPyFPropertyDef* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyPropTypeObj = nullptr;
			PyObject* PyMetaObj = nullptr;
			PyObject* PyPropGetterObj = nullptr;
			PyObject* PyPropSetterObj = nullptr;

			static const char *ArgsKwdList[] = { "type", "meta", "getter", "setter", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OOOO:call", (char**)ArgsKwdList, &PyPropTypeObj, &PyMetaObj, &PyPropGetterObj, &PyPropSetterObj))
			{
				return -1;
			}

			const FString ErrorCtxt = PyUtil::GetErrorContext(InSelf);

			FString PropGetter;
			if (!PyCoreUtil::ConvertOptionalString(PyPropGetterObj, PropGetter, *ErrorCtxt, TEXT("Failed to convert parameter 'getter' to a string (expected 'None' or 'str')")))
			{
				return -1;
			}

			FString PropSetter;
			if (!PyCoreUtil::ConvertOptionalString(PyPropSetterObj, PropSetter, *ErrorCtxt, TEXT("Failed to convert parameter 'setter' to a string (expected 'None' or 'str')")))
			{
				return -1;
			}

			return FPyFPropertyDef::Init(InSelf, PyPropTypeObj, PyMetaObj, MoveTemp(PropGetter), MoveTemp(PropSetter));
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"PropertyDef", /* tp_name */
		sizeof(FPyFPropertyDef), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type used to define FProperty fields from Python";

	return PyType;
}


PyTypeObject InitializePyUFunctionDefType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyUFunctionDef::New(InType);
		}

		static void Dealloc(FPyUFunctionDef* InSelf)
		{
			FPyUFunctionDef::Free(InSelf);
		}

		static int Init(FPyUFunctionDef* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyFuncObj = nullptr;
			PyObject* PyMetaObj = nullptr;
			PyObject* PyFuncRetTypeObj = nullptr;
			PyObject* PyFuncParamTypesObj = nullptr;
			PyObject* PyFuncOverrideObj = nullptr;
			PyObject* PyFuncStaticObj = nullptr;
			PyObject* PyFuncPureObj = nullptr;
			PyObject* PyFuncGetterObj = nullptr;
			PyObject* PyFuncSetterObj = nullptr;

			static const char *ArgsKwdList[] = { "func", "meta", "ret", "params", "override", "static", "pure", "getter", "setter", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OOOOOOOOO:call", (char**)ArgsKwdList, &PyFuncObj, &PyMetaObj, &PyFuncRetTypeObj, &PyFuncParamTypesObj, &PyFuncOverrideObj, &PyFuncStaticObj, &PyFuncPureObj, &PyFuncGetterObj, &PyFuncSetterObj))
			{
				return -1;
			}

			const FString ErrorCtxt = PyUtil::GetErrorContext(InSelf);

			EPyUFunctionDefFlags FuncFlags = EPyUFunctionDefFlags::None;
			if (!PyCoreUtil::ConvertOptionalFunctionFlag(PyFuncOverrideObj, FuncFlags, EPyUFunctionDefFlags::Override, EPyUFunctionDefFlags::None, *ErrorCtxt, TEXT("Failed to convert parameter 'override' to a flag (expected 'None' or 'bool')")))
			{
				return -1;
			}
			if (!PyCoreUtil::ConvertOptionalFunctionFlag(PyFuncStaticObj, FuncFlags, EPyUFunctionDefFlags::Static, EPyUFunctionDefFlags::None, *ErrorCtxt, TEXT("Failed to convert parameter 'static' to a flag (expected 'None' or 'bool')")))
			{
				return -1;
			}
			if (!PyCoreUtil::ConvertOptionalFunctionFlag(PyFuncPureObj, FuncFlags, EPyUFunctionDefFlags::Pure, EPyUFunctionDefFlags::Impure, *ErrorCtxt, TEXT("Failed to convert parameter 'pure' to a flag (expected 'None' or 'bool')")))
			{
				return -1;
			}
			if (!PyCoreUtil::ConvertOptionalFunctionFlag(PyFuncGetterObj, FuncFlags, EPyUFunctionDefFlags::Getter, EPyUFunctionDefFlags::None, *ErrorCtxt, TEXT("Failed to convert parameter 'getter' to a flag (expected 'None' or 'bool')")))
			{
				return -1;
			}
			if (!PyCoreUtil::ConvertOptionalFunctionFlag(PyFuncSetterObj, FuncFlags, EPyUFunctionDefFlags::Setter, EPyUFunctionDefFlags::None, *ErrorCtxt, TEXT("Failed to convert parameter 'setter' to a flag (expected 'None' or 'bool')")))
			{
				return -1;
			}

			return FPyUFunctionDef::Init(InSelf, PyFuncObj, PyFuncRetTypeObj, PyFuncParamTypesObj, PyMetaObj, FuncFlags);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"FunctionDef", /* tp_name */
		sizeof(FPyUFunctionDef), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type used to define UFunction fields from Python";

	return PyType;
}


PyTypeObject PyDelegateHandleType = InitializePyWrapperBasicType<FPyDelegateHandle>("_DelegateHandle", "Type for all Unreal exposed FDelegateHandle instances");
PyTypeObject PyTickerDelegateHandleType = InitializePyWrapperBasicType<FPyTickerDelegateHandle>("_TickerDelegateHandle", "Type for all Unreal exposed FTSTicker::FDelegateHandle instances");
PyTypeObject PyScopedSlowTaskType = InitializePyScopedSlowTaskType();
PyTypeObject PyObjectIteratorType = InitializePyObjectIteratorType();
PyTypeObject PyClassIteratorType = InitializePyTypeIteratorType<UClass, FPyClassIterator>("ClassIterator", "Type for iterating Unreal class types");
PyTypeObject PyStructIteratorType = InitializePyTypeIteratorType<UScriptStruct, FPyStructIterator>("StructIterator", "Type for iterating Unreal struct types");
PyTypeObject PyTypeIteratorType = InitializePyTypeIteratorType<UStruct, FPyTypeIterator>("TypeIterator", "Type for iterating Python types");
PyTypeObject PyUValueDefType = InitializePyUValueDefType();
PyTypeObject PyFPropertyDefType = InitializePyFPropertyDefType();
PyTypeObject PyUFunctionDefType = InitializePyUFunctionDefType();


namespace PyCore
{

bool ShouldLog(const FString& InLogMessage)
{
	if (InLogMessage.IsEmpty())
	{
		return false;
	}

	for (const TCHAR LogChar : InLogMessage)
	{
		if (!FChar::IsWhitespace(LogChar))
		{
			return true;
		}
	}

	return false;
}

PyObject* Log(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (PyArg_ParseTuple(InArgs, "O:log", &PyObj))
	{
		const FString LogMessage = PyUtil::PyObjectToUEString(PyObj);
		if (ShouldLog(LogMessage))
		{
			Py_BEGIN_ALLOW_THREADS
			UE_LOGF(LogPython, Log, "%ls", *LogMessage);
			Py_END_ALLOW_THREADS
			GetPythonLogCapture().Broadcast(EPythonLogOutputType::Info, *LogMessage);
		}

		Py_RETURN_NONE;
	}

	return nullptr;
}

PyObject* LogWarning(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (PyArg_ParseTuple(InArgs, "O:log_warning", &PyObj))
	{
		const FString LogMessage = PyUtil::PyObjectToUEString(PyObj);
		if (ShouldLog(LogMessage))
		{
			Py_BEGIN_ALLOW_THREADS
			UE_LOGF(LogPython, Warning, "%ls", *LogMessage);
			Py_END_ALLOW_THREADS
			GetPythonLogCapture().Broadcast(EPythonLogOutputType::Warning, *LogMessage);
		}

		Py_RETURN_NONE;
	}

	return nullptr;
}

PyObject* LogError(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (PyArg_ParseTuple(InArgs, "O:log_error", &PyObj))
	{
		const FString LogMessage = PyUtil::PyObjectToUEString(PyObj);
		if (ShouldLog(LogMessage))
		{
			Py_BEGIN_ALLOW_THREADS
			UE_LOGF(LogPython, Error, "%ls", *LogMessage);
			Py_END_ALLOW_THREADS
			GetPythonLogCapture().Broadcast(EPythonLogOutputType::Error, *LogMessage);
		}

		Py_RETURN_NONE;
	}

	return nullptr;
}

PyObject* LogFlush(PyObject* InSelf)
{
	if (GLog)
	{
		Py_BEGIN_ALLOW_THREADS
		GLog->Flush();
		Py_END_ALLOW_THREADS
	}

	Py_RETURN_NONE;
}

PyObject* Reload(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:reload", &PyObj))
	{
		return nullptr;
	}
	
	FString ModuleName;
	if (!PyConversion::Nativize(PyObj, ModuleName))
	{
		return nullptr;
	}

	FPythonScriptPlugin* PythonScriptPlugin = FPythonScriptPlugin::Get();
	if (PythonScriptPlugin)
	{
		PythonScriptPlugin->ImportUnrealModule(*ModuleName);
	}

	Py_RETURN_NONE;
}

PyObject* LoadModule(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:load_module", &PyObj))
	{
		return nullptr;
	}

	FString ModuleName;
	if (!PyConversion::Nativize(PyObj, ModuleName))
	{
		return nullptr;
	}

	if (!FModuleManager::Get().ModuleExists(*ModuleName))
	{
		PyUtil::SetPythonError(PyExc_KeyError, TEXT("load_module"), *FString::Printf(TEXT("'%s' isn't a known module name"), *ModuleName));
		return nullptr;
	}

	IModuleInterface* ModulePtr = FModuleManager::Get().LoadModule(*ModuleName);
	if (ModulePtr)
	{
		FPythonScriptPlugin* PythonScriptPlugin = FPythonScriptPlugin::Get();
		if (PythonScriptPlugin)
		{
			PythonScriptPlugin->ImportUnrealModule(*ModuleName);
		}
	}

	Py_RETURN_NONE;
}

PyObject* FindOrLoadObjectImpl(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds, const TCHAR* InFuncName, const TFunctionRef<UObject*(UClass*, UObject*, const TCHAR*)>& InFunc)
{
	PyObject* PyOuterObj = nullptr;
	PyObject* PyNameObj = nullptr;
	PyObject* PyTypeObj = nullptr;
	PyObject* PyFollowRedirectorsObj = nullptr;

	static const char *ArgsKwdList[] = { "outer", "name", "type", "follow_redirectors", nullptr };
	if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, TCHAR_TO_UTF8(*FString::Printf(TEXT("OO|OO:%s"), InFuncName)), (char**)ArgsKwdList, &PyOuterObj, &PyNameObj, &PyTypeObj, &PyFollowRedirectorsObj))
	{
		return nullptr;
	}

	UObject* ObjectOuter = nullptr;
	if (!PyConversion::Nativize(PyOuterObj, ObjectOuter))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'outer' (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyOuterObj)));
		return nullptr;
	}

	FString ObjectName;
	if (!PyConversion::Nativize(PyNameObj, ObjectName))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'String'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
		return nullptr;
	}

	UClass* ObjectType = nullptr;
	if (PyTypeObj && !PyConversion::NativizeClass(PyTypeObj, ObjectType, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyTypeObj)));
		return nullptr;
	}
	if (!ObjectType)
	{
		ObjectType = UObject::StaticClass();
	}

	bool bFollowRedirectors = true;
	if (PyFollowRedirectorsObj && !PyConversion::Nativize(PyFollowRedirectorsObj, bFollowRedirectors))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'follow_redirectors' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyFollowRedirectorsObj)));
		return nullptr;
	}

	UObject* PotentialObject = InFunc(UObject::StaticClass(), ObjectOuter, *ObjectName);

	// Follow any redirectors
	if (bFollowRedirectors)
	{
		while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(PotentialObject))
		{
			PotentialObject = Redirector->DestinationObject;
		}
	}

	// Make sure the object is of the correct type
	if (PotentialObject && !PotentialObject->IsA(ObjectType))
	{
		PotentialObject = nullptr;
	}

	return PyConversion::Pythonize(PotentialObject);
}

PyObject* NewObject(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	UClass* ObjectType = nullptr;
	UObject* ObjectOuter = GetTransientPackage();
	FName ObjectName;
	UClass* ObjectBaseType = nullptr;

	// Parse the args
	{
		PyObject* PyTypeObj = nullptr;
		PyObject* PyOuterObj = nullptr;
		PyObject* PyNameObj = nullptr;
		PyObject* PyBaseTypeObj = nullptr;

		static const char* ArgsKwdList[] = { "type", "outer", "name", "base_type", nullptr };
		if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|OOO:new_object", (char**)ArgsKwdList, &PyTypeObj, &PyOuterObj, &PyNameObj, &PyBaseTypeObj))
		{
			return nullptr;
		}

		if (!PyConversion::NativizeClass(PyTypeObj, ObjectType, UObject::StaticClass()))
		{
			PyUtil::SetPythonError(PyExc_TypeError, TEXT("new_object"), *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyTypeObj)));
			return nullptr;
		}

		// If PyOuterObj is None, Nativize() will convert 'PyOuterObj=None' into 'nullptr'. That will overwrite the desired default value 'transient package'.
		if (PyOuterObj && PyOuterObj != Py_None && !PyConversion::Nativize(PyOuterObj, ObjectOuter))
		{
			PyUtil::SetPythonError(PyExc_TypeError, TEXT("new_object"), *FString::Printf(TEXT("Failed to convert 'outer' (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyOuterObj)));
			return nullptr;
		}

		if (PyNameObj && !PyConversion::Nativize(PyNameObj, ObjectName))
		{
			PyUtil::SetPythonError(PyExc_TypeError, TEXT("new_object"), *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
			return nullptr;
		}

		if (PyBaseTypeObj && !PyConversion::NativizeClass(PyBaseTypeObj, ObjectBaseType, UObject::StaticClass()))
		{
			PyUtil::SetPythonError(PyExc_TypeError, TEXT("new_object"), *FString::Printf(TEXT("Failed to convert 'base_type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyBaseTypeObj)));
			return nullptr;
		}
	}

	// Create the instance
	UObject* ObjectInstance = PyUtil::NewObject(ObjectType, ObjectOuter, ObjectName, ObjectBaseType, TEXT("new_object"));
	if (!ObjectInstance)
	{
		// PyUtil::NewObject should have set an error state
		return nullptr;
	}

	return PyConversion::Pythonize(ObjectInstance);
}

PyObject* FindObject(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	return FindOrLoadObjectImpl(InSelf, InArgs, InKwds, TEXT("find_object"), [](UClass* ObjectType, UObject* ObjectOuter, const TCHAR* ObjectName)
	{
		UObject* Object = nullptr;
		Py_BEGIN_ALLOW_THREADS
		Object = ::StaticFindObject(ObjectType, ObjectOuter, ObjectName);
		Py_END_ALLOW_THREADS
		return Object;
	});
}

PyObject* LoadObject(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	return FindOrLoadObjectImpl(InSelf, InArgs, InKwds, TEXT("load_object"), [](UClass* ObjectType, UObject* ObjectOuter, const TCHAR* ObjectName)
	{
		UObject* Object = nullptr;
		Py_BEGIN_ALLOW_THREADS
		Object = ::StaticLoadObject(ObjectType, ObjectOuter, ObjectName);
		Py_END_ALLOW_THREADS
		return Object;
	});
}

PyObject* LoadClass(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	PyObject* PyOuterObj = nullptr;
	PyObject* PyNameObj = nullptr;
	PyObject* PyTypeObj = nullptr;

	static const char *ArgsKwdList[] = { "outer", "name", "type", nullptr };
	if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO|O:load_class", (char**)ArgsKwdList, &PyOuterObj, &PyNameObj, &PyTypeObj))
	{
		return nullptr;
	}

	UObject* ObjectOuter = nullptr;
	if (!PyConversion::Nativize(PyOuterObj, ObjectOuter))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("load_class"), *FString::Printf(TEXT("Failed to convert 'outer' (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyOuterObj)));
		return nullptr;
	}

	FString ObjectName;
	if (!PyConversion::Nativize(PyNameObj, ObjectName))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("load_class"), *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'String'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
		return nullptr;
	}

	UClass* ObjectType = nullptr;
	if (PyTypeObj && !PyConversion::NativizeClass(PyTypeObj, ObjectType, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("load_class"), *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyTypeObj)));
		return nullptr;
	}
	if (!ObjectType)
	{
		ObjectType = UObject::StaticClass();
	}

	UClass* PotentialClass = nullptr;
	Py_BEGIN_ALLOW_THREADS
	PotentialClass = ::StaticLoadClass(ObjectType, ObjectOuter, *ObjectName);
	Py_END_ALLOW_THREADS
	return PyConversion::PythonizeClass(PotentialClass);
}

PyObject* FindOrLoadAssetImpl(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds, const TCHAR* InFuncName, const TFunctionRef<UObject*(UClass*, UObject*, const TCHAR*)>& InFunc)
{
	PyObject* PyNameObj = nullptr;
	PyObject* PyTypeObj = nullptr;
	PyObject* PyFollowRedirectorsObj = nullptr;

	static const char *ArgsKwdList[] = { "name", "type", "follow_redirectors", nullptr };
	if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, TCHAR_TO_UTF8(*FString::Printf(TEXT("O|OO:%s"), InFuncName)), (char**)ArgsKwdList, &PyNameObj, &PyTypeObj, &PyFollowRedirectorsObj))
	{
		return nullptr;
	}

	FString ObjectName;
	if (!PyConversion::Nativize(PyNameObj, ObjectName))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'String'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
		return nullptr;
	}

	UClass* ObjectType = nullptr;
	if (PyTypeObj && !PyConversion::NativizeClass(PyTypeObj, ObjectType, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyTypeObj)));
		return nullptr;
	}
	if (!ObjectType)
	{
		ObjectType = UObject::StaticClass();
	}

	bool bFollowRedirectors = true;
	if (PyFollowRedirectorsObj && !PyConversion::Nativize(PyFollowRedirectorsObj, bFollowRedirectors))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'follow_redirectors' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyFollowRedirectorsObj)));
		return nullptr;
	}

	UObject* PotentialAsset = InFunc(UObject::StaticClass(), nullptr, *ObjectName);
	
	// If we found a package, try and get the primary asset from it
	if (UPackage* FoundPackage = Cast<UPackage>(PotentialAsset))
	{
		PotentialAsset = InFunc(UObject::StaticClass(), FoundPackage, *FPackageName::GetShortName(FoundPackage));
	}

	// Follow any redirectors
	if (bFollowRedirectors)
	{
		while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(PotentialAsset))
		{
			PotentialAsset = Redirector->DestinationObject;
		}
	}

	// Make sure the object is an asset of the correct type
	if (PotentialAsset && (!PotentialAsset->IsAsset() || !PotentialAsset->IsA(ObjectType)))
	{
		PotentialAsset = nullptr;
	}

	return PyConversion::Pythonize(PotentialAsset);
}

PyObject* FindAsset(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	return FindOrLoadAssetImpl(InSelf, InArgs, InKwds, TEXT("find_asset"), [](UClass* ObjectType, UObject* ObjectOuter, const TCHAR* ObjectName)
	{
		UObject* Object = nullptr;
		Py_BEGIN_ALLOW_THREADS
		Object = ::StaticFindObject(ObjectType, ObjectOuter, ObjectName);
		Py_END_ALLOW_THREADS
		return Object;
	});
}

PyObject* LoadAsset(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	return FindOrLoadAssetImpl(InSelf, InArgs, InKwds, TEXT("load_asset"), [](UClass* ObjectType, UObject* ObjectOuter, const TCHAR* ObjectName)
	{
		UObject* Object = nullptr;
		Py_BEGIN_ALLOW_THREADS
		Object = ::StaticLoadObject(ObjectType, ObjectOuter, ObjectName);
		Py_END_ALLOW_THREADS
		return Object;
	});
}

PyObject* FindOrLoadPackageImpl(PyObject* InSelf, PyObject* InArgs, const TCHAR* InFuncName, const TFunctionRef<UPackage*(const TCHAR*)>& InFunc)
{
	PyObject* PyNameObj = nullptr;

	if (!PyArg_ParseTuple(InArgs, TCHAR_TO_UTF8(*FString::Printf(TEXT("O:%s"), InFuncName)), &PyNameObj))
	{
		return nullptr;
	}

	FString PackageName;
	if (!PyConversion::Nativize(PyNameObj, PackageName))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InFuncName, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'String'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
		return nullptr;
	}

	return PyConversion::Pythonize(InFunc(*PackageName));
}

PyObject* FindPackage(PyObject* InSelf, PyObject* InArgs)
{
	return FindOrLoadPackageImpl(InSelf, InArgs, TEXT("find_package"), [](const TCHAR* PackageName)
	{
		UPackage* Package = nullptr;
		Py_BEGIN_ALLOW_THREADS
		Package = ::FindPackage(nullptr, PackageName);
		Py_END_ALLOW_THREADS
		return Package;
	});
}

PyObject* LoadPackage(PyObject* InSelf, PyObject* InArgs)
{
	return FindOrLoadPackageImpl(InSelf, InArgs, TEXT("load_package"), [](const TCHAR* PackageName)
	{
		UPackage* Package = nullptr;
		Py_BEGIN_ALLOW_THREADS
		Package = ::LoadPackage(nullptr, PackageName, LOAD_None);
		Py_END_ALLOW_THREADS
		return Package;
	});
}

PyObject* GetDefaultObject(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:get_default_object", &PyObj))
	{
		return nullptr;
	}

	UClass* Class = nullptr;
	if (!PyConversion::NativizeClass(PyObj, Class, nullptr))
	{
		return nullptr;
	}

	UObject* CDO = Class ? ::GetMutableDefault<UObject>(Class) : nullptr;
	return PyConversion::Pythonize(CDO);
}

PyObject* PurgeObjectReferences(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	PyObject* PyObj = nullptr;
	PyObject* PyIncludeInnersObj = nullptr;

	static const char *ArgsKwdList[] = { "obj", "include_inners", nullptr };
	if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:purge_object_references", (char**)ArgsKwdList, &PyObj, &PyIncludeInnersObj))
	{
		return nullptr;
	}

	UObject* Object = nullptr;
	if (!PyConversion::Nativize(PyObj, Object))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("purge_object_references"), *FString::Printf(TEXT("Failed to convert 'obj' (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	bool bIncludeInners = true;
	if (PyIncludeInnersObj && !PyConversion::Nativize(PyIncludeInnersObj, bIncludeInners))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("purge_object_references"), *FString::Printf(TEXT("Failed to convert 'include_inners' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyIncludeInnersObj)));
		return nullptr;
	}

	FPyReferenceCollector::Get().PurgeUnrealObjectReferences(Object, bIncludeInners);

	Py_RETURN_NONE;
}

PyObject* GenerateClass(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:generate_class", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (!PyType_Check(PyObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("generate_class"), *FString::Printf(TEXT("Parameter must be a 'type' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	PyTypeObject* PyType = (PyTypeObject*)PyObj;
	if (!PyType_IsSubtype(PyType, &PyWrapperObjectType))
	{
		PyUtil::SetPythonError(PyExc_Exception, TEXT("generate_class"), *FString::Printf(TEXT("Type '%s' does not derive from an Unreal class type"), *PyUtil::GetFriendlyTypename(PyType)));
		return nullptr;
	}
	
	// We only need to generate classes for types without meta-data, as any types with meta-data have already been generated
	if (!FPyWrapperObjectMetaData::GetMetaData(PyType) && !UPythonGeneratedClass::GenerateClass(PyType))
	{
		PyUtil::SetPythonError(PyExc_Exception, TEXT("generate_class"), *FString::Printf(TEXT("Failed to generate an Unreal class for the Python type '%s'"), *PyUtil::GetFriendlyTypename(PyType)));
		return nullptr;
	}

	Py_RETURN_NONE;
}

PyObject* GenerateStruct(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:generate_struct", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (!PyType_Check(PyObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("generate_struct"), *FString::Printf(TEXT("Parameter must be a 'type' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	PyTypeObject* PyType = (PyTypeObject*)PyObj;
	if (!PyType_IsSubtype(PyType, &PyWrapperStructType))
	{
		PyUtil::SetPythonError(PyExc_Exception, TEXT("generate_struct"), *FString::Printf(TEXT("Type '%s' does not derive from an Unreal struct type"), *PyUtil::GetFriendlyTypename(PyType)));
		return nullptr;
	}

	// We only need to generate structs for types without meta-data, as any types with meta-data have already been generated
	if (!FPyWrapperStructMetaData::GetMetaData(PyType) && !UPythonGeneratedStruct::GenerateStruct(PyType))
	{
		PyUtil::SetPythonError(PyExc_Exception, TEXT("generate_struct"), *FString::Printf(TEXT("Failed to generate an Unreal struct for the Python type '%s'"), *PyUtil::GetFriendlyTypename(PyType)));
		return nullptr;
	}

	Py_RETURN_NONE;
}

PyObject* GenerateEnum(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:generate_enum", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (!PyType_Check(PyObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("generate_enum"), *FString::Printf(TEXT("Parameter must be a 'type' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	PyTypeObject* PyType = (PyTypeObject*)PyObj;
	if (!PyType_IsSubtype(PyType, &PyWrapperEnumType))
	{
		PyUtil::SetPythonError(PyExc_Exception, TEXT("generate_enum"), *FString::Printf(TEXT("Type '%s' does not derive from the Unreal enum type"), *PyUtil::GetFriendlyTypename(PyType)));
		return nullptr;
	}

	// We only need to generate enums for types without meta-data, as any types with meta-data have already been generated
	if (!FPyWrapperEnumMetaData::GetMetaData(PyType) && !UPythonGeneratedEnum::GenerateEnum(PyType))
	{
		PyUtil::SetPythonError(PyExc_Exception, TEXT("generate_enum"), *FString::Printf(TEXT("Failed to generate an Unreal enum for the Python type '%s'"), *PyUtil::GetFriendlyTypename(PyType)));
		return nullptr;
	}

	Py_RETURN_NONE;
}

PyObject* FlushGeneratedTypeReinstancing(PyObject* InSelf)
{
	Py_BEGIN_ALLOW_THREADS
	FPyWrapperTypeReinstancer::Get().ProcessPending();
	Py_END_ALLOW_THREADS

	Py_RETURN_NONE;
}

PyObject* CollectGarbage(PyObject* InSelf)
{
	Py_BEGIN_ALLOW_THREADS
	::CollectGarbage(IsRunningCommandlet() ? RF_NoFlags : GARBAGE_COLLECTION_KEEPFLAGS);
	Py_END_ALLOW_THREADS

	Py_RETURN_NONE;
}

PyObject* GetTypeFromClass(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:get_type_from_class", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	UClass* Class = nullptr;
	if (!PyConversion::NativizeClass(PyObj, Class, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_type_from_class"), *FString::Printf(TEXT("Parameter must be a 'Class' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(Class);
	Py_INCREF(PyType);
	return (PyObject*)PyType;
}

PyObject* GetTypeFromStruct(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:get_type_from_struct", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	UScriptStruct* Struct = nullptr;
	if (!PyConversion::NativizeStruct(PyObj, Struct, nullptr))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_type_from_struct"), *FString::Printf(TEXT("Parameter must be a 'Struct' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(Struct);
	Py_INCREF(PyType);
	return (PyObject*)PyType;
}

PyObject* GetTypeFromEnum(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:get_type_from_enum", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	UEnum* Enum = nullptr;
	if (!PyConversion::Nativize(PyObj, Enum))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_type_from_enum"), *FString::Printf(TEXT("Parameter must be a 'Enum' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedEnumType(Enum);
	Py_INCREF(PyType);
	return (PyObject*)PyType;
}

PyObject* CreateLocalizedText(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyNamespaceObj = nullptr;
	PyObject* PyKeyObj = nullptr;
	PyObject* PySourceObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "OOO:NSLOCTEXT", &PyNamespaceObj, &PyKeyObj, &PySourceObj))
	{
		return nullptr;
	}

	FString Namespace;
	if (!PyConversion::Nativize(PyNamespaceObj, Namespace))
	{
		return nullptr;
	}

	FString Key;
	if (!PyConversion::Nativize(PyKeyObj, Key))
	{
		return nullptr;
	}

	FString Source;
	if (!PyConversion::Nativize(PySourceObj, Source))
	{
		return nullptr;
	}

	return PyConversion::Pythonize(FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(Source)));
}

PyObject* CreateLocalizedTextFromStringTable(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyIdObj = nullptr;
	PyObject* PyKeyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "OO:LOCTABLE", &PyIdObj, &PyKeyObj))
	{
		return nullptr;
	}

	FName Id;
	if (!PyConversion::Nativize(PyIdObj, Id))
	{
		return nullptr;
	}

	FString Key;
	if (!PyConversion::Nativize(PyKeyObj, Key))
	{
		return nullptr;
	}

	return PyConversion::Pythonize(FText::FromStringTable(Id, Key));
}

PyObject* IsEditor()
{
	return PyConversion::Pythonize(GIsEditor);
}

PyObject* GetInterpreterExecutablePath()
{
	const FString PythonPath = PyUtil::GetInterpreterExecutablePath();
	return PyConversion::Pythonize(PythonPath);
}

PyObject* CreatePythonObjectHandle(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:create_python_object_handle", &PyObj))
	{
		return nullptr;
	}

	UPythonObjectHandle* Handle = UPythonObjectHandle::Create(PyObj);
	return PyConversion::Pythonize(Handle);
}

PyObject* ResolvePythonObjectHandle(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyHandle = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:resolve_python_object_handle", &PyHandle))
	{
		return nullptr;
	}
	check(PyHandle);

	UPythonObjectHandle* Handle = nullptr;
	if (!PyConversion::Nativize(PyHandle, Handle))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("resolve_python_object_handle"), *FString::Printf(TEXT("Parameter must be a 'PythonObjectHandle' not '%s'"), *PyUtil::GetFriendlyTypename(PyHandle)));
		return nullptr;
	}
	
	if (Handle)
	{
		PyObject* PyObj = Handle->Resolve();
		Py_INCREF(PyObj);
		return PyObj;
	}

	Py_RETURN_NONE;
}

PyObject* DestroyPythonObjectHandle(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyHandle = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:destroy_python_object_handle", &PyHandle))
	{
		return nullptr;
	}
	check(PyHandle);

	UPythonObjectHandle* Handle = nullptr;
	if (!PyConversion::Nativize(PyHandle, Handle))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("destroy_python_object_handle"), *FString::Printf(TEXT("Parameter must be a 'PythonObjectHandle' not '%s'"), *PyUtil::GetFriendlyTypename(PyHandle)));
		return nullptr;
	}

	if (Handle)
	{
		Handle->ReleasePythonResources();
	}

	Py_RETURN_NONE;
}

PyObject* RegisterPythonShutdownCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:register_python_shutdown_callback", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	FDelegateHandle PythonShutdownDelegateHandle = FPythonScriptPlugin::Get()->OnPythonShutdown().AddLambda([PyCallable = FPyAutoGILPtr(FPyObjectPtr::NewReference(PyObj))]() mutable
	{
		FPyObjectPtr Result = FPyObjectPtr::StealReference(PyObject_CallObject(PyCallable.Get().GetPtr(), nullptr));
		if (!Result)
		{
			PyUtil::LogPythonError();
		}
	});

	return (PyObject*)FPyDelegateHandle::CreateInstance(PythonShutdownDelegateHandle);
}

PyObject* UnregisterPythonShutdownCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:unregister_python_shutdown_callback", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	FPyDelegateHandlePtr PythonEventDelegateHandle = FPyDelegateHandlePtr::StealReference(FPyDelegateHandle::CastPyObject(PyObj));
	if (!PythonEventDelegateHandle)
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("unregister_python_shutdown_callback"), *FString::Printf(TEXT("Failed to convert argument '%s' to '_DelegateHandle'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	if (PythonEventDelegateHandle->Value.IsValid())
	{
		FPythonScriptPlugin::Get()->OnPythonShutdown().Remove(PythonEventDelegateHandle->Value);
	}

	Py_RETURN_NONE;
}

PyObject* RegisterTickerCallback(PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	PyObject* PyCallableObj = nullptr;
	PyObject* PyDelayObj = nullptr;

	static const char* ArgsKwdList[] = { "callable", "delay", nullptr };
	if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:register_ticker_callback", (char**)ArgsKwdList, &PyCallableObj, &PyDelayObj))
	{
		return nullptr;
	}
	check(PyCallableObj);

	float Delay = 0.0f;
	if (PyDelayObj && !PyConversion::Nativize(PyDelayObj, Delay))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("register_ticker_callback"), *FString::Printf(TEXT("Failed to convert 'delay' (%s) to 'float'"), *PyUtil::GetFriendlyTypename(PyDelayObj)));
		return nullptr;
	}

	return (PyObject*)PyCoreUtil::RegisterTickerCallback(Delay, PyCallableObj);
}

PyObject* UnregisterTickerCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyCallableObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:unregister_ticker_callback", &PyCallableObj))
	{
		return nullptr;
	}
	check(PyCallableObj);

	if (!PyCoreUtil::UnregisterTickerCallback(PyCallableObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("unregister_ticker_callback"), *FString::Printf(TEXT("Failed to convert argument '%s' to '_TickerDelegateHandle'"), *PyUtil::GetFriendlyTypename(PyCallableObj)));
		return nullptr;
	}

	Py_RETURN_NONE;
}

// NOTE: Several functions like 'new_object' are marked as returning 'Any' because the exact return type cannot be deduced from the Class type. We could mark them as returning base 'Object'
//       type, but that would force user to cast down to please type checker while the returned type is already the exact requested type. For example, the instruction
//       'o: PyTestObject = unreal.new_object(unreal.PyTestObject.static_class())' must pass type checker and if new_object() was to return Object, it would not be automatically cast down.
PyMethodDef PyCoreMethods[] = {
	{ "log", PyCFunctionCast(&Log), METH_VARARGS, "log(arg: Any) -> None -- log the given argument as information in the LogPython category" },
	{ "log_warning", PyCFunctionCast(&LogWarning), METH_VARARGS, "log_warning(arg: Any) -> None -- log the given argument as a warning in the LogPython category" },
	{ "log_error", PyCFunctionCast(&LogError), METH_VARARGS, "log_error(arg: Any) -> None -- log the given argument as an error in the LogPython category" },
	{ "log_flush", PyCFunctionCast(&LogFlush), METH_NOARGS, "log_flush() -> None -- flush the log to disk." },
	{ "reload", PyCFunctionCast(&Reload), METH_VARARGS, "reload(module: str) -> None -- reload the given Unreal Python module." },
	{ "load_module", PyCFunctionCast(&LoadModule), METH_VARARGS, "load_module(module: str) -> None -- load the given Unreal module and generate any Python code for its reflected types" },
	{ "new_object", PyCFunctionCast(&NewObject), METH_VARARGS | METH_KEYWORDS, "new_object(type: Union[Class, type], outer: Optional[Object]=None, name: Union[Name, str]=\"\", base_type: Optional[Object]=None) -> Any -- create an Unreal object of the given class (and optional outer and name), optionally validating its type" },
	{ "find_object", PyCFunctionCast(&FindObject), METH_VARARGS | METH_KEYWORDS, "find_object(outer: Optional[Object], name: str, type: Union[Class, type]=Object.static_class(), follow_redirectors: bool=True) -> Any -- find an already loaded Unreal object with the given outer and name, optionally validating its type" },
	{ "load_object", PyCFunctionCast(&LoadObject), METH_VARARGS | METH_KEYWORDS, "load_object(outer: Optional[Object], name: str, type: Union[Class, type]=Object.static_class(), follow_redirectors: bool=True) -> Any -- load an Unreal object with the given outer and name, optionally validating its type" },
	{ "load_class", PyCFunctionCast(&LoadClass), METH_VARARGS | METH_KEYWORDS, "load_class(outer: Optional[Object], name: str, type: Union[Class, type]=Object.static_class()) -> Optional[Class] -- load an Unreal class with the given outer and name, optionally validating its base type" },
	{ "find_asset", PyCFunctionCast(&FindAsset), METH_VARARGS | METH_KEYWORDS, "find_asset(name: str, type: Union[Class, type]=Object.static_class(), follow_redirectors: bool = True) -> Any -- find an already loaded Unreal asset with the given name, optionally validating its type" },
	{ "load_asset", PyCFunctionCast(&LoadAsset), METH_VARARGS | METH_KEYWORDS, "load_asset(name: str, type: Union[Class, type]=Object.static_class(), follow_redirectors: bool = True) -> Any -- load an Unreal asset with the given name, optionally validating its type" },
	{ "find_package", PyCFunctionCast(&FindPackage), METH_VARARGS, "find_package(name: str) -> Optional[Package] -- find an already loaded Unreal package with the given name" },
	{ "load_package", PyCFunctionCast(&LoadPackage), METH_VARARGS, "load_package(name: str) -> Optional[Package] -- load an Unreal package with the given name" },
	{ "get_default_object", PyCFunctionCast(&GetDefaultObject), METH_VARARGS, "get_default_object(type: Union[Class, type]) -> Any -- get the Unreal class default object (CDO) of the given type" },
	{ "purge_object_references", PyCFunctionCast(&PurgeObjectReferences), METH_VARARGS | METH_KEYWORDS, "purge_object_references(obj: Object, include_inners: bool = True) -> None -- purge all references to the given Unreal object from any living Python objects" },
	{ "generate_class", PyCFunctionCast(&GenerateClass), METH_VARARGS, "generate_class(class_type: type) -> None -- generate an Unreal class for the given Python type" },
	{ "generate_struct", PyCFunctionCast(&GenerateStruct), METH_VARARGS, "generate_struct(struct_type: type) -> None -- generate an Unreal struct for the given Python type" },
	{ "generate_enum", PyCFunctionCast(&GenerateEnum), METH_VARARGS, "generate_enum(enum_type: type) -> None -- generate an Unreal enum for the given Python type" },
	{ "flush_generated_type_reinstancing", PyCFunctionCast(&FlushGeneratedTypeReinstancing), METH_NOARGS, "flush_generated_type_reinstancing() -> None -- flush any pending reinstancing requests for Python generated types" },
	{ "collect_garbage", PyCFunctionCast(&CollectGarbage), METH_NOARGS, "collect_garbage() -> None -- run Unreal garbage collection" },
	{ "get_type_from_class", PyCFunctionCast(&GetTypeFromClass), METH_VARARGS, "get_type_from_class(class_: Class) -> type -- get the best matching Python type for the given Unreal class" },
	{ "get_type_from_struct", PyCFunctionCast(&GetTypeFromStruct), METH_VARARGS, "get_type_from_struct(struct: Struct) -> type -- get the best matching Python type for the given Unreal struct" },
	{ "get_type_from_enum", PyCFunctionCast(&GetTypeFromEnum), METH_VARARGS, "get_type_from_enum(enum: Enum) -> type -- get the best matching Python type for the given Unreal enum" },
	{ "register_python_shutdown_callback", PyCFunctionCast(&RegisterPythonShutdownCallback), METH_VARARGS, "register_python_shutdown_callback(callable: Callable[[], None]) -> object -- register the given callable (with no input arguments) as a callback to execute immediately before Python shutdown"},
	{ "unregister_python_shutdown_callback", PyCFunctionCast(&UnregisterPythonShutdownCallback), METH_VARARGS, "unregister_python_shutdown_callback(handle: object) -> None -- unregister the given handle from a previous call to register_python_shutdown_callback"},
	{ "NSLOCTEXT", PyCFunctionCast(&CreateLocalizedText), METH_VARARGS, "NSLOCTEXT(ns: str, key: str, source: str) -> Text -- create a localized Text from the given namespace, key, and source string" },
	{ "LOCTABLE", PyCFunctionCast(&CreateLocalizedTextFromStringTable), METH_VARARGS, "LOCTABLE(id: Union[Name, str], key: str) -> Text -- get a localized Text from the given string table id and key" },
	{ "is_editor", PyCFunctionCast(&IsEditor), METH_NOARGS, "is_editor() -> bool -- tells if the editor is running or not" },
	{ "get_interpreter_executable_path", PyCFunctionCast(&GetInterpreterExecutablePath), METH_NOARGS, "get_interpreter_executable_path() -> str -- get the path to the Python interpreter executable of the Python SDK this plugin was compiled against" },
	{ "create_python_object_handle", PyCFunctionCast(&CreatePythonObjectHandle), METH_VARARGS, "create_python_object_handle(obj: Optional[Any]) -> Optional[PythonObjectHandle] -- create a PythonObjectHandle from the given PyObject" },
	{ "resolve_python_object_handle", PyCFunctionCast(&ResolvePythonObjectHandle), METH_VARARGS, "resolve_python_object_handle(handle: Optional[PythonObjectHandle]) -> Optional[Any] -- resolve a PythonObjectHandle to its PyObject, or None" },
	{ "destroy_python_object_handle", PyCFunctionCast(&DestroyPythonObjectHandle), METH_VARARGS, "destroy_python_object_handle(handle: Optional[PythonObjectHandle]) -- destroy a PythonObjectHandle (clearing its PyObject reference)" },
	{ "register_ticker_callback", PyCFunctionCast(&RegisterTickerCallback), METH_VARARGS | METH_KEYWORDS, "x.register_ticker_callback(callable: Union[Callable[[float], bool], DelegateBase], delay: float=0.0) -> object -- register the given callable (taking a single float and returning a bool where true will continue ticking) as a ticker callback in FTSTicker, to run at most every delay seconds" },
	{ "unregister_ticker_callback", PyCFunctionCast(&UnregisterTickerCallback), METH_VARARGS, "x.unregister_ticker_callback(handle: object) -> None -- unregister the given handle from a previous call to register_ticker_callback" },
	{ nullptr, nullptr, 0, nullptr }
};

void InitializeModule()
{
	GetPythonTypeContainerSingleton().Reset(::NewObject<UPackage>(nullptr, TEXT("/Engine/PythonTypes"), RF_Public | RF_Transient));
	GetPythonTypeContainerSingleton()->SetPackageFlags(PKG_ContainsScript);

	PyGenUtil::FNativePythonModule NativePythonModule;
	NativePythonModule.PyModuleMethods = PyCoreMethods;

	NativePythonModule.PyModule = PyImport_AddModule("_unreal_core");
	PyModule_AddFunctions(NativePythonModule.PyModule, PyCoreMethods);
	PyType_Ready(&PyDelegateHandleType);
	PyType_Ready(&PyTickerDelegateHandleType);

	if (PyType_Ready(&PyScopedSlowTaskType) == 0)
	{
		NativePythonModule.AddType(&PyScopedSlowTaskType);
	}

	if (PyType_Ready(&PyObjectIteratorType) == 0)
	{
		NativePythonModule.AddType(&PyObjectIteratorType);
	}

	if (PyType_Ready(&PyClassIteratorType) == 0)
	{
		NativePythonModule.AddType(&PyClassIteratorType);
	}

	if (PyType_Ready(&PyStructIteratorType) == 0)
	{
		NativePythonModule.AddType(&PyStructIteratorType);
	}

	if (PyType_Ready(&PyTypeIteratorType) == 0)
	{
		NativePythonModule.AddType(&PyTypeIteratorType);
	}

	if (PyType_Ready(&PyUValueDefType) == 0)
	{
		NativePythonModule.AddType(&PyUValueDefType);
	}

	if (PyType_Ready(&PyFPropertyDefType) == 0)
	{
		NativePythonModule.AddType(&PyFPropertyDefType);
	}

	if (PyType_Ready(&PyUFunctionDefType) == 0)
	{
		NativePythonModule.AddType(&PyUFunctionDefType);
	}

	InitializePyWrapperBase(NativePythonModule);
	InitializePyWrapperEnum(NativePythonModule);
	InitializePyWrapperObject(NativePythonModule);
	InitializePyWrapperStruct(NativePythonModule);
	InitializePyWrapperDelegate(NativePythonModule);
	InitializePyWrapperName(NativePythonModule);
	InitializePyWrapperText(NativePythonModule);
	InitializePyWrapperArray(NativePythonModule);
	InitializePyWrapperFixedArray(NativePythonModule);
	InitializePyWrapperSet(NativePythonModule);
	InitializePyWrapperMap(NativePythonModule);
	InitializePyWrapperFieldPath(NativePythonModule);
	InitializePyWrapperMath(NativePythonModule);

	FPyWrapperTypeRegistry::Get().RegisterNativePythonModule(MoveTemp(NativePythonModule));
}

void ShutdownModule()
{
	PyCoreUtil::ClearTickerCallbacks();
	if (GetPythonTypeContainerSingleton().IsValid())
	{
		UObject* PythonTypesPackage = GetPythonTypeContainerSingleton().Get();
		PythonTypesPackage->Rename(TEXT("PythonTypes_Garbage"), nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		PythonTypesPackage->MarkAsGarbage();
		if (PythonTypesPackage->IsRooted())
		{
			PythonTypesPackage->RemoveFromRoot();
		}
		PythonTypesPackage->ClearFlags(RF_Public | RF_Standalone);
		GetPythonTypeContainerSingleton().Reset();
	}
}

FPythonLogCapture& GetPythonLogCapture()
{
	static FPythonLogCapture PythonLogCapture;
	return PythonLogCapture;
}

}

#endif	// WITH_PYTHON
