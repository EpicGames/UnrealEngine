// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyConversion.h"
#include "PyUtil.h"
#include "PyWrapperBasic.h"
#include "PyPtr.h"
#include "Containers/Ticker.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/UObjectIterator.h"
#include "Delegates/DelegateCombinations.h"

#if WITH_PYTHON

enum class EPythonLogOutputType : uint8;

struct FSlowTask;

/** Get the object that Python created types should be outered to */
UObject* GetPythonTypeContainer();

/**
 * Create a redirector for a Python type, if needed.
 * This is for backwards compatability, as all types were historically generated under GetPythonTypeContainer() using their short name.
 * @note The caller must set DestinationObject to point at the generated type.
 * 
 * @param ShortName			The short name of the Python generated type.
 * @param GeneratedPathName The path name that the Python generated type will actually use.
 */
UObjectRedirector* CreatePythonTypeLegacyRedirector(const FString& ShortName, const FTopLevelAssetPath& GeneratedPathName);

/** Python type for FPyDelegateHandle */
extern PyTypeObject PyDelegateHandleType;

/** Python type for FPyTickerDelegateHandle */
extern PyTypeObject PyTickerDelegateHandleType;

/** Python type for FPyScopedSlowTask */
extern PyTypeObject PyScopedSlowTaskType;

/** Python type for FPyObjectIterator */
extern PyTypeObject PyObjectIteratorType;

/** Python type for FPyClassIterator */
extern PyTypeObject PyClassIteratorType;

/** Python type for FPyStructIterator */
extern PyTypeObject PyStructIteratorType;

/** Python type for FPyTypeIterator */
extern PyTypeObject PyTypeIteratorType;

/** Python type for FPyUValueDef */
extern PyTypeObject PyUValueDefType;

/** Python type for FPyFPropertyDef */
extern PyTypeObject PyFPropertyDefType;

/** Python type for FPyUFunctionDef */
extern PyTypeObject PyUFunctionDefType;

/** Type for all Unreal exposed FDelegateHandle instances */
struct FPyDelegateHandle : public TPyWrapperBasic<FDelegateHandle, FPyDelegateHandle>
{
	typedef TPyWrapperBasic<FDelegateHandle, FPyDelegateHandle> Super;

	/** Create and initialize a new wrapper instance from the given native instance */
	static FPyDelegateHandle* CreateInstance(const FDelegateHandle& InValue);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyDelegateHandle* CastPyObject(PyObject* InPyObject);
};

/** Type for all Unreal exposed FTSTicker::FDelegateHandle instances */
struct FPyTickerDelegateHandle : public TPyWrapperBasic<FTSTicker::FDelegateHandle, FPyTickerDelegateHandle>
{
	typedef TPyWrapperBasic<FTSTicker::FDelegateHandle, FPyTickerDelegateHandle> Super;

	/** Create and initialize a new wrapper instance from the given native instance */
	static FPyTickerDelegateHandle* CreateInstance(const FTSTicker::FDelegateHandle& InValue);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyTickerDelegateHandle* CastPyObject(PyObject* InPyObject);
};

/** Type used to create and managed a scoped slow task in Python */
struct FPyScopedSlowTask
{
	/** Common Python Object */
	PyObject_HEAD

	/** Internal slow task instance (created lazily due to having a custom constructor) */
	FSlowTask* SlowTask;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static FPyScopedSlowTask* New(PyTypeObject* InType);

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(FPyScopedSlowTask* InSelf);

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyScopedSlowTask* InSelf, const float InAmountOfWork, const FText& InDefaultMessage, const bool InEnabled);

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyScopedSlowTask* InSelf);

	/** Called to validate the internal state of this instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyScopedSlowTask* InSelf);
};

/** Type for iterating Unreal Object instances */
template <typename ObjectType, typename SelfType>
struct TPyObjectIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Internal iterator instance (created lazily due to having a custom constructor) */
	FPersistentThreadSafeObjectIterator* Iterator;

	/** Optional value used when filtering the iterator */
	ObjectType* IteratorFilter;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static SelfType* New(PyTypeObject* InType)
	{
		SelfType* Self = (SelfType*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->Iterator = nullptr;
			Self->IteratorFilter = nullptr;
		}
		return Self;
	}

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(SelfType* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(SelfType* InSelf, UClass* InClass, ObjectType* InIteratorFilter)
	{
		Deinit(InSelf);

		InSelf->Iterator = new FPersistentThreadSafeObjectIterator(InClass);
		InSelf->IteratorFilter = InIteratorFilter;

		FPersistentThreadSafeObjectIterator& Iter = *InSelf->Iterator;
		while (*Iter && !SelfType::PassesFilter(InSelf))
		{
			++Iter;
		}

		return 0;
	}

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(SelfType* InSelf)
	{
		delete InSelf->Iterator;
		InSelf->Iterator = nullptr;

		InSelf->IteratorFilter = nullptr;
	}

	/** Called to validate the internal state of this instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(SelfType* InSelf)
	{
		if (!InSelf->Iterator)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - Iterator is null!"));
			return false;
		}

		return true;
	}

	/** Get the iterator */
	static SelfType* GetIter(SelfType* InSelf)
	{
		Py_INCREF(InSelf);
		return InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(SelfType* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		FPersistentThreadSafeObjectIterator& Iter = *InSelf->Iterator;
		if (*Iter)
		{
			PyObject* PyIterObj = SelfType::GetIterValue(InSelf);
			do
			{
				++Iter;
			}
			while (*Iter && !SelfType::PassesFilter(InSelf));
			return PyIterObj;
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}

	/** Convert the current iterator value to a Python object (internal: define this in a derived type to "override" GetIterValue behavior) */
	static PyObject* GetIterValue(SelfType* InSelf)
	{
		FPersistentThreadSafeObjectIterator& Iter = *InSelf->Iterator;
		return PyConversion::Pythonize(*Iter);
	}

	/** True if the current iterator value passes the filter (internal: define this in a derived type to "override" PassesFilter behavior) */
	static bool PassesFilter(SelfType* InSelf)
	{
		return true;
	}
};

/** Type for iterating Unreal Object instances */
struct FPyObjectIterator : public TPyObjectIterator<UObject, FPyObjectIterator>
{
	typedef TPyObjectIterator<UObject, FPyObjectIterator> Super;
};

/** Type for iterating Unreal class types */
struct FPyClassIterator : public TPyObjectIterator<UClass, FPyClassIterator>
{
	typedef TPyObjectIterator<UClass, FPyClassIterator> Super;

	/** True if the current iterator value passes the filter */
	static bool PassesFilter(FPyClassIterator* InSelf);

	/** Extract the filter from the given Python object */
	static UClass* ExtractFilter(FPyClassIterator* InSelf, PyObject* InPyFilter);
};

/** Type for iterating Unreal struct types */
struct FPyStructIterator : public TPyObjectIterator<UScriptStruct, FPyStructIterator>
{
	typedef TPyObjectIterator<UScriptStruct, FPyStructIterator> Super;

	/** True if the current iterator value passes the filter */
	static bool PassesFilter(FPyStructIterator* InSelf);

	/** Extract the filter from the given Python object */
	static UScriptStruct* ExtractFilter(FPyStructIterator* InSelf, PyObject* InPyFilter);
};

/** Type for iterating Python types */
struct FPyTypeIterator : public TPyObjectIterator<UStruct, FPyTypeIterator>
{
	typedef TPyObjectIterator<UStruct, FPyTypeIterator> Super;

	/** Convert the current iterator value to a Python object */
	static PyObject* GetIterValue(FPyTypeIterator* InSelf);

	/** True if the current iterator value passes the filter */
	static bool PassesFilter(FPyTypeIterator* InSelf);

	/** Extract the filter from the given Python object */
	static UStruct* ExtractFilter(FPyTypeIterator* InSelf, PyObject* InPyFilter);
};

/** Type used to define constant values from Python */
struct FPyUValueDef
{
	/** Common Python Object */
	PyObject_HEAD

	/** Value of this definition */
	PyObject* Value;

	/** Dictionary of meta-data associated with this value */
	PyObject* MetaData;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static FPyUValueDef* New(PyTypeObject* InType);

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(FPyUValueDef* InSelf);

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyUValueDef* InSelf, PyObject* InValue, PyObject* InMetaData);

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyUValueDef* InSelf);

	/** Apply the meta-data on this instance via the given predicate */
	static void ApplyMetaData(FPyUValueDef* InSelf, const TFunctionRef<void(const FString&, const FString&)>& InPredicate);
};

/** Type used to define FProperty fields from Python */
struct FPyFPropertyDef
{
	/** Common Python Object */
	PyObject_HEAD

	/** Type of this property */
	PyObject* PropType;

	/** Dictionary of meta-data associated with this property */
	PyObject* MetaData;

	/** Getter function to use with this property */
	FString GetterFuncName;

	/** Setter function to use with this property */
	FString SetterFuncName;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static FPyFPropertyDef* New(PyTypeObject* InType);

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(FPyFPropertyDef* InSelf);

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyFPropertyDef* InSelf, PyObject* InPropType, PyObject* InMetaData, FString InGetterFuncName, FString InSetterFuncName);

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyFPropertyDef* InSelf);

	/** Apply the meta-data on this instance to the given property */
	static void ApplyMetaData(FPyFPropertyDef* InSelf, FProperty* InProp);
};

/** Flags used to define the attributes of a UFunction field from Python */
enum class EPyUFunctionDefFlags : uint8
{
	None = 0,
	Override = 1<<0,
	Static = 1<<1,
	Pure = 1<<2,
	Impure = 1<<3,
	Getter = 1<<4,
	Setter = 1<<5,
};
ENUM_CLASS_FLAGS(EPyUFunctionDefFlags);

/** Type used to define UFunction fields from Python */
struct FPyUFunctionDef
{
	/** Common Python Object */
	PyObject_HEAD

	/** Python function to call */
	PyObject* Func;

	/** Return type of this function */
	PyObject* FuncRetType;

	/** List of types for each parameter of this function */
	PyObject* FuncParamTypes;

	/** Dictionary of meta-data associated with this function */
	PyObject* MetaData;

	/** Flags used to define this function */
	EPyUFunctionDefFlags FuncFlags;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static FPyUFunctionDef* New(PyTypeObject* InType);

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(FPyUFunctionDef* InSelf);

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyUFunctionDef* InSelf, PyObject* InFunc, PyObject* InFuncRetType, PyObject* InFuncParamTypes, PyObject* InMetaData, EPyUFunctionDefFlags InFuncFlags);

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyUFunctionDef* InSelf);

	/** Apply the meta-data on this instance to the given function */
	static void ApplyMetaData(FPyUFunctionDef* InSelf, UFunction* InFunc);
};

typedef TPyPtr<FPyDelegateHandle> FPyDelegateHandlePtr;
typedef TPyPtr<FPyTickerDelegateHandle> FPyTickerDelegateHandlePtr;
typedef TPyPtr<FPyScopedSlowTask> FPyScopedSlowTaskPtr;
typedef TPyPtr<FPyObjectIterator> FPyObjectIteratorPtr;
typedef TPyPtr<FPyClassIterator> FPyClassIteratorPtr;
typedef TPyPtr<FPyStructIterator> FPyStructIteratorPtr;
typedef TPyPtr<FPyTypeIterator> FPyTypeIteratorPtr;
typedef TPyPtr<FPyUValueDef> FPyUValueDefPtr;
typedef TPyPtr<FPyFPropertyDef> FPyFPropertyDefPtr;
typedef TPyPtr<FPyUFunctionDef> FPyUFunctionDefPtr;

namespace PyCore
{
	void InitializeModule();
	void ShutdownModule();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FPythonLogCapture, EPythonLogOutputType, const TCHAR*);
	FPythonLogCapture& GetPythonLogCapture();
}

#endif	// WITH_PYTHON
