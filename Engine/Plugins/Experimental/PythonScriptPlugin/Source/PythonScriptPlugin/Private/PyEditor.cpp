// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyEditor.h"
#include "PyUtil.h"
#include "PyGenUtil.h"
#include "PyConversion.h"
#include "PyWrapperTypeRegistry.h"

#if WITH_EDITOR
#include "Editor.h"
#endif	// WITH_EDITOR

#if WITH_PYTHON

#if WITH_EDITOR

FPyScopedEditorTransaction* FPyScopedEditorTransaction::New(PyTypeObject* InType)
{
	FPyScopedEditorTransaction* Self = (FPyScopedEditorTransaction*)InType->tp_alloc(InType, 0);
	if (Self)
	{
		new(&Self->Description) FText();
		Self->PendingTransactionId = INDEX_NONE;
	}
	return Self;
}

void FPyScopedEditorTransaction::Free(FPyScopedEditorTransaction* InSelf)
{
	Deinit(InSelf);

	InSelf->Description.~FText();
	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

int FPyScopedEditorTransaction::Init(FPyScopedEditorTransaction* InSelf, const FText& InDescription)
{
	Deinit(InSelf);

	InSelf->Description = InDescription;

	return 0;
}

void FPyScopedEditorTransaction::Deinit(FPyScopedEditorTransaction* InSelf)
{
	InSelf->Description = FText();
	InSelf->PendingTransactionId = INDEX_NONE;
}

PyTypeObject InitializePyScopedEditorTransactionType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyScopedEditorTransaction::New(InType);
		}

		static void Dealloc(FPyScopedEditorTransaction* InSelf)
		{
			FPyScopedEditorTransaction::Free(InSelf);
		}

		static int Init(FPyScopedEditorTransaction* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyDescObj = nullptr;

			static const char *ArgsKwdList[] = { "desc", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:call", (char**)ArgsKwdList, &PyDescObj))
			{
				return -1;
			}

			FText Desc;
			if (!PyConversion::Nativize(PyDescObj, Desc))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'desc' (%s) to 'Text'"), *PyUtil::GetFriendlyTypename(PyDescObj)));
				return -1;
			}

			return FPyScopedEditorTransaction::Init(InSelf, Desc);
		}
	};

	struct FMethods
	{
		static PyObject* EnterScope(FPyScopedEditorTransaction* InSelf)
		{
			ensure(InSelf->PendingTransactionId == INDEX_NONE);
			InSelf->PendingTransactionId = GEditor->BeginTransaction(InSelf->Description);

			Py_INCREF(InSelf);
			return (PyObject*)InSelf;
		}

		static PyObject* ExitScope(FPyScopedEditorTransaction* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (InSelf->PendingTransactionId != INDEX_NONE)
			{
				GEditor->EndTransaction();
				InSelf->PendingTransactionId = INDEX_NONE;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Cancel(FPyScopedEditorTransaction* InSelf)
		{
			if (InSelf->PendingTransactionId != INDEX_NONE)
			{
				GEditor->CancelTransaction(InSelf->PendingTransactionId);
				InSelf->PendingTransactionId = INDEX_NONE;
			}

			Py_RETURN_NONE;
		}
	};

	static PyMethodDef PyMethods[] = {
		{ "__enter__", PyCFunctionCast(&FMethods::EnterScope), METH_NOARGS, "__enter__(self) -> ScopedEditorTransaction -- begin this transaction" },
		{ "__exit__", PyCFunctionCast(&FMethods::ExitScope), METH_VARARGS | METH_KEYWORDS, "__exit__(self, type: Optional[Type[BaseException]], value: Optional[BaseException], traceback: Optional[TracebackType]) -> bool -- end this transaction" },
		{ "cancel", PyCFunctionCast(&FMethods::Cancel), METH_NOARGS, "cancel(self) -> None -- cancel this transaction" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"ScopedEditorTransaction", /* tp_name */
		sizeof(FPyScopedEditorTransaction), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type used to create and managed a scoped editor transaction in Python";

	return PyType;
}

PyTypeObject PyScopedEditorTransactionType = InitializePyScopedEditorTransactionType();

namespace PyEditor
{

static PyObject* GetEngineSubsystem(FPyScopedEditorTransaction* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:get_engine_subsystem", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	UClass* Class = nullptr;
	if (!PyConversion::NativizeClass(PyObj, Class, UEngineSubsystem::StaticClass()))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_engine_subsystem"), *FString::Printf(TEXT("Parameter must be a 'Class' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	if (GEditor)
	{
		if (USubsystem* Subsystem = GEditor->GetEngineSubsystemBase(Class))
		{
			return PyConversion::Pythonize(Subsystem);
		}
	}

	Py_RETURN_NONE;
}

static PyObject* GetEditorSubsystem(FPyScopedEditorTransaction* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:get_editor_subsystem", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	UClass* Class = nullptr;
	if (!PyConversion::NativizeClass(PyObj, Class, UEditorSubsystem::StaticClass()))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_editor_subsystem"), *FString::Printf(TEXT("Parameter must be a 'Class' not '%s'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	if (GEditor)
	{
		if (USubsystem* Subsystem = GEditor->GetEditorSubsystemBase(Class))
		{
			return PyConversion::Pythonize(Subsystem);
		}
	}

	Py_RETURN_NONE;
}

PyMethodDef PyEditorMethods[] = {
	{ "get_engine_subsystem", PyCFunctionCast(&PyEditor::GetEngineSubsystem), METH_VARARGS, "get_engine_subsystem(subsystem : Union[Class, type[_EngineSubsystemTypeVar]]) -> Optional[_EngineSubsystemTypeVar] -- returns the requested Engine subsystem or None" },
	{ "get_editor_subsystem", PyCFunctionCast(&PyEditor::GetEditorSubsystem), METH_VARARGS, "get_editor_subsystem(subsystem : Union[Class, type[_EditorSubsystemTypeVar]]) -> Optional[_EditorSubsystemTypeVar] -- returns the requested Editor subsystem or None" },
	{ nullptr, nullptr, 0, nullptr }
};

void InitializeModule()
{
	PyGenUtil::FNativePythonModule NativePythonModule;
	NativePythonModule.PyModuleMethods = PyEditorMethods;

	NativePythonModule.PyModule = PyImport_AddModule("_unreal_editor");
	PyModule_AddFunctions(NativePythonModule.PyModule, PyEditorMethods);

	if (PyType_Ready(&PyScopedEditorTransactionType) == 0)
	{
		NativePythonModule.AddType(&PyScopedEditorTransactionType);
	}

	FPyWrapperTypeRegistry::Get().RegisterNativePythonModule(MoveTemp(NativePythonModule));
}

void ShutdownModule()
{
}

}

#endif	// WITH_EDITOR

#endif	// WITH_PYTHON
