// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"
#include "CoreMinimal.h"

#if WITH_PYTHON

#if WITH_EDITOR

/** Python type for FPyScopedEditorTransaction */
extern PyTypeObject PyScopedEditorTransactionType;

/** Type used to create and managed a scoped editor transaction in Python */
struct FPyScopedEditorTransaction
{
	/** Common Python Object */
	PyObject_HEAD

	/** Description of the transaction */
	FText Description;

	/** Id of the pending transaction, or INDEX_NONE if there is no pending transaction */
	int32 PendingTransactionId;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static FPyScopedEditorTransaction* New(PyTypeObject* InType);

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(FPyScopedEditorTransaction* InSelf);

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyScopedEditorTransaction* InSelf, const FText& InDescription);

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyScopedEditorTransaction* InSelf);
};

typedef TPyPtr<FPyScopedEditorTransaction> FPyScopedEditorTransactionPtr;

namespace PyEditor
{
	void InitializeModule();
	void ShutdownModule();
}

#endif	// WITH_EDITOR

#endif	// WITH_PYTHON
