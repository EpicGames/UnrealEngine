// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyEngine.h"
#include "PyGenUtil.h"
#include "PyWrapperTypeRegistry.h"

#include "EngineUtils.h"

#if WITH_PYTHON

template <typename IteratorType, typename SelfType>
PyTypeObject InitializePyActorIteratorType(const char* InTypeName, const char* InTypeDoc)
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)SelfType::New(InType);
		}

		static void Dealloc(SelfType* InSelf)
		{
			SelfType::Free(InSelf);
		}

		static int Init(SelfType* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyWorldObj = nullptr;
			PyObject* PyTypeObj = nullptr;

			static const char *ArgsKwdList[] = { "world", "type", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:call", (char**)ArgsKwdList, &PyWorldObj, &PyTypeObj))
			{
				return -1;
			}

			UWorld* IterWorld = nullptr;
			if (!PyConversion::Nativize(PyWorldObj, IterWorld))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'world' (%s) to 'World'"), *PyUtil::GetFriendlyTypename(PyWorldObj)));
				return -1;
			}
			if (!IterWorld)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("'world' cannot be 'None'"));
				return -1;
			}

			UClass* IterClass = AActor::StaticClass();
			if (PyTypeObj && !PyConversion::NativizeClass(PyTypeObj, IterClass, AActor::StaticClass()))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'type' (%s) to 'Class'"), *PyUtil::GetFriendlyTypename(PyTypeObj)));
				return -1;
			}

			return SelfType::Init(InSelf, IterWorld, IterClass);
		}

		static SelfType* GetIter(SelfType* InSelf)
		{
			return SelfType::GetIter(InSelf);
		}

		static PyObject* IterNext(SelfType* InSelf)
		{
			return SelfType::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		InTypeName, /* tp_name */
		sizeof(SelfType), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = InTypeDoc;

	return PyType;
}


PyTypeObject PyActorIteratorType = InitializePyActorIteratorType<FActorIterator, FPyActorIterator>("ActorIterator", "Type for iterating Unreal actor instances");
PyTypeObject PySelectedActorIteratorType = InitializePyActorIteratorType<FSelectedActorIterator, FPySelectedActorIterator>("SelectedActorIterator", "Type for iterating selected Unreal actor instances");


namespace PyEngine
{

PyObject* GetBlueprintGeneratedTypes(PyObject* InSelf, PyObject* InArgs)
{
	TArray<FString> AssetsToGenerate;

	// Extract the assets to generate
	if (InArgs)
	{
		const Py_ssize_t ArgsLen = PyTuple_Size(InArgs);
		for (Py_ssize_t ArgIndex = 0; ArgIndex < ArgsLen; ++ArgIndex)
		{
			PyObject* PyArg = PyTuple_GetItem(InArgs, ArgIndex);
			if (PyArg)
			{
				// Is this some kind of container, or a single value?
				const bool bIsStringType = static_cast<bool>(PyUnicode_Check(PyArg));
				if (!bIsStringType && PyUtil::HasLength(PyArg))
				{
					const Py_ssize_t SequenceLen = PyObject_Length(PyArg);
					check(SequenceLen != -1);

					FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(PyArg));
					if (PyObjIter)
					{
						// Conversion from a sequence
						for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
						{
							FPyObjectPtr ValueItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
							if (!ValueItem)
							{
								return nullptr;
							}

							FString& AssetToGenerate = AssetsToGenerate.AddDefaulted_GetRef();
							if (!PyConversion::Nativize(ValueItem, AssetToGenerate))
							{
								PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_blueprint_generated_types"), *FString::Printf(TEXT("Cannot convert argument %zd (%s) at index %zd to 'string'"), ArgIndex, *PyUtil::GetFriendlyTypename(PyArg), SequenceIndex));
								return nullptr;
							}
						}
					}
				}
				else
				{
					FString& AssetToGenerate = AssetsToGenerate.AddDefaulted_GetRef();
					if (!PyConversion::Nativize(PyArg, AssetToGenerate))
					{
						PyUtil::SetPythonError(PyExc_TypeError, TEXT("get_blueprint_generated_types"), *FString::Printf(TEXT("Cannot convert argument %zd (%s) to 'string'"), ArgIndex, *PyUtil::GetFriendlyTypename(PyArg)));
						return nullptr;
					}
				}
			}
		}
	}

	if (AssetsToGenerate.Num() == 0)
	{
		Py_RETURN_NONE;
	}

	FPyWrapperTypeRegistry& PyWrapperTypeRegistry = FPyWrapperTypeRegistry::Get();
	FPyWrapperTypeRegistry::FGeneratedWrappedTypeReferences GeneratedWrappedTypeReferences;
	TSet<FName> DirtyModules;

	// Process each asset
	TArray<PyTypeObject*> WrappedAssets;
	for (const FString& AssetToGenerate : AssetsToGenerate)
	{
		const UObject* LoadedAsset = StaticLoadObject(UObject::StaticClass(), nullptr, *AssetToGenerate);
		if (!LoadedAsset)
		{
			PyUtil::SetPythonError(PyExc_RuntimeError, TEXT("get_blueprint_generated_types"), *FString::Printf(TEXT("Cannot find asset for '%s'"), *AssetToGenerate));
			return nullptr;
		}
		LoadedAsset = PyGenUtil::GetAssetTypeRegistryType(LoadedAsset);

		PyTypeObject* WrappedAsset = WrappedAssets.Add_GetRef(PyWrapperTypeRegistry.GenerateWrappedTypeForObject(LoadedAsset, GeneratedWrappedTypeReferences, DirtyModules, EPyTypeGenerationFlags::IncludeBlueprintGeneratedTypes));
		if (!WrappedAsset)
		{
			PyUtil::SetPythonError(PyExc_RuntimeError, TEXT("get_blueprint_generated_types"), *FString::Printf(TEXT("Asset '%s' (%s) cannot be wrapped for Python"), *AssetToGenerate, *LoadedAsset->GetName()));
			return nullptr;
		}
	}
	check(AssetsToGenerate.Num() == WrappedAssets.Num());

	PyWrapperTypeRegistry.GenerateWrappedTypesForReferences(GeneratedWrappedTypeReferences, DirtyModules);
	PyWrapperTypeRegistry.NotifyModulesDirtied(DirtyModules);

	check(WrappedAssets.Num() > 0);
	if (WrappedAssets.Num() == 1)
	{
		// Return the single object
		Py_INCREF(WrappedAssets[0]);
		return (PyObject*)WrappedAssets[0];
	}
	else
	{
		// Return the result as a tuple
		FPyObjectPtr PyWrappedAssets = FPyObjectPtr::StealReference(PyTuple_New(WrappedAssets.Num()));
		for (int32 WrappedAssetIndex = 0; WrappedAssetIndex < WrappedAssets.Num(); ++WrappedAssetIndex)
		{
			Py_INCREF(WrappedAssets[WrappedAssetIndex]);
			PyTuple_SetItem(PyWrappedAssets, WrappedAssetIndex, (PyObject*)WrappedAssets[WrappedAssetIndex]); // SetItem steals the reference
		}
		return PyWrappedAssets.Release();
	}
}

PyMethodDef PyEngineMethods[] = {
	{ "get_blueprint_generated_types", PyCFunctionCast(&GetBlueprintGeneratedTypes), METH_VARARGS, "get_blueprint_generated_types(paths: Iterable[str]) -> Optional[Union[type, Tuple[type, ...]]] -- get the Python types (will return a tuple for multiple types) for the given set of Blueprint asset paths (may be a sequence type or set of arguments)" },
	{ nullptr, nullptr, 0, nullptr }
};

void InitializeModule()
{
	PyGenUtil::FNativePythonModule NativePythonModule;
	NativePythonModule.PyModuleMethods = PyEngineMethods;

	NativePythonModule.PyModule = PyImport_AddModule("_unreal_engine");
	PyModule_AddFunctions(NativePythonModule.PyModule, PyEngineMethods);

	if (PyType_Ready(&PyActorIteratorType) == 0)
	{
		NativePythonModule.AddType(&PyActorIteratorType);
	}

	if (PyType_Ready(&PySelectedActorIteratorType) == 0)
	{
		NativePythonModule.AddType(&PySelectedActorIteratorType);
	}

	FPyWrapperTypeRegistry::Get().RegisterNativePythonModule(MoveTemp(NativePythonModule));
}

void ShutdownModule()
{
}

}

#endif	// WITH_PYTHON
