// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyConversion.h"
#include "PyPtr.h"
#include "PyUtil.h"
#include "CoreMinimal.h"

#if WITH_PYTHON

class FActorIterator;
class FSelectedActorIterator;

/** Python type for FPyActorIterator */
extern PyTypeObject PyActorIteratorType;

/** Python type for FPySelectedActorIterator */
extern PyTypeObject PySelectedActorIteratorType;

/** Type for iterating Unreal actor instances */
template <typename IteratorType, typename SelfType>
struct TPyActorIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Internal iterator instance (created lazily due to having a custom constructor) */
	IteratorType* Iterator;

	/** New this instance (called via tp_new for Python, or directly in C++) */
	static SelfType* New(PyTypeObject* InType)
	{
		SelfType* Self = (SelfType*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->Iterator = nullptr;
		}
		return Self;
	}

	/** Free this instance (called via tp_dealloc for Python) */
	static void Free(SelfType* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this instance (called via tp_init for Python, or directly in C++) */
	static int Init(SelfType* InSelf, UWorld* InWorld, UClass* InClass)
	{
		Deinit(InSelf);

		InSelf->Iterator = new IteratorType(InWorld, InClass);

		return 0;
	}

	/** Deinitialize this instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(SelfType* InSelf)
	{
		delete InSelf->Iterator;
		InSelf->Iterator = nullptr;
	}

	/** Called to validate the internal state of this instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(SelfType* InSelf)
	{
		if (!InSelf->Iterator)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - Iterator is null!"));
			return false;
		}

		return true;
	}

	/** Get the iterator */
	static SelfType* GetIter(SelfType* InSelf)
	{
		Py_INCREF(InSelf);
		return InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(SelfType* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		IteratorType& Iter = *InSelf->Iterator;
		if (Iter)
		{
			PyObject* PyIterObj = PyConversion::Pythonize(*Iter);
			++Iter;
			return PyIterObj;
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}
};

/** Type for iterating Unreal actor instances */
struct FPyActorIterator : public TPyActorIterator<FActorIterator, FPyActorIterator>
{
	typedef TPyActorIterator<FActorIterator, FPyActorIterator> Super;
};

/** Type for iterating selected Unreal actor instances */
struct FPySelectedActorIterator : public TPyActorIterator<FSelectedActorIterator, FPySelectedActorIterator>
{
	typedef TPyActorIterator<FSelectedActorIterator, FPySelectedActorIterator> Super;
};

typedef TPyPtr<FPyActorIterator> FPyActorIteratorPtr;
typedef TPyPtr<FPySelectedActorIterator> FPySelectedActorIteratorPtr;

namespace PyEngine
{
	void InitializeModule();
	void ShutdownModule();
}

#endif	// WITH_PYTHON
