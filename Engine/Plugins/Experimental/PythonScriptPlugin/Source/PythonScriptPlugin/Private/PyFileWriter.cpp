// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyFileWriter.h"
#include "PyGenUtil.h"

#if WITH_PYTHON

FPyFileWriter::FPyFileWriter()
	: Indentation(0)
{
}

void FPyFileWriter::WriteLine(const TCHAR* InStr)
{
	WriteIndentation();
	Write(InStr);
	WriteNewLine();
}

void FPyFileWriter::WriteLine(const FString& InStr)
{
	WriteIndentation();
	Write(InStr);
	WriteNewLine();
}

void FPyFileWriter::WriteIndentation()
{
	for (int32 Idx = 0; Idx < Indentation; ++Idx)
	{
		FileContents += TEXT("    ");
	}
}

void FPyFileWriter::WriteNewLine()
{
	FileContents += LINE_TERMINATOR;
}

void FPyFileWriter::Write(const TCHAR* InStr)
{
	FileContents += InStr;
}

void FPyFileWriter::Write(const FString& InStr)
{
	FileContents += InStr;
}

void FPyFileWriter::WriteDocString(const TCHAR* InDocString)
{
	if (InDocString && *InDocString)
	{
		WriteDocString(FString(InDocString));
	}
}

void FPyFileWriter::WriteDocString(const FString& InDocString)
{
	if (InDocString.IsEmpty())
	{
		return;
	}

	TArray<FString> DocStringLines;
	InDocString.ParseIntoArrayLines(DocStringLines, false);

	WriteLine(TEXT("r\"\"\""));
	for (const FString& DocStringLine : DocStringLines)
	{
		WriteLine(DocStringLine);
	}
	WriteLine(TEXT("\"\"\""));
}

void FPyFileWriter::IncreaseIndent(const int32 InCount)
{
	Indentation += InCount;
}

void FPyFileWriter::DecreaseIndent(const int32 InCount)
{
	Indentation -= InCount;
	check(Indentation >= 0);
}

bool FPyFileWriter::SaveFile(const TCHAR* InFilename)
{
	return PyGenUtil::SaveGeneratedTextFile(InFilename, FileContents);
}

#endif	// WITH_PYTHON
