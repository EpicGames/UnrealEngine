// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"


#if WITH_PYTHON


/** Util class to help format and write a Python file to disk */
class FPyFileWriter
{
public:
	FPyFileWriter();

	/** Write a single line to the file - this is equivalent to calling WriteIndentation, Write, then WriteNewLine */
	void WriteLine(const TCHAR* InStr);

	/** Write a single line to the file - this is equivalent to calling WriteIndentation, Write, then WriteNewLine */
	void WriteLine(const FString& InStr);

	/** Write the current indentation level to the file */
	void WriteIndentation();

	/** Write a new-line to the file */
	void WriteNewLine();

	/** Write the given string to the file */
	void Write(const TCHAR* InStr);

	/** Write the given string to the file */
	void Write(const FString& InStr);

	/** Write a doc string to the file */
	void WriteDocString(const TCHAR* InDocString);

	/** Write a doc string to the file */
	void WriteDocString(const FString& InDocString);

	/** Increase the indentation level */
	void IncreaseIndent(const int32 InCount = 1);

	/** Decrease the indentation level */
	void DecreaseIndent(const int32 InCount = 1);

	/** Save the file to disk */
	bool SaveFile(const TCHAR* InFilename);

private:
	int32 Indentation;
	FString FileContents;
};


#endif	// WITH_PYTHON
