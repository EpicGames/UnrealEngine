// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"

#if WITH_PYTHON

/** Utility to handle taking a releasing the Python GIL within a scope */
class FPyScopedGIL
{
public:
	/** Constructor - take the GIL */
	FPyScopedGIL()
		: GILState(PyGILState_Ensure())
	{
	}

	/** Destructor - release the GIL */
	~FPyScopedGIL()
	{
		PyGILState_Release(GILState);
	}

	/** Non-copyable */
	FPyScopedGIL(const FPyScopedGIL&) = delete;
	FPyScopedGIL& operator=(const FPyScopedGIL&) = delete;

private:
	/** Internal GIL state */
	PyGILState_STATE GILState;
};

/** Wrapper of a TPyPtr that can be safely copied/moved/destroyed by code where the GIL might not currently be held (eg, a lambda bound to a delegate) */
template <typename TPythonType>
class TPyAutoGILPtr
{
public:
	TPyAutoGILPtr() = default;

	explicit TPyAutoGILPtr(TPyPtr<TPythonType>&& InPtr)
		: Ptr(MoveTemp(InPtr))
	{
	}

	TPyAutoGILPtr(const TPyAutoGILPtr& InOther)
	{
		// This may be called after Python has already shut down
		if (Py_IsInitialized())
		{
			FPyScopedGIL GIL;
			Ptr = InOther.Ptr;
		}
	}

	TPyAutoGILPtr(TPyAutoGILPtr&& InOther)
	{
		// This may be called after Python has already shut down
		if (Py_IsInitialized())
		{
			FPyScopedGIL GIL;
			Ptr = MoveTemp(InOther.Ptr);
		}
	}

	~TPyAutoGILPtr()
	{
		// This may be called after Python has already shut down
		if (Py_IsInitialized())
		{
			FPyScopedGIL GIL;
			Ptr.Reset();
		}
		else
		{
			// Release ownership if Python has been shut down to avoid attempting to delete the object (which is already dead)
			Ptr.Release();
		}
	}

	TPyAutoGILPtr& operator=(const TPyAutoGILPtr& InOther)
	{
		if (this != &InOther)
		{
			// This may be called after Python has already shut down
			if (Py_IsInitialized())
			{
				FPyScopedGIL GIL;
				Ptr = InOther.Ptr;
			}
			else
			{
				// Release ownership if Python has been shut down to avoid attempting to delete the object (which is already dead)
				Ptr.Release();
			}
		}
		return *this;
	}

	TPyAutoGILPtr& operator=(TPyAutoGILPtr&& InOther)
	{
		if (this != &InOther)
		{
			// This may be called after Python has already shut down
			if (Py_IsInitialized())
			{
				FPyScopedGIL GIL;
				Ptr = MoveTemp(InOther.Ptr);
			}
			else
			{
				// Release ownership if Python has been shut down to avoid attempting to delete the object (which is already dead)
				Ptr.Release();
			}
		}
		return *this;
	}

	TPyPtr<TPythonType>& Get()
	{
		return Ptr;
	}

	const TPyPtr<TPythonType>& Get() const
	{
		return Ptr;
	}

private:
	TPyPtr<TPythonType> Ptr;
};

typedef TPyAutoGILPtr<PyObject> FPyAutoGILPtr;

#endif	// WITH_PYTHON
