// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyGenUtil.h"
#include "PyUtil.h"
#include "PyGIL.h"
#include "PyConversion.h"
#include "PyWrapperBase.h"
#include "PyWrapperEnum.h"
#include "PyWrapperStruct.h"
#include "Internationalization/BreakIterator.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/StringBuilder.h"
#include "UObject/Class.h"
#include "UObject/Stack.h"
#include "UObject/Package.h"
#include "UObject/EnumProperty.h"
#include "UObject/NameTypes.h"
#include "UObject/TextProperty.h"
#include "UObject/CoreRedirects.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "Async/EventCount.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Engine/BlueprintCore.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Engine/UserDefinedEnum.h"
#include "GameFramework/Actor.h"
#include "Interfaces/IPluginManager.h"

#if WITH_PYTHON

namespace PyGenUtil
{

/** Case sensitive hashing function for TSet */
struct FCaseSensitiveStringSetFuncs : BaseKeyFuncs<FString, FString>
{
	static FORCEINLINE const FString& GetSetKey(const FString& Element)
	{
		return Element;
	}
	static FORCEINLINE bool Matches(const FString& A, const FString& B)
	{
		return A.Equals(B, ESearchCase::CaseSensitive);
	}
	static FORCEINLINE uint32 GetKeyHash(const FString& Key)
	{
		return FCrc::StrCrc32<TCHAR>(*Key);
	}
};


const FName ScriptNameMetaDataKey = TEXT("ScriptName");
const FName ScriptNoExportMetaDataKey = TEXT("ScriptNoExport");
const FName ScriptMethodMetaDataKey = TEXT("ScriptMethod");
const FName ScriptMethodSelfReturnMetaDataKey = TEXT("ScriptMethodSelfReturn");
const FName ScriptMethodMutableMetaDataKey = TEXT("ScriptMethodMutable");
const FName ScriptOperatorMetaDataKey = TEXT("ScriptOperator");
const FName ScriptConstantMetaDataKey = TEXT("ScriptConstant");
const FName ScriptConstantHostMetaDataKey = TEXT("ScriptConstantHost");
const FName ScriptDefaultMakeMetaDataKey = TEXT("ScriptDefaultMake");
const FName ScriptDefaultBreakMetaDataKey = TEXT("ScriptDefaultBreak");
const FName BlueprintTypeMetaDataKey = TEXT("BlueprintType");
const FName NotBlueprintTypeMetaDataKey = TEXT("NotBlueprintType");
const FName BlueprintSpawnableComponentMetaDataKey = TEXT("BlueprintSpawnableComponent");
const FName BlueprintGetterMetaDataKey = TEXT("BlueprintGetter");
const FName BlueprintSetterMetaDataKey = TEXT("BlueprintSetter");
const FName BlueprintInternalUseOnlyMetaDataKey = TEXT("BlueprintInternalUseOnly");
const FName BlueprintInternalUseOnlyHierarchicalMetaDataKey = TEXT("BlueprintInternalUseOnlyHierarchical");
const FName CustomThunkMetaDataKey = TEXT("CustomThunk");
const FName DeprecatedPropertyMetaDataKey = TEXT("DeprecatedProperty");
const FName DeprecatedFunctionMetaDataKey = TEXT("DeprecatedFunction");
const FName DeprecationMessageMetaDataKey = TEXT("DeprecationMessage");
const FName HasNativeMakeMetaDataKey = TEXT("HasNativeMake");
const FName HasNativeBreakMetaDataKey = TEXT("HasNativeBreak");
const FName NativeBreakFuncMetaDataKey = TEXT("NativeBreakFunc");
const FName NativeMakeFuncMetaDataKey = TEXT("NativeMakeFunc");
const FName ReturnValueKey = TEXT("ReturnValue");
const TCHAR* HiddenMetaDataKey = TEXT("Hidden");

static thread_local TSharedPtr<IBreakIterator> NameBreakIterator;
static ETypeHintingMode GTypeHintingMode = ETypeHintingMode::Off; // Use through SetTypeHintingMode()/IsTypeHintingEnabled()

namespace MultithreadedGenerationImpl
{
	static UE::FEventCount EventCount;
	static UE::TConsumeAllMpmcQueue<TFunction<void()>> GameThreadTasks;
}

FMultithreadedGenerationContext::FMultithreadedGenerationContext()
{
	check(IsInGameThread());
	check(GetMultithreadedGenerationRef() == false);
	GetMultithreadedGenerationRef() = true;
}

FMultithreadedGenerationContext::~FMultithreadedGenerationContext()
{
	check(MultithreadedGenerationImpl::GameThreadTasks.IsEmpty());
	GetMultithreadedGenerationRef() = false;
}

bool FMultithreadedGenerationContext::IsMultithreadedGenerationActive()
{
	return GetMultithreadedGenerationRef();
}

void FMultithreadedGenerationContext::EnqueueGameThreadTask(TFunction<void()>&& Task)
{
	if (IsInGameThread())
	{
		Task();
	}
	else
	{
		check(IsMultithreadedGenerationActive());
		MultithreadedGenerationImpl::GameThreadTasks.ProduceItem(MoveTemp(Task));
		MultithreadedGenerationImpl::EventCount.Notify();
	}
}

void FMultithreadedGenerationContext::ExecuteGameThreadTask(TFunctionRef<void()> Task)
{
	if (IsInGameThread())
	{
		Task();
	}
	else
	{
		check(IsMultithreadedGenerationActive());
		UE::FManualResetEvent Event;
		MultithreadedGenerationImpl::GameThreadTasks.ProduceItem(
			[&Task, &Event]()
			{
				Task();
				Event.Notify();
			}
		);
		MultithreadedGenerationImpl::EventCount.Notify();
		Event.Wait();
	}
}

void FMultithreadedGenerationContext::ExecuteGameThreadTasksAndWait()
{
	bool bWasEmpty = !ExecuteGameThreadTasks();

	if (bWasEmpty)
	{
		UE::FEventCountToken Token = MultithreadedGenerationImpl::EventCount.PrepareWait();
		if (MultithreadedGenerationImpl::GameThreadTasks.IsEmpty())
		{
			MultithreadedGenerationImpl::EventCount.Wait(Token);
		}
	}
}

bool FMultithreadedGenerationContext::ExecuteGameThreadTasks()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMultithreadedGenerationContext::ExecuteGameThreadTasks);

	check(IsInGameThread());

	bool bHadTasks = false;
	MultithreadedGenerationImpl::GameThreadTasks.ConsumeAllLifo(
		[&bHadTasks](TFunction<void()> Function)
		{
			Function();
			bHadTasks = true;
		}
	);

	return bHadTasks;
}

bool& FMultithreadedGenerationContext::GetMultithreadedGenerationRef()
{
	static bool bMultithreadedGeneration = false;
	return bMultithreadedGeneration;
}

void FNativePythonModule::AddType(PyTypeObject* InType)
{
	Py_INCREF(InType);
	PyModule_AddObject(PyModule, InType->tp_name, (PyObject*)InType);
	PyModuleTypes.Add(InType);
}


void FGeneratedWrappedFunction::SetFunction(const UFunction* InFunc, const uint32 InSetFuncFlags)
{
	Func = InFunc;
	InputParams.Reset();
	OutputParams.Reset();
	DeprecationMessage.Reset();

	if (Func && (InSetFuncFlags & SFF_CalculateDeprecationState))
	{
		FString DeprecationMessageStr;
		if (IsDeprecatedFunction(Func, &DeprecationMessageStr))
		{
			DeprecationMessage = MoveTemp(DeprecationMessageStr);
		}
	}

	if (Func && (InSetFuncFlags & SFF_ExtractParameters))
	{
		ExtractFunctionParams(Func, InputParams, OutputParams);
	}
}


void FGeneratedWrappedMethod::ToPython(FPyMethodWithClosureDef& OutPyMethod) const
{
	OutPyMethod.MethodName = MethodName.GetData();
	OutPyMethod.MethodDoc = MethodDoc.GetData();
	OutPyMethod.MethodCallback = MethodCallback;
	OutPyMethod.MethodFlags = MethodFlags;
	OutPyMethod.MethodClosure = (void*)this;
}


void FGeneratedWrappedMethods::Finalize()
{
	check(PyMethods.Num() == 0);

	PyMethods.Reserve(TypeMethods.Num() + 1);
	for (const FGeneratedWrappedMethod& TypeMethod : TypeMethods)
	{
		FPyMethodWithClosureDef& PyMethod = PyMethods.AddZeroed_GetRef();
		TypeMethod.ToPython(PyMethod);
	}
	PyMethods.AddZeroed(); // null terminator
}


void FGeneratedWrappedDynamicMethodWithClosure::Finalize()
{
	ToPython(PyMethod);
}


void FGeneratedWrappedDynamicMethodsMixinBase::AddDynamicMethodImpl(FGeneratedWrappedDynamicMethod&& InDynamicMethod, PyTypeObject* InPyType)
{
	TSharedRef<FGeneratedWrappedDynamicMethodWithClosure> DynamicMethod = DynamicMethods.Add_GetRef(MakeShared<FGeneratedWrappedDynamicMethodWithClosure>());
	static_cast<FGeneratedWrappedDynamicMethod&>(*DynamicMethod) = MoveTemp(InDynamicMethod);
	DynamicMethod->Finalize();
	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		FPyMethodWithClosureDef::AddMethod(&DynamicMethod->PyMethod, InPyType);
	}
}


const FGeneratedWrappedOperatorSignature& FGeneratedWrappedOperatorSignature::OpTypeToSignature(const EGeneratedWrappedOperatorType InOpType)
{
	static const FGeneratedWrappedOperatorSignature OperatorSignatures[(int32)EGeneratedWrappedOperatorType::Num] = {
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Bool,				TEXT("bool"),	TEXT("__bool__"),		EType::Bool,	EType::None),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Equal,			TEXT("=="),		TEXT("__eq__"),			EType::Bool,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::NotEqual,			TEXT("!="),		TEXT("__ne__"),			EType::Bool,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Less,				TEXT("<"),		TEXT("__lt__"),			EType::Bool,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::LessEqual,		TEXT("<="),		TEXT("__le__"),			EType::Bool,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Greater,			TEXT(">"),		TEXT("__gt__"),			EType::Bool,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::GreaterEqual,		TEXT(">="),		TEXT("__ge__"),			EType::Bool,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Add,				TEXT("+"),		TEXT("__add__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineAdd,		TEXT("+="),		TEXT("__iadd__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Subtract,			TEXT("-"),		TEXT("__sub__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineSubtract,	TEXT("-="),		TEXT("__isub__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Multiply,			TEXT("*"),		TEXT("__mul__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineMultiply,	TEXT("*="),		TEXT("__imul__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Divide,			TEXT("/"),		TEXT("__truediv__"),	EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineDivide,		TEXT("/="),		TEXT("__truediv__"),	EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Modulus,			TEXT("%"),		TEXT("__mod__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineModulus,	TEXT("%="),		TEXT("__imod__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::And,				TEXT("&"),		TEXT("__and__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineAnd,		TEXT("&="),		TEXT("__iand__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Or,				TEXT("|"),		TEXT("__or__"),			EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineOr,			TEXT("|="),		TEXT("__ior__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Xor,				TEXT("^"),		TEXT("__xor__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineXor,		TEXT("^="),		TEXT("__ixor__"),		EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::RightShift,		TEXT(">>"),		TEXT("__rshift__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineRightShift,	TEXT(">>="),	TEXT("__irshift__"),	EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::LeftShift,		TEXT("<<"),		TEXT("__lshift__"),		EType::Any,		EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::InlineLeftShift,	TEXT("<<="),	TEXT("__ilshift__"),	EType::Struct,	EType::Any),
		FGeneratedWrappedOperatorSignature(EGeneratedWrappedOperatorType::Negated,			TEXT("neg"),	TEXT("__neg__"),		EType::Struct,	EType::None),
	};

	check(InOpType != EGeneratedWrappedOperatorType::Num);
	return OperatorSignatures[(int32)InOpType];
}

bool FGeneratedWrappedOperatorSignature::StringToSignature(const TCHAR* InStr, FGeneratedWrappedOperatorSignature& OutSignature)
{
	for (int32 OpTypeIndex = 0; OpTypeIndex < (int32)EGeneratedWrappedOperatorType::Num; ++OpTypeIndex)
	{
		const FGeneratedWrappedOperatorSignature& PotentialSignature = OpTypeToSignature((EGeneratedWrappedOperatorType)OpTypeIndex);
		if (FCString::Strcmp(InStr, PotentialSignature.OpTypeStr) == 0)
		{
			check(OpTypeIndex == (int32)PotentialSignature.OpType);
			OutSignature = PotentialSignature;
			return true;
		}
	}

	return false;
}

bool FGeneratedWrappedOperatorSignature::ValidateParam(const FGeneratedWrappedMethodParameter& InParam, const EType InType, const UScriptStruct* InStructType, FString* OutError)
{
	switch (InType)
	{
	case EType::None:
		if (InParam.ParamProp)
		{
			if (OutError)
			{
				*OutError = TEXT("Expected None");
			}
			return false;
		}
		return true;

	case EType::Any:
		if (!InParam.ParamProp)
		{
			if (OutError)
			{
				*OutError = TEXT("Expected Any");
			}
			return false;
		}
		return true;

	case EType::Struct:
		if (!InParam.ParamProp || !InParam.ParamProp->IsA<FStructProperty>() || (InStructType && CastFieldChecked<const FStructProperty>(InParam.ParamProp)->Struct != InStructType))
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("Expected Struct (%s)"), *InStructType->GetName());
			}
			return false;
		}
		return true;

	case EType::Bool:
		if (!InParam.ParamProp || !InParam.ParamProp->IsA<FBoolProperty>())
		{
			if (OutError)
			{
				*OutError = TEXT("Expected Bool");
			}
			return false;
		}
		return true;

	default:
		checkf(false, TEXT("Unexpected parameter type!"));
		break;
	}

	return false;
}

int32 FGeneratedWrappedOperatorSignature::GetInputParamCount() const
{
	return OtherType == EType::None ? 1 : 2;
}

int32 FGeneratedWrappedOperatorSignature::GetOutputParamCount() const
{
	return ReturnType == EType::None ? 0 : 1;
}


bool FGeneratedWrappedOperatorFunction::SetFunction(const UFunction* InFunc, const FGeneratedWrappedOperatorSignature& InSignature, FString* OutError)
{
	FGeneratedWrappedFunction FuncDef;
	FuncDef.SetFunction(InFunc);
	return SetFunction(FuncDef, InSignature, OutError);
}

bool FGeneratedWrappedOperatorFunction::SetFunction(const FGeneratedWrappedFunction& InFuncDef, const FGeneratedWrappedOperatorSignature& InSignature, FString* OutError)
{
	const int32 ExpectedInputParamCount = InSignature.GetInputParamCount();
	const int32 ExpectedOutputParamCount = InSignature.GetOutputParamCount();

	// Count the number of significant (non-defaulted) input parameters
	// We allow additional input parameters as long as they're defaulted and the basic signature requirements are met
	int32 SignificantInputParamCount = 0;
	for (const FGeneratedWrappedMethodParameter& InputParam : InFuncDef.InputParams)
	{
		if (!InputParam.ParamDefaultValue.IsSet())
		{
			++SignificantInputParamCount;
		}
	}

	// In some cases a required input argument may have also been defaulted, so as long as we have enough 
	// input parameters without having too many significant input parameters, still accept this function
	const bool bValidInputParamCount = SignificantInputParamCount <= ExpectedInputParamCount && InFuncDef.InputParams.Num() >= ExpectedInputParamCount;
	const bool bValidOutputParamCount = InFuncDef.OutputParams.Num() == ExpectedOutputParamCount;
	if (!bValidInputParamCount || !bValidOutputParamCount)
	{
		if (OutError)
		{
			*OutError = FString::Printf(TEXT("Incorrect number of arguments; expected %d input and %d output, but got %d input (%d default) and %d output"), ExpectedInputParamCount, ExpectedOutputParamCount, InFuncDef.InputParams.Num(), InFuncDef.InputParams.Num() - SignificantInputParamCount, InFuncDef.OutputParams.Num());
		}
		return false;
	}

	// The 'self' parameter should be the first parameter
	check(InFuncDef.InputParams.IsValidIndex(0)); // always expect a 'self' argument; ExpectedInputParamCount should have verified this
	if (InFuncDef.InputParams[0].ParamProp->IsA<FStructProperty>())
	{
		SelfParam = InFuncDef.InputParams[0];
	}
	else
	{
		if (OutError)
		{
			*OutError = TEXT("A valid struct was not found as the first argument");
		}
		return false;
	}

	// Extract and validate the 'other' parameter
	if (ExpectedInputParamCount > 1 && InFuncDef.InputParams.IsValidIndex(1))
	{
		FString OtherParamError;
		if (FGeneratedWrappedOperatorSignature::ValidateParam(InFuncDef.InputParams[1], InSignature.OtherType, CastFieldChecked<const FStructProperty>(SelfParam.ParamProp)->Struct, &OtherParamError))
		{
			OtherParam = InFuncDef.InputParams[1];
		}
		else
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("Other parameter was invalid (%s)"), *OtherParamError);
			}
			return false;
		}
	}

	// Extract any additional input parameters - these should all be defaulted
	for (int32 AdditionalParamIndex = ExpectedInputParamCount; AdditionalParamIndex < InFuncDef.InputParams.Num(); ++AdditionalParamIndex)
	{
		const FGeneratedWrappedMethodParameter& InputParam = InFuncDef.InputParams[AdditionalParamIndex];
		check(InputParam.ParamDefaultValue.IsSet());
		AdditionalParams.Add(InputParam);
	}

	// Extract and validate the return type
	if (InFuncDef.OutputParams.IsValidIndex(0))
	{
		FString ReturnValueError;
		if (FGeneratedWrappedOperatorSignature::ValidateParam(InFuncDef.OutputParams[0], InSignature.ReturnType, CastFieldChecked<const FStructProperty>(SelfParam.ParamProp)->Struct, &ReturnValueError))
		{
			ReturnParam = InFuncDef.OutputParams[0];
			if (InSignature.ReturnType == FGeneratedWrappedOperatorSignature::EType::Struct)
			{
				SelfReturn = InFuncDef.OutputParams[0];
			}
		}
		else
		{
			if (OutError)
			{
				*OutError = FString::Printf(TEXT("Return value was invalid (%s)"), *ReturnValueError);
			}
			return false;
		}
	}

	Func = InFuncDef.Func;

	return true;
}


void FGeneratedWrappedProperty::SetProperty(const FProperty* InProp, const uint32 InSetPropFlags)
{
	Prop = InProp;
	DeprecationMessage.Reset();

	if (Prop && (InSetPropFlags & SPF_CalculateDeprecationState))
	{
		FString DeprecationMessageStr;
		if (IsDeprecatedProperty(Prop, &DeprecationMessageStr))
		{
			DeprecationMessage = MoveTemp(DeprecationMessageStr);
		}
	}
}


void FGeneratedWrappedGetSet::ToPython(PyGetSetDef& OutPyGetSet) const
{
	OutPyGetSet.name = (char*)GetSetName.GetData();
	OutPyGetSet.doc = (char*)GetSetDoc.GetData();
	OutPyGetSet.get = GetCallback;
	OutPyGetSet.set = SetCallback;
	OutPyGetSet.closure = (void*)this;
}


void FGeneratedWrappedGetSets::Finalize()
{
	check(PyGetSets.Num() == 0);

	PyGetSets.Reserve(TypeGetSets.Num() + 1);
	for (const FGeneratedWrappedGetSet& TypeGetSet : TypeGetSets)
	{
		PyGetSetDef& PyGetSet = PyGetSets.AddZeroed_GetRef();
		TypeGetSet.ToPython(PyGetSet);
	}
	PyGetSets.AddZeroed(); // null terminator
}


void FGeneratedWrappedConstant::ToPython(FPyConstantDef& OutPyConstant) const
{
	auto ConstantGetterImpl = [](PyTypeObject* InType, const void* InClosure) -> PyObject*
	{
		const FGeneratedWrappedConstant* This = (FGeneratedWrappedConstant*)InClosure;
		
		if (ensureAlways(This->ConstantFunc.Func))
		{
			const FString ErrorCtxt = PyUtil::GetErrorContext(InType);

			UClass* Class = This->ConstantFunc.Func->GetOwnerClass();
			UObject* Obj = Class->GetDefaultObject();
	
			// Deprecated functions emit a warning
			if (This->ConstantFunc.DeprecationMessage.IsSet())
			{
				if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, *ErrorCtxt, *FString::Printf(TEXT("Constant '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(This->ConstantName.GetData()), *PyUtil::GetCleanTypename(InType), *This->ConstantFunc.DeprecationMessage.GetValue())) == -1)
				{
					// -1 from SetPythonWarning means the warning should be an exception
					return nullptr;
				}
			}
	
			// Return value requires that we create a params struct to hold the result
			PY_UFUNCTION_STACK(FuncParams, This->ConstantFunc.Func);
			if (!PyUtil::InvokeFunctionCall(Obj, This->ConstantFunc.Func, FuncParams.GetMemory(), *ErrorCtxt))
			{
				return nullptr;
			}
			return PyGenUtil::PackReturnValues(FuncParams.GetMemory(), This->ConstantFunc.OutputParams, *ErrorCtxt, *FString::Printf(TEXT("constant '%s' on '%s'"), UTF8_TO_TCHAR(This->ConstantName.GetData()), *PyUtil::GetCleanTypename(InType)));
		}
	
		Py_RETURN_NONE;
	};

	OutPyConstant.ConstantContext = this;
	OutPyConstant.ConstantGetter = ConstantGetterImpl;
	OutPyConstant.ConstantName = ConstantName.GetData();
	OutPyConstant.ConstantDoc = ConstantDoc.GetData();
}


void FGeneratedWrappedConstants::Finalize()
{
	check(PyConstants.Num() == 0);

	PyConstants.Reserve(TypeConstants.Num() + 1);
	for (const FGeneratedWrappedConstant& TypeConstant : TypeConstants)
	{
		FPyConstantDef& PyConstant = PyConstants.AddZeroed_GetRef();
		TypeConstant.ToPython(PyConstant);
	}
	PyConstants.AddZeroed(); // null terminator
}


void FGeneratedWrappedDynamicConstantWithClosure::Finalize()
{
	ToPython(PyConstant);
}


void FGeneratedWrappedDynamicConstantsMixinBase::AddDynamicConstantImpl(FGeneratedWrappedConstant&& InDynamicConstant, PyTypeObject* InPyType)
{
	TSharedRef<FGeneratedWrappedDynamicConstantWithClosure> DynamicConstant = DynamicConstants.Add_GetRef(MakeShared<FGeneratedWrappedDynamicConstantWithClosure>());
	static_cast<FGeneratedWrappedConstant&>(*DynamicConstant) = MoveTemp(InDynamicConstant);
	DynamicConstant->Finalize();
	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		FPyConstantDef::AddConstantToType(&DynamicConstant->PyConstant, InPyType);
	}
}


FGeneratedWrappedPropertyDoc::FGeneratedWrappedPropertyDoc(const FProperty* InProp)
{
	PythonPropName = GetPropertyPythonName(InProp);

	const FString PropTooltip = GetFieldTooltip(InProp);
	const FParsedTooltip ParsedPropTooltip = ParseTooltip(PropTooltip);
	DocString = PythonizePropertyTooltip(ParsedPropTooltip, InProp, PropertyAccessUtil::RuntimeReadOnlyFlags);
	EditorDocString = PythonizePropertyTooltip(ParsedPropTooltip, InProp, PropertyAccessUtil::EditorReadOnlyFlags);
}

bool FGeneratedWrappedPropertyDoc::SortPredicate(const FGeneratedWrappedPropertyDoc& InOne, const FGeneratedWrappedPropertyDoc& InTwo)
{
	return InOne.PythonPropName < InTwo.PythonPropName;
}

FString FGeneratedWrappedPropertyDoc::BuildDocString(const TArray<FGeneratedWrappedPropertyDoc>& InDocs)
{
	FString Str;
	AppendDocString(InDocs, Str);
	return Str;
}

void FGeneratedWrappedPropertyDoc::AppendDocString(const TArray<FGeneratedWrappedPropertyDoc>& InDocs, FString& OutStr)
{
	if (!InDocs.Num())
	{
		return;
	}

	TStringBuilder<4096> Buffer;
	if (!OutStr.IsEmpty())
	{
		if (OutStr[OutStr.Len() - 1] != TEXT('\n'))
		{
			Buffer += LINE_TERMINATOR;
		}
	}

	Buffer += LINE_TERMINATOR TEXT("**Editor Properties:** (see get_editor_property/set_editor_property)") LINE_TERMINATOR;
	for (const FGeneratedWrappedPropertyDoc& Doc : InDocs)
	{
		Buffer += LINE_TERMINATOR TEXT("- ``");  // add as a list and code style
		Buffer += Doc.PythonPropName;
		Buffer += TEXT("`` ");

		bool bMultipleLines = false;

		const FStringView EditorDocString = Doc.EditorDocString;
		int32 EditorDocStringParseIndex = 0;

		while (EditorDocStringParseIndex < EditorDocString.Len())
		{
			if (bMultipleLines)
			{
				Buffer += LINE_TERMINATOR TEXT("  ");
			}
			bMultipleLines = true;

			const int32 LineStartIndex = EditorDocStringParseIndex;
			while (EditorDocStringParseIndex < EditorDocString.Len() && !FChar::IsLinebreak(EditorDocString[EditorDocStringParseIndex]))
			{
				++EditorDocStringParseIndex;
			}

			Buffer += EditorDocString.Mid(LineStartIndex, EditorDocStringParseIndex - LineStartIndex);

			if (EditorDocStringParseIndex + 1 < EditorDocString.Len() && EditorDocString[EditorDocStringParseIndex] == TEXT('\r') && EditorDocString[EditorDocStringParseIndex + 1] == TEXT('\n'))
			{
				EditorDocStringParseIndex += 2;
			}
			else if (EditorDocStringParseIndex < EditorDocString.Len() && FChar::IsLinebreak(EditorDocString[EditorDocStringParseIndex]))
			{
				++EditorDocStringParseIndex;
			}
		}
	}

	OutStr += Buffer.ToString();
}


void FGeneratedWrappedFieldTracker::RegisterPythonFieldName(const FString& InPythonFieldName, const FFieldVariant& InUnrealField)
{
	FFieldVariant ExistingUnrealField;
	const UField* ExistingUField = nullptr;
	const FField* ExistingFField = nullptr;

	if (InUnrealField.IsUObject())
	{
		ExistingUField = PythonWrappedFieldNameToUnrealField.FindRef(InPythonFieldName).Get();
		if (!ExistingUField)
		{
			PythonWrappedFieldNameToUnrealField.Add(InPythonFieldName, InUnrealField.Get<UField>());
		}
		else
		{
			ExistingUnrealField = ExistingUField;
		}
	}
	else
	{
		ExistingFField = PythonWrappedFieldNameToFField.FindRef(InPythonFieldName).Get();
		if (!ExistingFField)
		{
			PythonWrappedFieldNameToFField.Add(InPythonFieldName, InUnrealField.ToField());
		}
		else
		{
			ExistingUnrealField = ExistingFField;
		}
	}
	if (ExistingUnrealField.IsValid())
	{
		auto GetScopedFieldName = [](const FFieldVariant& InField) -> FString
		{
			// Note: We don't use GetOwnerStruct here, as UFunctions are UStructs so it
			// doesn't work correctly for them as it includes 'this' in the look-up chain
			FFieldVariant OwnerStruct = InField.GetOwnerVariant();
			while (OwnerStruct.IsValid() && !OwnerStruct.IsA<UStruct>())
			{
				OwnerStruct = OwnerStruct.GetOwnerVariant();
			}
			return OwnerStruct.IsValid() ? FString::Printf(TEXT("%s.%s"), *OwnerStruct.GetName(), *InField.GetName()) : InField.GetName();
		};

		REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("'%s' and '%s' have the same name (%s) when exposed to Python. Rename one of them using 'ScriptName' meta-data (or 'ScriptMethod' or 'ScriptConstant' for extension functions)."), *GetScopedFieldName(ExistingUnrealField), *GetScopedFieldName(InUnrealField), *InPythonFieldName);
	}
}


bool FGeneratedWrappedType::Finalize()
{
	Finalize_PreReady();

	bool bSuccess = false;
	// Execute Python code within this block
	if (FinalizedState != EFinalizedState::Finalized)
	{
		FPyScopedGIL GIL;
		if (FinalizedState == EFinalizedState::Initial)
		{
			bSuccess = PyType_Ready(&PyType) == 0;
		}
		else if (FinalizedState == EFinalizedState::Reset)
		{
			PyType_Modified(&PyType);

			// PyType_Modified doesn't update __doc__
			FPyObjectPtr PyDocString = FPyObjectPtr::StealReference(PyUnicode_FromString(PyType.tp_doc ? PyType.tp_doc : ""));
			PyDict_SetItemString(PyType.tp_dict, "__doc__", PyDocString);

			bSuccess = true;
		}
		FinalizedState = EFinalizedState::Finalized;

		// Run these under the same GIL as above since we're going to need it in both functions below
		// and it will avoid 2 more locks in rapid successions.
		if (bSuccess)
		{
			Finalize_PostReady();
			FPyWrapperBaseMetaData::SetMetaData(&PyType, MetaData.Get());
			return true;
		}
	}

	return false;
}

void FGeneratedWrappedType::Reset()
{
	Reset_CleansePyType();
	Reset_CleanseSelf();
}

void FGeneratedWrappedType::Finalize_PreReady()
{
	PyType.tp_name = TypeName.GetData();
	PyType.tp_doc = TypeDoc.GetData();
}

void FGeneratedWrappedType::Finalize_PostReady()
{
}

void FGeneratedWrappedType::Reset_CleansePyType()
{
	PyType.tp_name = nullptr;
	PyType.tp_doc = nullptr;

	FPyWrapperBaseMetaData::SetMetaData(&PyType, nullptr);
}

void FGeneratedWrappedType::Reset_CleanseSelf()
{
	TypeName.Reset();
	TypeDoc.Reset();
	MetaData.Reset();
	FinalizedState = EFinalizedState::Reset;
}


void FGeneratedWrappedStructType::Finalize_PreReady()
{
	FGeneratedWrappedType::Finalize_PreReady();

	GetSets.Finalize();
	PyType.tp_getset = GetSets.PyGetSets.GetData();
}


void FGeneratedWrappedClassType::Finalize_PreReady()
{
	FGeneratedWrappedType::Finalize_PreReady();

	Methods.Finalize();

	GetSets.Finalize();
	PyType.tp_getset = GetSets.PyGetSets.GetData();

	Constants.Finalize();
}

void FGeneratedWrappedClassType::Finalize_PostReady()
{
	FGeneratedWrappedType::Finalize_PostReady();

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		FPyMethodWithClosureDef::AddMethods(Methods.PyMethods.GetData(), &PyType);
		FPyConstantDef::AddConstantsToType(Constants.PyConstants.GetData(), &PyType);
	}
}


void FGeneratedWrappedEnumType::Finalize_PostReady()
{
	FGeneratedWrappedType::Finalize_PostReady();

	check(!MetaData || MetaData->GetTypeId() == FPyWrapperEnumMetaData::StaticTypeId());
	TSharedPtr<FPyWrapperEnumMetaData> EnumMetaData = StaticCastSharedPtr<FPyWrapperEnumMetaData>(MetaData);

	if (EnumMetaData)
	{
		// Execute Python code within this block
		{
			FPyScopedGIL GIL;
			for (const FGeneratedWrappedEnumEntry& EnumEntry : EnumEntries)
			{
				FPyWrapperEnum* PyEnumEntry = FPyWrapperEnum::AddEnumEntry(&PyType, EnumEntry.EntryValue, EnumEntry.EntryName.GetData(), EnumEntry.EntryDoc.GetData());
				if (PyEnumEntry)
				{
					EnumMetaData->EnumEntries.Add(PyEnumEntry);

					for (const FUTF8Buffer& EntryDeprecatedAlias : EnumEntry.EntryDeprecatedAliases)
					{
						FPyWrapperEnum::AddDeprecatedEnumEntry(&PyType, PyEnumEntry, EntryDeprecatedAlias.GetData(), EnumEntry.EntryDoc.GetData());
					}
				}
			}
		}

		EnumMetaData->bFinalized = true;
	}
}

void FGeneratedWrappedEnumType::Reset_CleansePyType()
{
	// Execute Python code within this block
	{
		// Unregister the existing enum entries
		FPyScopedGIL GIL;
		for (const FGeneratedWrappedEnumEntry& EnumEntry : EnumEntries)
		{
			PyDict_DelItemString(PyType.tp_dict, EnumEntry.EntryName.GetData());
		}
	}

	FGeneratedWrappedType::Reset_CleansePyType();
}

void FGeneratedWrappedEnumType::Reset_CleanseSelf()
{
	EnumEntries.Reset();

	FGeneratedWrappedType::Reset_CleanseSelf();
}

void FGeneratedWrappedEnumType::ExtractEnumEntries(const UEnum* InEnum)
{
	for (int32 EnumEntryIndex = 0; EnumEntryIndex < InEnum->NumEnums() - 1; ++EnumEntryIndex)
	{
		if (ShouldExportEnumEntry(InEnum, EnumEntryIndex))
		{
			FGeneratedWrappedEnumEntry& EnumEntry = EnumEntries.AddDefaulted_GetRef();
			EnumEntry.EntryName = TCHARToUTF8Buffer(*GetEnumEntryPythonName(InEnum, EnumEntryIndex));
			EnumEntry.EntryDoc = TCHARToUTF8Buffer(*PythonizeTooltip(ParseTooltip(GetEnumEntryTooltip(InEnum, EnumEntryIndex))));
			EnumEntry.EntryValue = InEnum->GetValueByIndex(EnumEntryIndex);

			const TArray<FString> DeprecatedEnumEntryNames = GetDeprecatedEnumEntryPythonNames(InEnum, EnumEntryIndex);
			EnumEntry.EntryDeprecatedAliases.Reserve(DeprecatedEnumEntryNames.Num());
			for (const FString& DeprecatedEnumEntryName : DeprecatedEnumEntryNames)
			{
				EnumEntry.EntryDeprecatedAliases.Add(TCHARToUTF8Buffer(*DeprecatedEnumEntryName));
			}
		}
	}
}


FPythonizeTooltipContext::FPythonizeTooltipContext(const FProperty* InProp, const uint64 InReadOnlyFlags)
	: Prop(InProp)
	, Func(nullptr)
	, ReadOnlyFlags(InReadOnlyFlags)
{
	if (Prop)
	{
		IsDeprecatedProperty(Prop, &DeprecationMessage);
	}
	}

FPythonizeTooltipContext::FPythonizeTooltipContext(const UFunction* InFunc, const TSet<FName>& InParamsToIgnore)
	: Prop(nullptr)
	, Func(InFunc)
	, ReadOnlyFlags(PropertyAccessUtil::RuntimeReadOnlyFlags)
	, ParamsToIgnore(InParamsToIgnore)
{
	if (Func)
	{
		IsDeprecatedFunction(Func, &DeprecationMessage);
	}
}


FUTF8Buffer TCHARToUTF8Buffer(const TCHAR* InStr)
{
	auto ToUTF8Buffer = [](const char* InUTF8Str)
	{
		int32 UTF8StrLen = 0;
		while (InUTF8Str[UTF8StrLen++] != 0) {} // Count includes the null terminator

		FUTF8Buffer UTF8Buffer;
		UTF8Buffer.Append(InUTF8Str, UTF8StrLen);
		return UTF8Buffer;
	};

	return ToUTF8Buffer(TCHAR_TO_UTF8(InStr));
}

PyObject* GetPostInitFunc(PyTypeObject* InPyType)
{
	FPyObjectPtr PostInitFunc = FPyObjectPtr::StealReference(PyObject_GetAttrString((PyObject*)InPyType, PostInitFuncName));
	if (!PostInitFunc)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InPyType, *FString::Printf(TEXT("Python type has no '%s' function"), UTF8_TO_TCHAR(PostInitFuncName)));
		return nullptr;
	}

	if (!PyCallable_Check(PostInitFunc))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InPyType, *FString::Printf(TEXT("Python type attribute '%s' is not callable"), UTF8_TO_TCHAR(PostInitFuncName)));
		return nullptr;
	}

	// Only test arguments for actual functions and methods (the base type exposed from C will be a 'method_descriptor')
	if (PyFunction_Check(PostInitFunc) || PyMethod_Check(PostInitFunc))
	{
		TArray<FString> FuncArgNames;
		if (!PyUtil::InspectFunctionArgs(PostInitFunc, FuncArgNames))
		{
			PyUtil::SetPythonError(PyExc_Exception, InPyType, *FString::Printf(TEXT("Failed to inspect the arguments for '%s'"), UTF8_TO_TCHAR(PostInitFuncName)));
			return nullptr;
		}
		if (FuncArgNames.Num() != 1)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InPyType, *FString::Printf(TEXT("'%s' must take a single parameter ('self')"), UTF8_TO_TCHAR(PostInitFuncName)));
			return nullptr;
		}
	}

	return PostInitFunc.Release();
}

void AddStructInitParam(const FProperty* InUnrealProp, const TCHAR* InPythonAttrName, TArray<FGeneratedWrappedMethodParameter>& OutInitParams)
{
	FGeneratedWrappedMethodParameter& InitParam = OutInitParams.AddDefaulted_GetRef();
	InitParam.ParamName = TCHARToUTF8Buffer(InPythonAttrName);
	InitParam.ParamProp = InUnrealProp;
	InitParam.ParamDefaultValue = FString();
}

void ExtractFunctionParams(const UFunction* InFunc, TArray<FGeneratedWrappedMethodParameter>& OutInputParams, TArray<FGeneratedWrappedMethodParameter>& OutOutputParams)
{
	auto AddGeneratedWrappedMethodParameter = [InFunc](const FProperty* InParam, TArray<FGeneratedWrappedMethodParameter>& OutParams)
	{
		const FString ParamName = InParam->GetName();
		FString PythonParamName = PythonizePropertyName(ParamName, EPythonizeNameCase::Lower);
		const FName DefaultValueMetaDataKey = *FString::Printf(TEXT("CPP_Default_%s"), *ParamName);

		FGeneratedWrappedMethodParameter& GeneratedWrappedMethodParam = OutParams.AddDefaulted_GetRef();
		GeneratedWrappedMethodParam.ParamName = TCHARToUTF8Buffer(*PythonParamName);
		GeneratedWrappedMethodParam.ParamProp = InParam;
		if (InFunc->HasMetaData(DefaultValueMetaDataKey))
		{
			// This is a default specified in C++ that UHT was able to parse and set as meta-data
			GeneratedWrappedMethodParam.ParamDefaultValue = InFunc->GetMetaData(DefaultValueMetaDataKey);
		}
		else if (InFunc->HasMetaData(InParam->GetFName()))
		{
			// This is either a default specified in Blueprint, or specified in C++ directly as meta-data
			// (ie, because UHT can't parse the C++ default argument, eg, a default TMap argument)
			FString ParamDefaultValue = InFunc->GetMetaData(InParam->GetFName());

			// If the parameter is a class then try and get the full name as the metadata might just be the short name
			if (InParam->IsA<FClassProperty>() && !FPackageName::IsValidObjectPath(ParamDefaultValue))
			{
				if (const UClass* DefaultClass = UClass::TryFindTypeSlow<UClass>(ParamDefaultValue, EFindFirstObjectOptions::ExactClass))
				{
					ParamDefaultValue = DefaultClass->GetPathName();
				}
			}

			// Validate that the default is actually valid (ie, can be imported), as we may hit 
			// false positives for this meta-data (eg, from LatentInfo, WorldContext, etc)
			void* TempParam = FMemory_Alloca_Aligned(InParam->GetSize(), InParam->GetMinAlignment());
			InParam->InitializeValue(TempParam);
			{
				FOutputDeviceNull NullOutputDevice;
				if (PropertyAccessUtil::ImportDefaultPropertyValue(InParam, TempParam, *ParamDefaultValue, &NullOutputDevice))
				{
					GeneratedWrappedMethodParam.ParamDefaultValue = MoveTemp(ParamDefaultValue);
				}
			}
			InParam->DestroyValue(TempParam);
		}
	};

	if (const FProperty* ReturnProp = InFunc->GetReturnProperty())
	{
		AddGeneratedWrappedMethodParameter(ReturnProp, OutOutputParams);
	}

	for (TFieldIterator<const FProperty> ParamIt(InFunc); ParamIt; ++ParamIt)
	{
		const FProperty* Param = *ParamIt;

		if (PyUtil::IsInputParameter(Param))
		{
			AddGeneratedWrappedMethodParameter(Param, OutInputParams);
		}

		if (PyUtil::IsOutputParameter(Param))
		{
			AddGeneratedWrappedMethodParameter(Param, OutOutputParams);
		}
	}
}

void ApplyParamDefaults(void* InBaseParamsAddr, const TArray<FGeneratedWrappedMethodParameter>& InParamDef)
{
	for (const FGeneratedWrappedMethodParameter& ParamDef : InParamDef)
	{
		if (ParamDef.ParamDefaultValue.IsSet())
		{
			PyUtil::ImportDefaultValue(ParamDef.ParamProp, ParamDef.ParamProp->ContainerPtrToValuePtr<void>(InBaseParamsAddr), ParamDef.ParamDefaultValue.GetValue());
		}
	}
}

bool ParseMethodParameters(PyObject* InArgs, PyObject* InKwds, const TArray<FGeneratedWrappedMethodParameter>& InParamDef, const char* InPyMethodName, TArray<PyObject*>& OutPyParams)
{
	if (!InArgs || !PyTuple_Check(InArgs) || (InKwds && !PyDict_Check(InKwds)) || !InPyMethodName)
	{
		PyErr_BadInternalCall();
		return false;
	}

	const Py_ssize_t NumArgs = PyTuple_GET_SIZE(InArgs);
	const Py_ssize_t NumKeywords = InKwds ? PyDict_Size(InKwds) : 0;
	if (NumArgs + NumKeywords > InParamDef.Num())
	{
		PyErr_Format(PyExc_TypeError, "%s() takes at most %d argument%s (%d given)", InPyMethodName, InParamDef.Num(), (InParamDef.Num() == 1 ? "" : "s"), (int32)(NumArgs + NumKeywords));
		return false;
	}

	// Parse both keyword and index args in the same loop (favor keywords, fallback to index)
	Py_ssize_t RemainingKeywords = NumKeywords;
	for (int32 Index = 0; Index < InParamDef.Num(); ++Index)
	{
		const FGeneratedWrappedMethodParameter& ParamDef = InParamDef[Index];

		PyObject* ParsedArg = nullptr;
		if (RemainingKeywords > 0)
		{
			ParsedArg = PyDict_GetItemString(InKwds, ParamDef.ParamName.GetData());
		}

		if (ParsedArg)
		{
			--RemainingKeywords;
			if (Index < NumArgs)
			{
				PyErr_Format(PyExc_TypeError, "%s() argument given by name ('%s') and position (%d)", InPyMethodName, ParamDef.ParamName.GetData(), Index + 1);
				return false;
			}
		}
		else if (RemainingKeywords > 0 && PyErr_Occurred())
		{
			return false;
		}
		else if (Index < NumArgs)
		{
			ParsedArg = PyTuple_GET_ITEM(InArgs, Index);
		}

		if (ParsedArg || ParamDef.ParamDefaultValue.IsSet())
		{
			OutPyParams.Add(ParsedArg);
			continue;
		}

		PyErr_Format(PyExc_TypeError, "%s() required argument '%s' (pos %d) not found", InPyMethodName, ParamDef.ParamName.GetData(), Index + 1);
		return false;
	}

	// Report any extra keyword args
	if (RemainingKeywords > 0)
	{
		PyObject* Key = nullptr;
		PyObject* Value = nullptr;
		Py_ssize_t Index = 0;
		while (PyDict_Next(InKwds, &Index, &Key, &Value))
		{
			const FUTF8Buffer Keyword = TCHARToUTF8Buffer(*PyUtil::PyObjectToUEString(Key));
			const bool bIsExpected = InParamDef.ContainsByPredicate([&Keyword](const FGeneratedWrappedMethodParameter& ParamDef)
			{
				return FCStringAnsi::Strcmp(Keyword.GetData(), ParamDef.ParamName.GetData()) == 0;
			});

			if (!bIsExpected)
			{
				PyErr_Format(PyExc_TypeError, "%s() '%s' is an invalid keyword argument for this function", InPyMethodName, Keyword.GetData());
				return false;
			}
		}
	}

	return true;
}

PyObject* PackReturnValues(const void* InBaseParamsAddr, const TArray<FGeneratedWrappedMethodParameter>& InOutputParams, const TCHAR* InErrorCtxt, const TCHAR* InCallingCtxt)
{
	if (!InOutputParams.Num())
	{
		Py_RETURN_NONE;
	}

	int32 ReturnPropIndex = 0;

	// If we have multiple return values and the main return value is a bool, we return None (for false) or the (potentially packed) return value without the bool (for true)
	if (InOutputParams.Num() > 1 && InOutputParams[0].ParamProp->HasAnyPropertyFlags(CPF_ReturnParm) && InOutputParams[0].ParamProp->IsA<FBoolProperty>())
	{
		const FBoolProperty* BoolReturn = CastFieldChecked<const FBoolProperty>(InOutputParams[0].ParamProp);
		const bool bReturnValue = BoolReturn->GetPropertyValue(BoolReturn->ContainerPtrToValuePtr<void>(InBaseParamsAddr));
		if (!bReturnValue)
		{
			Py_RETURN_NONE;
		}

		ReturnPropIndex = 1; // Start packing at the 1st out value
	}

	// Do we need to return a packed tuple, or just a single value?
	const int32 NumPropertiesToPack = InOutputParams.Num() - ReturnPropIndex;
	if (NumPropertiesToPack == 1)
	{
		PyObject* OutParamPyObj = nullptr;
		if (!PyConversion::PythonizeProperty_InContainer(InOutputParams[ReturnPropIndex].ParamProp, InBaseParamsAddr, 0, OutParamPyObj, EPyConversionMethod::Steal))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert return property '%s' (%s) when calling %s"), *InOutputParams[ReturnPropIndex].ParamProp->GetName(), *InOutputParams[ReturnPropIndex].ParamProp->GetClass()->GetName(), InCallingCtxt));
			return nullptr;
		}
		return OutParamPyObj;
	}
	else
	{
		int32 OutParamTupleIndex = 0;
		FPyObjectPtr OutParamTuple = FPyObjectPtr::StealReference(PyTuple_New(NumPropertiesToPack));
		for (; ReturnPropIndex < InOutputParams.Num(); ++ReturnPropIndex)
		{
			PyObject* OutParamPyObj = nullptr;
			if (!PyConversion::PythonizeProperty_InContainer(InOutputParams[ReturnPropIndex].ParamProp, InBaseParamsAddr, 0, OutParamPyObj, EPyConversionMethod::Steal))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert return property '%s' (%s) when calling function %s"), *InOutputParams[ReturnPropIndex].ParamProp->GetName(), *InOutputParams[ReturnPropIndex].ParamProp->GetClass()->GetName(), InCallingCtxt));
				return nullptr;
			}
			PyTuple_SetItem(OutParamTuple, OutParamTupleIndex++, OutParamPyObj); // SetItem steals the reference
		}
		return OutParamTuple.Release();
	}
}

bool UnpackReturnValues(PyObject* InRetVals, const FOutParmRec* InOutputParms, const TCHAR* InErrorCtxt, const TCHAR* InCallingCtxt)
{
	if (!InOutputParms)
	{
		return true;
	}

	const FOutParmRec* OutParamRec = InOutputParms;

	// If we have multiple return values and the main return value is a bool, we expect None (for false) or the (potentially packed) return value without the bool (for true)
	if (OutParamRec->NextOutParm && OutParamRec->Property->HasAnyPropertyFlags(CPF_ReturnParm) && OutParamRec->Property->IsA<FBoolProperty>())
	{
		const FBoolProperty* BoolReturn = CastFieldChecked<const FBoolProperty>(OutParamRec->Property);
		const bool bReturnValue = InRetVals != Py_None;
		BoolReturn->SetPropertyValue(OutParamRec->PropAddr, bReturnValue);

		if (!bReturnValue)
		{
			// None was returned, so there's nothing to unpack into any additional output arguments
			return true;
		}

		OutParamRec = OutParamRec->NextOutParm; // Start unpacking at the 1st out value
		check(OutParamRec);
	}

	// Do we need to expect a packed tuple, or just a single value?
	if (OutParamRec->NextOutParm == nullptr)
	{
		if (!PyConversion::NativizeProperty_Direct(InRetVals, OutParamRec->Property, OutParamRec->PropAddr))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert return property '%s' (%s) when calling %s"), *OutParamRec->Property->GetName(), *OutParamRec->Property->GetClass()->GetName(), InCallingCtxt));
			return false;
		}
	}
	else
	{
		if (!PyTuple_Check(InRetVals))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Expected a 'tuple' return type, but got '%s' when calling %s"), *PyUtil::GetFriendlyTypename(InRetVals), InCallingCtxt));
			return false;
		}

		check(OutParamRec->NextOutParm);
		int32 NumPropertiesToUnpack = 2; // We can start from 2 since we know we already have at least 2 output parameters to get to this code
		for (const FOutParmRec* Tmp = OutParamRec->NextOutParm->NextOutParm; Tmp; Tmp = Tmp->NextOutParm)
		{
			++NumPropertiesToUnpack;
		}

		const int32 RetTupleSize = PyTuple_Size(InRetVals);
		if (RetTupleSize != NumPropertiesToUnpack)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Expected a 'tuple' return type containing '%d' items but got one containing '%d' items when calling %s"), NumPropertiesToUnpack, RetTupleSize, InCallingCtxt));
			return false;
		}

		int32 RetTupleIndex = 0;
		do
		{
			PyObject* RetVal = PyTuple_GetItem(InRetVals, RetTupleIndex++);
			if (!PyConversion::NativizeProperty_Direct(RetVal, OutParamRec->Property, OutParamRec->PropAddr))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert return property '%s' (%s) when calling %s"), *OutParamRec->Property->GetName(), *OutParamRec->Property->GetClass()->GetName(), InCallingCtxt));
				return false;
			}
			OutParamRec = OutParamRec->NextOutParm;
		}
		while (OutParamRec);
	}

	return true;
}

bool InvokePythonCallableFromUnrealFunctionThunk(FPyObjectPtr InSelf, PyObject* InCallable, const UFunction* InFunc, UObject* Context, FFrame& Stack, RESULT_DECL)
{
	// Allocate memory to store our local argument data
	void* LocalStruct = FMemory_Alloca(FMath::Max<int32>(1, InFunc->GetStructureSize()));
	InFunc->InitializeStruct(LocalStruct);
	ON_SCOPE_EXIT
	{
		InFunc->DestroyStruct(LocalStruct);
	};

	// Stores information about inputs and outputs
	FOutParmRec* OutParms = nullptr;
	TArray<FPyObjectPtr, TInlineAllocator<4>> PyParams;

	// Add any return property to the output params chain
	if (FProperty* ReturnProp = InFunc->GetReturnProperty())
	{
		FOutParmRec* Out = (FOutParmRec*)FMemory_Alloca(sizeof(FOutParmRec));
		Out->Property = ReturnProp;
		Out->PropAddr = (uint8*)RESULT_PARAM;

		// Link it to the head of the list, as UnpackReturnValues expects the return value to be first in the list
		Out->NextOutParm = OutParms;
		OutParms = Out;
	}

	// Get the value of the input params for the Python args, and cache the addresses that return and output data should be unpacked to
	bool bProcessedInputs = true;
	{
		int32 ArgIndex = 0;
		FOutParmRec** LastOut = &OutParms;

		// We iterate the fields directly here as we need to process input and output properties in the 
		// correct stack order, as we're potentially popping data off the bytecode stack
		for (TFieldIterator<FProperty> ParamIt(InFunc); ParamIt; ++ParamIt)
		{
			FProperty* Param = *ParamIt;

			// Skip the return value; it never has data on the bytecode stack and was added to the output params chain before this loop
			if (Param->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				continue;
			}

			// Step the property data to populate the local value
			Stack.MostRecentPropertyAddress = nullptr;
			Stack.MostRecentPropertyContainer = nullptr;
			void* LocalValue = Param->ContainerPtrToValuePtr<void>(LocalStruct);
			Stack.StepCompiledIn(LocalValue, Param->GetClass());

			// Output parameters (even const ones) need to read their data from the property address (if available) rather than the local struct
			void* ValueAddress = LocalValue;
			if (Param->HasAnyPropertyFlags(CPF_OutParm) && Stack.MostRecentPropertyAddress)
			{
				ValueAddress = Stack.MostRecentPropertyAddress;
			}

			// Add any output parameters to the output params chain
			if (PyUtil::IsOutputParameter(Param))
			{
				CA_SUPPRESS(6263) // using _alloca in a loop
				FOutParmRec* Out = (FOutParmRec*)FMemory_Alloca(sizeof(FOutParmRec));
				Out->Property = Param;
				Out->PropAddr = (uint8*)ValueAddress;
				Out->NextOutParm = nullptr;

				// Link it to the end of the list
				if (*LastOut)
				{
					(*LastOut)->NextOutParm = Out;
					LastOut = &(*LastOut)->NextOutParm;
				}
				else
				{
					*LastOut = Out;
				}
			}

			// Convert any input parameters for use with Python
			if (PyUtil::IsInputParameter(Param))
			{
				FPyObjectPtr& PyParam = PyParams.AddDefaulted_GetRef();
				if (!PyConversion::PythonizeProperty_Direct(Param, ValueAddress, PyParam.Get()))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InCallable ? *PyUtil::GetErrorContext(InCallable) : TEXT("<null>"), *FString::Printf(TEXT("Failed to convert argument at pos '%d' when calling function '%s' on '%s'"), ArgIndex + 1, *InFunc->GetName(), *P_THIS_OBJECT->GetName()));
					bProcessedInputs = false;
				}
				++ArgIndex;
			}
		}
	}

	// Validate we reached the end of the parameters when stepping the bytecode stack
	if (Stack.Code)
	{
		checkSlow(*Stack.Code == EX_EndFunctionParms);
		++Stack.Code;
	}

	// If any errors happened during parameter processing then we can bail now that we've finished stepping the bytecode stack
	// We can also bail if we have no Python callable to invoke
	if (!bProcessedInputs || !InCallable)
	{
		return false;
	}

	TGuardValue<bool> IsRunningUserScriptGuard(UE::Python::Private::GIsRunningUserScript, true);

	// Prepare the arguments tuple for the Python callable
	FPyObjectPtr PyArgs;
	if (InSelf || PyParams.Num() > 0)
	{
		const int32 PyParamOffset = (InSelf ? 1 : 0);
		PyArgs = FPyObjectPtr::StealReference(PyTuple_New(PyParams.Num() + PyParamOffset));
		if (InSelf)
		{
			PyTuple_SetItem(PyArgs, 0, InSelf.Release()); // SetItem steals the reference
		}
		for (int32 PyParamIndex = 0; PyParamIndex < PyParams.Num(); ++PyParamIndex)
		{
			PyTuple_SetItem(PyArgs, PyParamIndex + PyParamOffset, PyParams[PyParamIndex].Release()); // SetItem steals the reference
		}
	}

	// Invoke the Python callable
	FPyObjectPtr RetVals = FPyObjectPtr::StealReference(PyObject_CallObject(InCallable, PyArgs));
	if (!RetVals)
	{
		return false;
	}

	// Unpack any output values
	if (!UnpackReturnValues(RetVals, OutParms, *PyUtil::GetErrorContext(InCallable), *FString::Printf(TEXT("function '%s' on '%s'"), *InFunc->GetName(), *P_THIS_OBJECT->GetName())))
	{
		return false;
	}

	return true;
}

PyObject* GetPropertyValue(const UStruct* InStruct, const void* InStructData, const FGeneratedWrappedProperty& InPropDef, const char *InAttributeName, PyObject* InOwnerPyObject, const TCHAR* InErrorCtxt)
{
	// Has this property been deprecated?
	if (InStruct && InPropDef.Prop && InPropDef.DeprecationMessage.IsSet())
	{
		// If the property is fully deprecated (rather than just renamed) it can no longer be accessed and cause an error rather than a warning
		const FString FormattedDeprecationMessage = FString::Printf(TEXT("Property '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName(), *InPropDef.DeprecationMessage.GetValue());
		if (InPropDef.Prop->HasAnyPropertyFlags(CPF_Deprecated))
		{
			PyUtil::SetPythonError(PyExc_DeprecationWarning, InErrorCtxt, *FormattedDeprecationMessage);
			return nullptr;
		}
		else
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FormattedDeprecationMessage) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return nullptr;
			}
		}
	}

	return PyUtil::GetPropertyValue(InStruct, InStructData, InPropDef.Prop, InAttributeName, InOwnerPyObject, InErrorCtxt);
}

int SetPropertyValue(const UStruct* InStruct, void* InStructData, PyObject* InValue, const FGeneratedWrappedProperty& InPropDef, const char *InAttributeName, const FPropertyAccessChangeNotify* InChangeNotify, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const TCHAR* InErrorCtxt, const TConstArrayView<void*>& InArchetypeInstContainers)
{
	// Has this property been deprecated?
	if (InStruct && InPropDef.Prop && InPropDef.DeprecationMessage.IsSet())
	{
		// If the property is fully deprecated (rather than just renamed) it can no longer be accessed and cause an error rather than a warning
		const FString FormattedDeprecationMessage = FString::Printf(TEXT("Property '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName(), *InPropDef.DeprecationMessage.GetValue());
		if (InPropDef.Prop->HasAnyPropertyFlags(CPF_Deprecated))
		{
			PyUtil::SetPythonError(PyExc_DeprecationWarning, InErrorCtxt, *FormattedDeprecationMessage);
			return -1;
		}
		else
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FormattedDeprecationMessage) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return -1;
			}
		}
	}

	return PyUtil::SetPropertyValue(InStruct, InStructData, InValue, InPropDef.Prop, InAttributeName, InChangeNotify, InReadOnlyFlags, InOwnerIsTemplate, InErrorCtxt, InArchetypeInstContainers);
}

FString BuildFunctionDocString(const UFunction* InFunc, const FString& InFuncPythonName, const TArray<FGeneratedWrappedMethodParameter>& InInputParams, const TArray<FGeneratedWrappedMethodParameter>& InOutputParams, const bool* InStaticOverride)
{
	const bool bIsStatic = (InStaticOverride) ? *InStaticOverride : InFunc->HasAnyFunctionFlags(FUNC_Static);

	FString FunctionDeclDocString = FString::Printf(TEXT("%s.%s("), (bIsStatic ? TEXT("X") : TEXT("x")), *InFuncPythonName);
	for (const FGeneratedWrappedMethodParameter& InputParam : InInputParams)
	{
		if (FunctionDeclDocString[FunctionDeclDocString.Len() - 1] != TEXT('('))
		{
			FunctionDeclDocString += TEXT(", ");
		}
		FunctionDeclDocString += UTF8_TO_TCHAR(InputParam.ParamName.GetData());
		if (InputParam.ParamDefaultValue.IsSet())
		{
			FunctionDeclDocString += TEXT('=');
			FunctionDeclDocString += PythonizeDefaultValue(InputParam.ParamProp, InputParam.ParamDefaultValue.GetValue());
		}
	}
	FunctionDeclDocString += TEXT(") -> ");

	if (InOutputParams.Num() > 0)
	{
		// If we have multiple return values and the main return value is a bool, we return None (for false) or the (potentially packed) return value without the bool (for true)
		int32 IndexOffset = 0;
		if (InOutputParams.Num() > 1)
		{
			if (InOutputParams[0].ParamProp->HasAnyPropertyFlags(CPF_ReturnParm) && InOutputParams[0].ParamProp->IsA<FBoolProperty>())
			{
				++IndexOffset;
			}
		}

		if (InOutputParams.Num() - IndexOffset == 1)
		{
			FunctionDeclDocString += GetPropertyTypePythonName(InOutputParams[IndexOffset].ParamProp);
		}
		else
		{
			const bool bHasReturnValue = InOutputParams[0].ParamProp->HasAnyPropertyFlags(CPF_ReturnParm);
			FunctionDeclDocString += TEXT('(');
			for (int32 OutParamIndex = IndexOffset; OutParamIndex < InOutputParams.Num(); ++OutParamIndex)
			{
				if (OutParamIndex > IndexOffset)
				{
					FunctionDeclDocString += TEXT(", ");
				}
				if (OutParamIndex > 0 || !bHasReturnValue)
				{
					FunctionDeclDocString += UTF8_TO_TCHAR(InOutputParams[OutParamIndex].ParamName.GetData());
					FunctionDeclDocString += TEXT('=');
				}
				FunctionDeclDocString += GetPropertyTypePythonName(InOutputParams[OutParamIndex].ParamProp);
			}
			FunctionDeclDocString += TEXT(')');
		}

		if (IndexOffset > 0)
		{
			FunctionDeclDocString += TEXT(" or None");
		}
	}
	else
	{
		FunctionDeclDocString += TEXT("None");
	}

	return FunctionDeclDocString;
}

bool IsScriptExposedClass(const UClass* InClass)
{
	for (const UClass* ParentClass = InClass; ParentClass; ParentClass = ParentClass->GetSuperClass())
	{
		if (IsBlueprintGeneratedClass(ParentClass))
		{
			return true;
		}

		if (ParentClass->GetBoolMetaData(BlueprintTypeMetaDataKey) || ParentClass->HasMetaData(BlueprintSpawnableComponentMetaDataKey))
		{
			return true;
		}

		if (ParentClass->GetBoolMetaData(NotBlueprintTypeMetaDataKey))
		{
			return false;
		}
	}

	return false;
}

bool IsScriptExposedStruct(const UScriptStruct* InStruct)
{
	for (const UScriptStruct* ParentStruct = InStruct; ParentStruct; ParentStruct = Cast<UScriptStruct>(ParentStruct->GetSuperStruct()))
	{
		if (IsBlueprintGeneratedStruct(ParentStruct))
		{
			return true;
		}

		if (ParentStruct->GetBoolMetaData(BlueprintTypeMetaDataKey))
		{
			return true;
		}

		if (ParentStruct->GetBoolMetaData(NotBlueprintTypeMetaDataKey))
		{
			return false;
		}
	}

	return false;
}

bool IsScriptExposedEnum(const UEnum* InEnum)
{
	if (IsBlueprintGeneratedEnum(InEnum))
	{
		return true;
	}

	if (InEnum->GetBoolMetaData(BlueprintTypeMetaDataKey))
	{
		return true;
	}

	if (InEnum->GetBoolMetaData(NotBlueprintTypeMetaDataKey))
	{
		return false;
	}

	return false;
}

bool IsScriptExposedEnumEntry(const UEnum* InEnum, int32 InEnumEntryIndex)
{
	return InEnumEntryIndex != INDEX_NONE
		&& !InEnum->HasMetaData(HiddenMetaDataKey, InEnumEntryIndex);
}

bool IsScriptExposedProperty(const FProperty* InProp)
{
	return !InProp->HasMetaData(ScriptNoExportMetaDataKey) 
		&& InProp->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintAssignable);
}

bool IsScriptExposedFunction(const UFunction* InFunc)
{
	return !InFunc->HasMetaData(ScriptNoExportMetaDataKey)
		&& InFunc->HasAnyFunctionFlags(FUNC_BlueprintCallable | FUNC_BlueprintEvent)
		&& !InFunc->HasMetaData(BlueprintGetterMetaDataKey)
		&& !InFunc->HasMetaData(BlueprintSetterMetaDataKey)
		&& !InFunc->HasMetaData(BlueprintInternalUseOnlyMetaDataKey)
		&& !InFunc->HasMetaData(CustomThunkMetaDataKey)
		&& !InFunc->HasMetaData(NativeBreakFuncMetaDataKey)
		&& !InFunc->HasMetaData(NativeMakeFuncMetaDataKey);
}

bool HasScriptExposedFields(const UStruct* InStruct)
{
	for (TFieldIterator<const UFunction> FieldIt(InStruct); FieldIt; ++FieldIt)
	{
		const UFunction* Func = *FieldIt;
		if (IsScriptExposedFunction(Func))
		{
			return true;
		}
	}

	for (TFieldIterator<const FProperty> FieldIt(InStruct); FieldIt; ++FieldIt)
	{
		const FProperty* Prop = *FieldIt;
		if (IsScriptExposedProperty(Prop))
		{
			return true;
		}
	}

	return false;
}

bool IsBlueprintGeneratedClass(const UClass* InClass)
{
	// Need to use IsA rather than IsChildOf since we want to test the type of InClass itself *NOT* the class instance represented by InClass
	const UObject* ClassObject = InClass;
	return ClassObject->IsA<UBlueprintGeneratedClass>();
}

bool IsBlueprintGeneratedStruct(const UScriptStruct* InStruct)
{
	return InStruct->IsA<UUserDefinedStruct>();
}

bool IsBlueprintGeneratedEnum(const UEnum* InEnum)
{
	return InEnum->IsA<UUserDefinedEnum>();
}

bool IsDeprecatedClass(const UClass* InClass, FString* OutDeprecationMessage)
{
	if (InClass->HasAnyClassFlags(CLASS_Deprecated))
	{
		if (OutDeprecationMessage)
		{
			*OutDeprecationMessage = InClass->GetMetaData(DeprecationMessageMetaDataKey);
			if (OutDeprecationMessage->IsEmpty())
			{
				*OutDeprecationMessage = FString::Printf(TEXT("Class '%s' is deprecated."), *InClass->GetName());
			}
		}

		return true;
	}

	return false;
}

bool IsDeprecatedProperty(const FProperty* InProp, FString* OutDeprecationMessage)
{
	if (InProp->HasMetaData(DeprecatedPropertyMetaDataKey))
	{
		if (OutDeprecationMessage)
		{
			*OutDeprecationMessage = InProp->GetMetaData(DeprecationMessageMetaDataKey);
			if (OutDeprecationMessage->IsEmpty())
			{
				*OutDeprecationMessage = FString::Printf(TEXT("Property '%s' is deprecated."), *InProp->GetName());
			}
		}

		return true;
	}

	return false;
}

bool IsDeprecatedFunction(const UFunction* InFunc, FString* OutDeprecationMessage)
{
	if (InFunc->HasMetaData(DeprecatedFunctionMetaDataKey))
	{
		if (OutDeprecationMessage)
		{
			*OutDeprecationMessage = InFunc->GetMetaData(DeprecationMessageMetaDataKey);
			if (OutDeprecationMessage->IsEmpty())
			{
				*OutDeprecationMessage = FString::Printf(TEXT("Function '%s' is deprecated."), *InFunc->GetName());
			}
		}

		return true;
	}

	return false;
}

void GetExportedInterfacesForClass_Recursive(const UClass* InInterface, TSet<const UClass*>& InOutProcessedInterfaces, TArray<const UClass*>& InOutExportedInterfaces)
{
	check(InInterface->HasAnyClassFlags(CLASS_Interface));

	{
		bool bAlreadyProcessed = false;
		InOutProcessedInterfaces.Add(InInterface, &bAlreadyProcessed);
		if (bAlreadyProcessed)
		{
			return;
		}
	}

	if (!ShouldExportClass(InInterface))
	{
		return;
	}

	InOutExportedInterfaces.Add(InInterface);

	for (UClass* SuperClass = InInterface->GetSuperClass(); 
		SuperClass && SuperClass->HasAnyClassFlags(CLASS_Interface); 
		SuperClass = SuperClass->GetSuperClass()
		)
	{
		GetExportedInterfacesForClass_Recursive(SuperClass, InOutProcessedInterfaces, InOutExportedInterfaces);
	}

	for (const FImplementedInterface& Interface : InInterface->Interfaces)
	{
		GetExportedInterfacesForClass_Recursive(Interface.Class, InOutProcessedInterfaces, InOutExportedInterfaces);
	}
}

TArray<const UClass*> GetExportedInterfacesForClass(const UClass* InClass)
{
	TSet<const UClass*> ProcessedInterfaces;
	TArray<const UClass*> ExportedInterfaces;
	for (const FImplementedInterface& Interface : InClass->Interfaces)
	{
		GetExportedInterfacesForClass_Recursive(Interface.Class, ProcessedInterfaces, ExportedInterfaces);
	}
	return ExportedInterfaces;
}

bool ShouldExportClass(const UClass* InClass)
{
	return InClass->GetOuter()->IsA<UPackage>() && (IsScriptExposedClass(InClass) || HasScriptExposedFields(InClass));
}

bool ShouldExportStruct(const UScriptStruct* InStruct)
{
	// Disabling BlueprintInternalUseOnly/Hierarchical for structs for now, as some existing scripts depend on these internal types
	/*
	if (InStruct->HasMetaData(BlueprintInternalUseOnlyMetaDataKey))
	{
		return false;
	}

	for (const UScriptStruct* ParentStruct = InStruct; ParentStruct; ParentStruct = Cast<UScriptStruct>(ParentStruct->GetSuperStruct()))
	{
		if (ParentStruct->HasMetaData(BlueprintInternalUseOnlyHierarchicalMetaDataKey))
		{
			return false;
		}
	}
	*/

	return InStruct->GetOuter()->IsA<UPackage>() && (IsScriptExposedStruct(InStruct) || HasScriptExposedFields(InStruct));
}

bool ShouldExportEnum(const UEnum* InEnum)
{
	return InEnum->GetOuter()->IsA<UPackage>() && IsScriptExposedEnum(InEnum);
}

bool ShouldExportEnumEntry(const UEnum* InEnum, int32 InEnumEntryIndex)
{
	return IsScriptExposedEnumEntry(InEnum, InEnumEntryIndex);
}

bool ShouldExportProperty(const FProperty* InProp)
{
	const bool bCanScriptExport = !InProp->HasMetaData(ScriptNoExportMetaDataKey); // Need to test this again here as IsScriptExposedProperty checks it internally, but IsDeprecatedProperty doesn't
	return bCanScriptExport && (IsScriptExposedProperty(InProp) || IsDeprecatedProperty(InProp));
}

bool ShouldExportEditorOnlyProperty(const FProperty* InProp)
{
	const bool bCanScriptExport = !InProp->HasMetaData(ScriptNoExportMetaDataKey);
	return bCanScriptExport && GIsEditor && (InProp->HasAnyPropertyFlags(CPF_Edit) || IsDeprecatedProperty(InProp));
}

bool ShouldExportFunction(const UFunction* InFunc)
{
	return IsScriptExposedFunction(InFunc);
}

bool IsValidName(FStringView InName, FText* OutError)
{
	if (InName.Len() == 0)
	{
		if (OutError)
		{
			*OutError = NSLOCTEXT("PyGenUtil", "InvalidName_EmptyName", "Name is empty");
		}
		return false;
	}

	// Names must be a valid symbol (alnum or _ and doesn't start with a digit)

	if (FChar::IsDigit(InName[0]))
	{
		if (OutError)
		{
			*OutError = NSLOCTEXT("PyGenUtil", "InvalidName_LeadingDigit", "Name starts with a digit which is invalid for Python");
		}
		return false;
	}

	for (const TCHAR Char : InName)
	{
		if (!FChar::IsAlnum(Char) && Char != TEXT('_'))
		{
			if (OutError)
			{
				*OutError = FText::Format(NSLOCTEXT("PyGenUtil", "InvalidName_RestrictedCharacter", "Name contains '{0}' which is invalid for Python"), FText::AsCultureInvariant(FString::ConstructFromPtrSize(&Char, 1)));
			}
			return false;
		}
	}

	return true;
}

FString PythonizeName(FStringView InName, const EPythonizeNameCase InNameCase)
{
	static const TSet<FString, FCaseSensitiveStringSetFuncs> ReservedKeywords = {
		TEXT("and"),
		TEXT("as"),
		TEXT("assert"),
		TEXT("async"),
		TEXT("break"),
		TEXT("class"),
		TEXT("continue"),
		TEXT("def"),
		TEXT("del"),
		TEXT("elif"),
		TEXT("else"),
		TEXT("except"),
		TEXT("finally"),
		TEXT("for"),
		TEXT("from"),
		TEXT("global"),
		TEXT("if"),
		TEXT("import"),
		TEXT("in"),
		TEXT("is"),
		TEXT("lambda"),
		TEXT("nonlocal"),
		TEXT("not"),
		TEXT("or"),
		TEXT("pass"),
		TEXT("raise"),
		TEXT("return"),
		TEXT("try"),
		TEXT("while"),
		TEXT("with"),
		TEXT("yield"),
		TEXT("property"),
	};

	FString PythonizedName;
	PythonizedName.Reserve(InName.Len() + 10);

	if (!NameBreakIterator.IsValid())
	{
		NameBreakIterator = FBreakIterator::CreateCamelCaseBreakIterator();
	}

	NameBreakIterator->SetStringRef(InName);
	for (int32 PrevBreak = 0, NameBreak = NameBreakIterator->MoveToNext(); NameBreak != INDEX_NONE; NameBreak = NameBreakIterator->MoveToNext())
	{
		const int32 OrigPythonizedNameLen = PythonizedName.Len();

		// Append an underscore if this was a break between two parts of the identifier, *and* the previous character isn't already an underscore
		if (OrigPythonizedNameLen > 0 && PythonizedName[OrigPythonizedNameLen - 1] != TEXT('_'))
		{
			PythonizedName += TEXT('_');
		}

		// Append this part of the identifier
		PythonizedName.AppendChars(&InName[PrevBreak], NameBreak - PrevBreak);

		// Remove any trailing underscores in the last part of the identifier
		while (PythonizedName.Len() > OrigPythonizedNameLen)
		{
			const int32 CharIndex = PythonizedName.Len() - 1;
			if (PythonizedName[CharIndex] != TEXT('_'))
			{
				break;
			}
			PythonizedName.RemoveAt(CharIndex, EAllowShrinking::No);
		}

		PrevBreak = NameBreak;
	}
	NameBreakIterator->ClearString();

	if (InNameCase == EPythonizeNameCase::Lower)
	{
		PythonizedName.ToLowerInline();
	}
	else if (InNameCase == EPythonizeNameCase::Upper)
	{
		PythonizedName.ToUpperInline();
	}

	// Don't allow the name to conflict with a keyword
	if (ReservedKeywords.Contains(PythonizedName))
	{
		PythonizedName += TEXT('_');
	}

	return PythonizedName;
}

FString PythonizePropertyName(FStringView InName, const EPythonizeNameCase InNameCase)
{
	int32 NameOffset = 0;

	for (;;)
	{
		// Strip the "b" prefix from bool names
		if (InName.Len() - NameOffset >= 2 && InName[NameOffset] == TEXT('b') && FChar::IsUpper(InName[NameOffset + 1]))
		{
			NameOffset += 1;
			continue;
		}

		// Strip the "In" prefix from names
		if (InName.Len() - NameOffset >= 3 && InName[NameOffset] == TEXT('I') && InName[NameOffset + 1] == TEXT('n') && FChar::IsUpper(InName[NameOffset + 2]))
		{
			NameOffset += 2;
			continue;
		}

		// Nothing more to strip
		break;
	}

	FString PythonPropertyName = PythonizeName(NameOffset ? InName.RightChop(NameOffset) : InName, InNameCase);
	if (PythonPropertyName == TEXTVIEW("self"))
	{
		PythonPropertyName.InsertAt(0, TEXT('_'));
	}

	return PythonPropertyName;
}

FString PythonizePropertyTooltip(const FParsedTooltip& InTooltip, const FProperty* InProp, const uint64 InReadOnlyFlags)
{
	return PythonizeTooltip(InTooltip, FPythonizeTooltipContext(InProp, InReadOnlyFlags));
}

FString PythonizeFunctionTooltip(const FParsedTooltip& InTooltip, const UFunction* InFunc, const TSet<FName>& ParamsToIgnore)
{
	return PythonizeTooltip(InTooltip, FPythonizeTooltipContext(InFunc, ParamsToIgnore));
}

FString PythonizeTooltip(const FParsedTooltip& InTooltip, const FPythonizeTooltipContext& InContext)
{
	// Use Google style docstrings - http://google.github.io/styleguide/pyguide.html?showone=Comments#Comments

	FString PythonizedTooltip;
	PythonizedTooltip.Reserve(InTooltip.SourceTooltipLen);

	// Append the property type (if given)
	if (InContext.Prop)
	{
		PythonizedTooltip += TEXT('(');
		AppendPropertyPythonType(InContext.Prop, PythonizedTooltip);
		PythonizedTooltip += TEXT("): ");
		AppendPropertyPythonReadWriteState(InContext.Prop, PythonizedTooltip, InContext.ReadOnlyFlags);
		PythonizedTooltip += TEXT(' ');
	}

	// Append the basic tooltip text
	PythonizedTooltip += InTooltip.BasicTooltipText;
	PythonizedTooltip.TrimEndInline();

	// Add the deprecation message
	if (!InContext.DeprecationMessage.IsEmpty())
	{
		PythonizedTooltip += LINE_TERMINATOR TEXT("deprecated: ");
		PythonizedTooltip += InContext.DeprecationMessage;
	}

	// Process the misc tokens into PythonizedTooltip
	for (const FParsedTooltip::FMiscToken& MiscToken : InTooltip.MiscTokens)
	{
		PythonizedTooltip += LINE_TERMINATOR;
		PythonizedTooltip += MiscToken.TokenName.GetValue();
		PythonizedTooltip += TEXT(": ");
		PythonizedTooltip += MiscToken.TokenValue.GetValue();
	}

	// If we have a function, we populate the input and output parm tokens from the parsed tooltip data
	if (InContext.Func)
	{
		bool bIsBoolReturn = false;
		FParsedTooltip::FParamToken ReturnToken;
		if (const FProperty* ReturnProp = InContext.Func->GetReturnProperty())
		{
			bIsBoolReturn = ReturnProp->IsA<FBoolProperty>();
			ReturnToken.ParamName.SetValue(ReturnProp->GetName());
			ReturnToken.ParamType.SetValue(GetPropertyPythonType(ReturnProp));
			ReturnToken.ParamComment.SetValue(InTooltip.ReturnToken.ParamComment.GetValue());
		}

		FParsedTooltip::FParamTokensArray InputParamTokens;
		FParsedTooltip::FParamTokensArray OutputParamTokens;
		for (TFieldIterator<const FProperty> ParamIt(InContext.Func); ParamIt; ++ParamIt)
		{
			const FProperty* Param = *ParamIt;

			if (InContext.ParamsToIgnore.Contains(Param->GetFName()))
			{
				// Skip any params we were asked to ignore
				continue;
			}

			TStringBuilder<FName::StringBufferSize> ParamNameStrBuffer;
			Param->GetFName().ToString(ParamNameStrBuffer);
			FStringView ParamNameStr = ParamNameStrBuffer;

			const FParsedTooltip::FParamToken* ParsedParamToken = InTooltip.ParamTokens.FindByPredicate([ParamNameStr](const FParsedTooltip::FParamToken& InParamToken)
			{
				return InParamToken.ParamName.GetValue() == ParamNameStr;
			});

			auto PopulateFinalFuncParamToken = [ParamNameStr, ParsedParamToken](FParsedTooltip::FParamToken& InOutParamToken, const FProperty* InParam)
			{
				if (ParsedParamToken && ParsedParamToken->ParamName.GetValue().Equals(ParamNameStr, ESearchCase::CaseSensitive))
				{
					// Name is cased correctly, so reference the parsed view to avoid a string copy
					InOutParamToken.ParamName.SetValue(ParsedParamToken->ParamName.GetValue());
				}
				else
				{
					// Name is incorrect or missing, so copy the name with the correct case
					InOutParamToken.ParamName.SetValue(FString(ParamNameStr));
				}

				InOutParamToken.ParamType.SetValue(GetPropertyPythonType(InParam));

				if (ParsedParamToken)
				{
					InOutParamToken.ParamComment.SetValue(ParsedParamToken->ParamComment.GetValue());
				}
			};

			if (PyUtil::IsInputParameter(Param))
			{
				FParsedTooltip::FParamToken& InputParamToken = InputParamTokens.AddDefaulted_GetRef();
				PopulateFinalFuncParamToken(InputParamToken, Param);
			}

			if (PyUtil::IsOutputParameter(Param))
			{
				FParsedTooltip::FParamToken& OutputParamToken = OutputParamTokens.AddDefaulted_GetRef();
				PopulateFinalFuncParamToken(OutputParamToken, Param);
			}
		}

		// Append input parameters
		if (InputParamTokens.Num() > 0)
		{
			PythonizedTooltip += LINE_TERMINATOR LINE_TERMINATOR TEXT("Args:");

			for (const FParsedTooltip::FParamToken& ParamToken : InputParamTokens)
			{
				// The parameters need to be indented
				PythonizedTooltip += LINE_TERMINATOR TEXT("    ");
				PythonizedTooltip += PythonizePropertyName(ParamToken.ParamName.GetValue(), EPythonizeNameCase::Lower);

				if (!ParamToken.ParamType.GetValue().IsEmpty())
				{
					PythonizedTooltip += TEXT(" (");
					PythonizedTooltip += ParamToken.ParamType.GetValue();
					PythonizedTooltip += TEXT(')');
				}

				// Add colon even if there's no comment
				PythonizedTooltip += TEXT(": ");
				PythonizedTooltip += ParamToken.ParamComment.GetValue();
			}
		}

		// Process return and output parameters
		if (!ReturnToken.ParamName.GetValue().IsEmpty() || OutputParamTokens.Num() > 0)
		{
			// Work out the return value type
			PythonizedTooltip += LINE_TERMINATOR LINE_TERMINATOR TEXT("Returns:") LINE_TERMINATOR TEXT("    ");
			{
				// Track whether we actually wrote anything below
				const int32 PreReturnLen = PythonizedTooltip.Len();
				{
					if (OutputParamTokens.Num() > 0)
					{
						PythonizedTooltip += (OutputParamTokens.Num() == 1 ? OutputParamTokens[0].ParamType.GetValue() : TEXT("tuple"));
						if (bIsBoolReturn)
						{
							PythonizedTooltip += TEXT(" or None");
						}
					}
					else
					{
						PythonizedTooltip += ReturnToken.ParamType.GetValue();
					}
				}
				if (PythonizedTooltip.Len() > PreReturnLen)
				{
					// Add colon even if there's no comment
					PythonizedTooltip += TEXT(": ");
				}
			}
			PythonizedTooltip += ReturnToken.ParamComment.GetValue();

			for (const FParsedTooltip::FParamToken& ParamToken : OutputParamTokens)
			{
				// The parameters need to be indented
				PythonizedTooltip += LINE_TERMINATOR LINE_TERMINATOR TEXT("    ");
				PythonizedTooltip += PythonizePropertyName(ParamToken.ParamName.GetValue(), EPythonizeNameCase::Lower);

				if (!ParamToken.ParamType.GetValue().IsEmpty())
				{
					PythonizedTooltip += TEXT(" (");
					PythonizedTooltip += ParamToken.ParamType.GetValue();
					PythonizedTooltip += TEXT(')');
				}

				// Add colon even if there's no comment
				PythonizedTooltip += TEXT(": ");
				PythonizedTooltip += ParamToken.ParamComment.GetValue();
			}
		}
	}

	PythonizedTooltip.TrimEndInline();

	return PythonizedTooltip;
}

FParsedTooltip ParseTooltip(FStringView InTooltip)
{
	const FStringView SourceTooltip = InTooltip;
	int32 SourceTooltipParseIndex = 0;

	FParsedTooltip ParsedTooltip;
	ParsedTooltip.SourceTooltipLen = InTooltip.Len();
	ParsedTooltip.BasicTooltipText.Reserve(SourceTooltip.Len());

	auto SkipToNextToken = [&SourceTooltip, &SourceTooltipParseIndex]()
	{
		while (SourceTooltipParseIndex < SourceTooltip.Len() && (FChar::IsWhitespace(SourceTooltip[SourceTooltipParseIndex]) || SourceTooltip[SourceTooltipParseIndex] == TEXT('-')))
		{
			++SourceTooltipParseIndex;
		}
	};

	auto ParseSimpleToken = [&SourceTooltip, &SourceTooltipParseIndex](FParsedTooltip::FTokenString& OutToken)
	{
		const int32 TokenStartIndex = SourceTooltipParseIndex;
		while (SourceTooltipParseIndex < SourceTooltip.Len() && !FChar::IsWhitespace(SourceTooltip[SourceTooltipParseIndex]))
		{
			++SourceTooltipParseIndex;
		}
		OutToken.SimpleValue = SourceTooltip.Mid(TokenStartIndex, SourceTooltipParseIndex - TokenStartIndex);
	};

	auto ParseComplexToken = [&SourceTooltip, &SourceTooltipParseIndex](FParsedTooltip::FTokenString& OutToken)
	{
		int32 TokenStartIndex = SourceTooltipParseIndex;
		while (SourceTooltipParseIndex < SourceTooltip.Len() && SourceTooltip[SourceTooltipParseIndex] != TEXT('@'))
		{
			// Convert a new-line within a token to a space
			if (FChar::IsLinebreak(SourceTooltip[SourceTooltipParseIndex]))
			{
				// Can no longer process this as a simple token - copy what we've parsed so far and reset...
				if (TokenStartIndex != INDEX_NONE)
				{
					OutToken.ComplexValue = SourceTooltip.Mid(TokenStartIndex, SourceTooltipParseIndex - TokenStartIndex);
					TokenStartIndex = INDEX_NONE;
				}

				while (SourceTooltipParseIndex < SourceTooltip.Len() && FChar::IsLinebreak(SourceTooltip[SourceTooltipParseIndex]))
				{
					++SourceTooltipParseIndex;
				}

				while (SourceTooltipParseIndex < SourceTooltip.Len() && FChar::IsWhitespace(SourceTooltip[SourceTooltipParseIndex]))
				{
					++SourceTooltipParseIndex;
				}

				OutToken.ComplexValue += TEXT(' ');
			}

			// Sanity check in case the first character after the new-line is @
			if (SourceTooltipParseIndex < SourceTooltip.Len() && SourceTooltip[SourceTooltipParseIndex] != TEXT('@'))
			{
				if (TokenStartIndex == INDEX_NONE)
				{
					OutToken.ComplexValue += SourceTooltip[SourceTooltipParseIndex];
				}
				++SourceTooltipParseIndex;
			}
		}
		if (TokenStartIndex == INDEX_NONE)
		{
			OutToken.ComplexValue.TrimEndInline();
		}
		else
		{
			OutToken.SimpleValue = SourceTooltip.Mid(TokenStartIndex, SourceTooltipParseIndex - TokenStartIndex);
			OutToken.SimpleValue.TrimEndInline();
		}
	};

	// Parse the tooltip for its tokens and values (basic content goes directly into PythonizedBaseTooltip)
	// TODO: Can we parse BasicTooltipText as multiple FTokenString blocks? It's mostly a contiguous blob...
	while (SourceTooltipParseIndex < SourceTooltip.Len())
	{
		if (SourceTooltip[SourceTooltipParseIndex] == TEXT('@'))
		{
			++SourceTooltipParseIndex; // Walk over the @
			if (SourceTooltip[SourceTooltipParseIndex] == TEXT('@'))
			{
				// Literal @ character
				ParsedTooltip.BasicTooltipText += TEXT('@');
				continue;
			}

			// Parse out the token name
			FParsedTooltip::FTokenString TokenName;
			SkipToNextToken();
			ParseSimpleToken(TokenName);

			if (TokenName.GetValue() == TEXT("param"))
			{
				FParsedTooltip::FParamToken& ParamToken = ParsedTooltip.ParamTokens.AddDefaulted_GetRef();

				// Parse out the parameter name
				SkipToNextToken();
				ParseSimpleToken(ParamToken.ParamName);

				// Parse out the parameter comment
				SkipToNextToken();
				ParseComplexToken(ParamToken.ParamComment);
			}
			else if (TokenName.GetValue() == TEXT("return") || TokenName.GetValue() == TEXT("returns"))
			{
				// Parse out the return value token
				SkipToNextToken();
				ParseComplexToken(ParsedTooltip.ReturnToken.ParamComment);
			}
			else
			{
				FParsedTooltip::FMiscToken& MiscToken = ParsedTooltip.MiscTokens.AddDefaulted_GetRef();
				MiscToken.TokenName = MoveTemp(TokenName);

				// Parse out the token value
				SkipToNextToken();
				ParseComplexToken(MiscToken.TokenValue);
			}
		}
		else
		{
			// Normal character
			ParsedTooltip.BasicTooltipText += SourceTooltip[SourceTooltipParseIndex++];
		}
	}

	return ParsedTooltip;
}

void PythonizeStructValueImpl(const UScriptStruct* InStruct, const void* InStructValue, const EPythonizeFlags InFlags, FString& OutPythonDefaultValue);

void PythonizeValueImpl(const FProperty* InProp, const void* InPropValue, const EPythonizeFlags InFlags, FString& OutPythonDefaultValue)
{
	static const bool bIsForDocString = false;

	const bool bIncludeUnrealNamespace = EnumHasAnyFlags(InFlags, EPythonizeFlags::IncludeUnrealNamespace);
	const bool bUseStrictTyping = EnumHasAnyFlags(InFlags, EPythonizeFlags::UseStrictTyping);

	const TCHAR* UnrealNamespace = bIncludeUnrealNamespace ? TEXT("unreal.") : TEXT("");

	if (InProp->ArrayDim > 1)
	{
		OutPythonDefaultValue += bUseStrictTyping
			? FString::Printf(TEXT("%sFixedArray.cast(%s, ["), UnrealNamespace, *GetPropertyTypePythonName(InProp, bIncludeUnrealNamespace, bIsForDocString))
			: TEXT("[");
	}
	for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
	{
		const void* PropArrValue = ((uint8*)InPropValue) + (InProp->GetElementSize() * ArrIndex);
		if (ArrIndex > 0)
		{
			OutPythonDefaultValue += TEXT(", ");
		}

		if (const FByteProperty* ByteProp = CastField<const FByteProperty>(InProp))
		{
			if (ByteProp->Enum)
			{
				const uint8 EnumVal = ByteProp->GetPropertyValue(PropArrValue);
				const int32 EnumIndex = ByteProp->Enum->GetIndexByValue(EnumVal);
				const FString EnumValStr = IsScriptExposedEnumEntry(ByteProp->Enum, EnumIndex) ? GetEnumEntryPythonName(ByteProp->Enum, EnumIndex) : FString();
				OutPythonDefaultValue += EnumValStr.IsEmpty() ? TEXT("0") : *FString::Printf(TEXT("%s%s.%s"), UnrealNamespace, *GetEnumPythonName(ByteProp->Enum), *EnumValStr);
			}
			else
			{
				ByteProp->ExportText_Direct(OutPythonDefaultValue, PropArrValue, PropArrValue, nullptr, PPF_None);
			}
		}
		else if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProp))
		{
			FNumericProperty* EnumInternalProp = EnumProp->GetUnderlyingProperty();
			const int64 EnumVal = EnumInternalProp->GetSignedIntPropertyValue(PropArrValue);
			const int32 EnumIndex = EnumProp->GetEnum()->GetIndexByValue(EnumVal);
			const FString EnumValStr = IsScriptExposedEnumEntry(EnumProp->GetEnum(), EnumIndex) ? GetEnumEntryPythonName(EnumProp->GetEnum(), EnumIndex) : FString();
			OutPythonDefaultValue += EnumValStr.IsEmpty() ? TEXT("0") : *FString::Printf(TEXT("%s%s.%s"), UnrealNamespace, *GetEnumPythonName(EnumProp->GetEnum()), *EnumValStr);
		}
		else if (const FBoolProperty* BoolProp = CastField<const FBoolProperty>(InProp))
		{
			OutPythonDefaultValue += BoolProp->GetPropertyValue(PropArrValue) ? TEXT("True") : TEXT("False");
		}
		else if (const FNameProperty* NameProp = CastField<const FNameProperty>(InProp))
		{
			FString NameStrValue = NameProp->GetPropertyValue(PropArrValue).ToString();
			NameStrValue.ReplaceCharWithEscapedCharInline();
			OutPythonDefaultValue += bUseStrictTyping
				? FString::Printf(TEXT("%sName(\"%s\")"), UnrealNamespace, *NameStrValue)
				: FString::Printf(TEXT("\"%s\""), *NameStrValue);
		}
		else if (const FTextProperty* TextProp = CastField<const FTextProperty>(InProp))
		{
			FString TextStrValue = TextProp->GetPropertyValue(PropArrValue).BuildSourceString();
			TextStrValue.ReplaceCharWithEscapedCharInline();
			OutPythonDefaultValue += bUseStrictTyping
				? FString::Printf(TEXT("%sText(\"%s\")"), UnrealNamespace, *TextStrValue)
				: FString::Printf(TEXT("\"%s\""), *TextStrValue);
		}
		else if (const FFieldPathProperty* FieldPathProp =  CastField<const FFieldPathProperty>(InProp))
		{
			const FString FieldPathStrValue = FieldPathProp->GetPropertyValue(PropArrValue).ToString();
			if (FieldPathStrValue.IsEmpty())
			{
				OutPythonDefaultValue += bUseStrictTyping
					? FString::Printf(TEXT("%sFieldPath()"), UnrealNamespace)
					: FString::Printf(TEXT("FieldPath()"));
			}
			else // Use the path string as default value.
			{
				OutPythonDefaultValue += bUseStrictTyping
					? FString::Printf(TEXT("%sFieldPath(\"%s\")"), UnrealNamespace, *FieldPathStrValue)
					: FString::Printf(TEXT("FieldPath(\"%s\")"), *FieldPathStrValue);
			}
		}
		else if (const FObjectPropertyBase* ObjProp = CastField<const FObjectPropertyBase>(InProp))
		{
			OutPythonDefaultValue += TEXT("None");
		}
		else if (const FInterfaceProperty* InterfaceProp = CastField<const FInterfaceProperty>(InProp))
		{
			OutPythonDefaultValue += TEXT("None");
		}
		else if (const FOptionalProperty* OptionalProp = CastField<const FOptionalProperty>(InProp))
		{
			OutPythonDefaultValue += TEXT("None");
		}
		else if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProp))
		{
			PythonizeStructValueImpl(StructProp->Struct, PropArrValue, InFlags, OutPythonDefaultValue);
		}
		else if (const FDelegateProperty* DelegateProp = CastField<const FDelegateProperty>(InProp))
		{
			OutPythonDefaultValue += FString::Printf(TEXT("%s%s()"), UnrealNamespace, *GetDelegatePythonName(DelegateProp->SignatureFunction));
		}
		else if (const FMulticastDelegateProperty* MulticastDelegateProp = CastField<const FMulticastDelegateProperty>(InProp))
		{
			OutPythonDefaultValue += FString::Printf(TEXT("%s%s()"), UnrealNamespace, *GetDelegatePythonName(MulticastDelegateProp->SignatureFunction));
		}
		else if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(InProp))
		{
			OutPythonDefaultValue += bUseStrictTyping
				? FString::Printf(TEXT("%sArray.cast(%s, ["), UnrealNamespace, *GetPropertyTypePythonName(ArrayProperty->Inner, bIncludeUnrealNamespace, bIsForDocString))
				: TEXT("[");
			{
				FScriptArrayHelper ScriptArrayHelper(ArrayProperty, PropArrValue);
				const int32 ElementCount = ScriptArrayHelper.Num();
				for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
				{
					if (ElementIndex > 0)
					{
						OutPythonDefaultValue += TEXT(", ");
					}
					PythonizeValueImpl(ArrayProperty->Inner, ScriptArrayHelper.GetRawPtr(ElementIndex), InFlags, OutPythonDefaultValue);
				}
			}
			OutPythonDefaultValue += bUseStrictTyping
				? TEXT("])")
				: TEXT("]");
		}
		else if (const FSetProperty* SetProperty = CastField<const FSetProperty>(InProp))
		{
			OutPythonDefaultValue += bUseStrictTyping
				? FString::Printf(TEXT("%sSet.cast(%s, ["), UnrealNamespace, *GetPropertyTypePythonName(SetProperty->ElementProp, bIncludeUnrealNamespace, bIsForDocString))
				: TEXT("[");
			{
				FScriptSetHelper ScriptSetHelper(SetProperty, PropArrValue);
				for (FScriptSetHelper::FIterator It(ScriptSetHelper); It; ++It)
				{
					if (It.GetLogicalIndex() > 0)
					{
						OutPythonDefaultValue += TEXT(", ");
					}
					PythonizeValueImpl(ScriptSetHelper.GetElementProperty(), ScriptSetHelper.GetElementPtr(It), InFlags, OutPythonDefaultValue);
				}
			}
			OutPythonDefaultValue += bUseStrictTyping
				? TEXT("])")
				: TEXT("]");
		}
		else if (const FMapProperty* MapProperty = CastField<const FMapProperty>(InProp))
		{
			OutPythonDefaultValue += bUseStrictTyping
				? FString::Printf(TEXT("%sMap.cast(%s, %s, {"), UnrealNamespace, *GetPropertyTypePythonName(MapProperty->KeyProp, bIncludeUnrealNamespace, bIsForDocString), *GetPropertyTypePythonName(MapProperty->ValueProp, bIncludeUnrealNamespace, bIsForDocString))
				: TEXT("{");
			{
				FScriptMapHelper ScriptMapHelper(MapProperty, PropArrValue);
				for (FScriptMapHelper::FIterator It(ScriptMapHelper); It; ++It)
				{
					if (It.GetLogicalIndex() > 0)
					{
						OutPythonDefaultValue += TEXT(", ");
					}
					PythonizeValueImpl(ScriptMapHelper.GetKeyProperty(), ScriptMapHelper.GetKeyPtr(It), InFlags, OutPythonDefaultValue);
					OutPythonDefaultValue += TEXT(": ");
					PythonizeValueImpl(ScriptMapHelper.GetValueProperty(), ScriptMapHelper.GetValuePtr(It), InFlags, OutPythonDefaultValue);
				}
			}
			OutPythonDefaultValue += bUseStrictTyping
				? TEXT("})")
				: TEXT("}");
		}
		else
		{
			// Property ExportText is already in the correct form for Python (PPF_Delimited so that strings get quoted)
			InProp->ExportText_Direct(OutPythonDefaultValue, PropArrValue, PropArrValue, nullptr, PPF_Delimited);
		}
	}
	if (InProp->ArrayDim > 1)
	{
		OutPythonDefaultValue += bUseStrictTyping
			? TEXT("])")
			: TEXT("]");
	}
}

void PythonizeStructValueImpl(const UScriptStruct* InStruct, const void* InStructValue, const EPythonizeFlags InFlags, FString& OutPythonDefaultValue)
{
	const bool bIncludeUnrealNamespace = EnumHasAnyFlags(InFlags, EPythonizeFlags::IncludeUnrealNamespace);
	const bool bUseStrictTyping = EnumHasAnyFlags(InFlags, EPythonizeFlags::UseStrictTyping);
	const bool bDefaultConstructStructs = EnumHasAnyFlags(InFlags, EPythonizeFlags::DefaultConstructStructs);
	const bool bDefaultConstructDateTime = EnumHasAnyFlags(InFlags, EPythonizeFlags::DefaultConstructDateTime);

	const TCHAR* UnrealNamespace = bIncludeUnrealNamespace ? TEXT("unreal.") : TEXT("");

	// Note: We deliberately don't use any FPyWrapperStruct functionality here as this function may be called as part of generating the wrapped type

	auto FindMakeBreakFunction = [InStruct](const FName& InKey) -> const UFunction*
	{
		const FString MakeBreakName = InStruct->GetMetaData(InKey);
		if (!MakeBreakName.IsEmpty())
		{
			return FindObject<UFunction>(nullptr, *MakeBreakName, EFindObjectFlags::ExactClass);
		}
		return nullptr;
	};

	// If the struct has a make function, we assume the output of the break function matches the input of the make function
	OutPythonDefaultValue += bUseStrictTyping 
		? FString::Printf(TEXT("%s%s("), UnrealNamespace, *GetStructPythonName(InStruct))
		: TEXT("[");
	if (!bDefaultConstructStructs && (!bDefaultConstructDateTime || !InStruct->IsChildOf(TBaseStructure<FDateTime>::Get())))
	{
		const UFunction* MakeFunc = FindMakeBreakFunction(HasNativeMakeMetaDataKey);
		const UFunction* BreakFunc = FindMakeBreakFunction(HasNativeBreakMetaDataKey);

		if (MakeFunc && BreakFunc)
		{
			UClass* Class = BreakFunc->GetOwnerClass();
			UObject* Obj = Class->GetDefaultObject();

			FGeneratedWrappedFunction BreakFuncDef;
			BreakFuncDef.SetFunction(BreakFunc, FGeneratedWrappedFunction::SFF_ExtractParameters);

			// Python can only support 255 parameters, so if we have more than that for this struct just use the default constructor
			if (BreakFuncDef.OutputParams.Num() <= 255)
			{
				// Call the break function using the instance we were given
				PY_UFUNCTION_STACK(FuncParams, BreakFuncDef.Func);
				if (BreakFuncDef.InputParams.Num() == 1 && CastField<FStructProperty>(BreakFuncDef.InputParams[0].ParamProp) && InStruct->IsChildOf(CastFieldChecked<const FStructProperty>(BreakFuncDef.InputParams[0].ParamProp)->Struct))
				{
					// Copy the given instance as the 'self' argument
					const FGeneratedWrappedMethodParameter& SelfParam = BreakFuncDef.InputParams[0];
					void* SelfArgInstance = SelfParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
					CastFieldChecked<const FStructProperty>(SelfParam.ParamProp)->Struct->CopyScriptStruct(SelfArgInstance, InStructValue);
				}

				PyGenUtil::FMultithreadedGenerationContext::ExecuteGameThreadTask(
					[&Obj, &BreakFuncDef, &FuncParams]()
					{
						FPyScopedGIL GIL;
						PyUtil::InvokeFunctionCall(Obj, BreakFuncDef.Func, FuncParams.GetMemory(), TEXT("pythonize default struct value"));
						PyErr_Clear(); // Clear any errors in case InvokeFunctionCall failed
					}
				);
				
				// Extract the output argument values as defaults for the struct
				for (int32 OuputParamIndex = 0; OuputParamIndex < BreakFuncDef.OutputParams.Num(); ++OuputParamIndex)
				{
					const FGeneratedWrappedMethodParameter& OutputParam = BreakFuncDef.OutputParams[OuputParamIndex];
					if (OuputParamIndex > 0)
					{
						OutPythonDefaultValue += TEXT(", ");
					}
					PythonizeValueImpl(OutputParam.ParamProp, OutputParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory()), InFlags, OutPythonDefaultValue);
				}
			}
		}
		else
		{
			int32 ExportedPropertyCount = 0;
			FString StructInitParamsStr;
			for (TFieldIterator<const FProperty> PropIt(InStruct, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				const FProperty* Prop = *PropIt;
				if (ShouldExportProperty(Prop) && !IsDeprecatedProperty(Prop))
				{
					if (ExportedPropertyCount++ > 0)
					{
						StructInitParamsStr += TEXT(", ");
					}
					PythonizeValueImpl(Prop, Prop->ContainerPtrToValuePtr<void>(InStructValue), InFlags, StructInitParamsStr);
				}
			}

			// Python can only support 255 parameters, so if we have more than that for this struct just use the default constructor
			if (ExportedPropertyCount <= 255)
			{
				OutPythonDefaultValue += StructInitParamsStr;
			}
		}
	}
	OutPythonDefaultValue += bUseStrictTyping
		? TEXT(")")
		: TEXT("]");
}

FString PythonizeValue(const FProperty* InProp, const void* InPropValue, const EPythonizeFlags InFlags)
{
	FString PythonValue;
	PythonizeValueImpl(InProp, InPropValue, InFlags, PythonValue);
	return PythonValue;
}

FString PythonizeDefaultValue(const FProperty* InProp, const FString& InDefaultValue, const EPythonizeFlags InFlags)
{
	PyUtil::FPropValueOnScope PropValue(PyUtil::FConstPropOnScope::ExternalReference(InProp));
	PyUtil::ImportDefaultValue(InProp, PropValue.GetValue(), InDefaultValue);
	return PythonizeValue(InProp, PropValue.GetValue(), InFlags);
}

FString PythonizeMethodParam(const FGeneratedWrappedMethodParameter& MethodParam, EPythonizeFlags InPythonizeFlags, const TFunction<bool(const FString&)>& ShouldEllipseDefaultValue)
{
	bool bWithTypeHinting = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeHinting);
	bool bIncludeUnrealNamespace = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::IncludeUnrealNamespace);
	bool bEllipseMissingDefaultValue = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::AddMissingDefaultValueEllipseWorkaround);

	FString ParamName(UTF8_TO_TCHAR(MethodParam.ParamName.GetData()));
	FString ParamType(bWithTypeHinting ? GetPropertyTypePythonName(MethodParam.ParamProp, bIncludeUnrealNamespace, /*bIsForDocString*/false, InPythonizeFlags) : FString());
	FString DefaultValue;

	if (bEllipseMissingDefaultValue && !MethodParam.ParamDefaultValue.IsSet())
	{
		DefaultValue = TEXT("..."); // Add the missing default value.
	}
	else if (MethodParam.ParamDefaultValue.IsSet())
	{
		DefaultValue = PyGenUtil::PythonizeDefaultValue(MethodParam.ParamProp, MethodParam.ParamDefaultValue.GetValue(), InPythonizeFlags);
		if (ShouldEllipseDefaultValue && ShouldEllipseDefaultValue(DefaultValue))
		{
			DefaultValue = TEXT("..."); // In some context (like generated stub), the default value might refer to not-yet declared types, which is illegal
		}
	}

	return PythonizeMethodParam(ParamName, ParamType, DefaultValue);
}

FString PythonizeMethodParam(const FString& InParamName, const FProperty* ParamProp, EPythonizeFlags InPythonizeFlags)
{
	bool bWithTypeHinting = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeHinting);
	bool bIncludeUnrealNamespace = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::IncludeUnrealNamespace);

	FString ParamType(bWithTypeHinting ? GetPropertyTypePythonName(ParamProp, bIncludeUnrealNamespace, /*bIsForDocString*/false, InPythonizeFlags) : FString());

	return PythonizeMethodParam(InParamName, ParamType);
}

FString PythonizeMethodParam(const FString& InParamName, const FString& InParamType, const FString& InDefaultValue)
{
	return InDefaultValue.IsEmpty()
		? InParamType.IsEmpty() ? FString::Printf(TEXT("%s"), *InParamName) : FString::Printf(TEXT("%s: %s"), *InParamName, *InParamType)
		: InParamType.IsEmpty() ? FString::Printf(TEXT("%s = %s"), *InParamName, *InDefaultValue) : FString::Printf(TEXT("%s: %s = %s"), *InParamName, *InParamType, *InDefaultValue);
}

FString PythonizeMethodReturnType(const TArray<PyGenUtil::FGeneratedWrappedMethodParameter>& InOutputParams, EPythonizeFlags InPythonizeFlags)
{
	// If type hinting is disabled, return type is not specified with the method signature.
	if (!EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeHinting))
	{
		return FString();
	}
	if (InOutputParams.Num() == 0)
	{
		return TEXT("None");
	}

	bool bIncludeUnrealNamespace = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::IncludeUnrealNamespace);
	bool bCanReturnNone = false;

	// If we have multiple return values and the main return value is a bool, skip it (to mimic PyGenUtils::PackReturnValues)
	int32 ReturnPropIndex = 0;
	if (InOutputParams.Num() > 1 && InOutputParams[0].ParamProp->HasAnyPropertyFlags(CPF_ReturnParm) && InOutputParams[0].ParamProp->IsA<FBoolProperty>())
	{
		ReturnPropIndex = 1; // Start packing at the 1st out value
		bCanReturnNone = true; // When the C++ function returns false, the Python methods returns None, otherwise, it returns the output parameter(s) as a single value or a tuple.
	}

	// Do we need to return a packed tuple, or just a single value?
	const int32 NumPropertiesToPack = InOutputParams.Num() - ReturnPropIndex;
	if (NumPropertiesToPack == 1)
	{
		const PyGenUtil::FGeneratedWrappedMethodParameter& ReturnParam = InOutputParams[ReturnPropIndex];
		if (bCanReturnNone)
		{
			return FString::Printf(TEXT("Optional[%s]"), *PyGenUtil::GetPropertyTypePythonName(ReturnParam.ParamProp, bIncludeUnrealNamespace, /*bIsForDocString*/false, InPythonizeFlags));
		}
		return PyGenUtil::GetPropertyTypePythonName(ReturnParam.ParamProp, bIncludeUnrealNamespace, /*bIsForDocString*/false, InPythonizeFlags);
	}
	else // Returns a tuple
	{
		FString ReturnType;
		ReturnType = bCanReturnNone ? TEXT("Optional[Tuple[") : TEXT("Tuple[");
		for (int32 PackedPropIndex = 0; ReturnPropIndex < InOutputParams.Num(); ++ReturnPropIndex, ++PackedPropIndex)
		{
			if (PackedPropIndex > 0)
			{
				ReturnType += TEXT(", ");
			}
			const PyGenUtil::FGeneratedWrappedMethodParameter& ReturnParam = InOutputParams[ReturnPropIndex];
			ReturnType += PyGenUtil::GetPropertyTypePythonName(ReturnParam.ParamProp, bIncludeUnrealNamespace, /*bIsForDocString*/false, InPythonizeFlags);
		}
		ReturnType += (bCanReturnNone ? TEXT("]]") : TEXT("]"));
		return ReturnType;
	}
}

FString PythonizeMethodReturnType(const FProperty* InProp, EPythonizeFlags InPythonizeFlags)
{
	// If type hinting is disabled, return type is not specified with the method signature.
	if (!EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeHinting))
	{
		return FString();
	}

	bool bIncludeUnrealNamespace = EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::IncludeUnrealNamespace);
	return PyGenUtil::GetPropertyTypePythonName(InProp, bIncludeUnrealNamespace, /*bIsForDocString*/false, InPythonizeFlags);
}

const UObject* GetAssetTypeRegistryType(const UObject* InObj)
{
	if (const UBlueprintCore* BlueprintAsset = Cast<const UBlueprintCore>(InObj))
	{
		return BlueprintAsset->GeneratedClass;
	}

	return InObj;
}

FString GetFieldModule(const UField* InField)
{
	UPackage* ScriptPackage = InField->GetOutermost();
	
	const FString PackageName = ScriptPackage->GetName();
	if (PackageName.StartsWith(TEXT("/Script/")))
	{
		return PackageName.RightChop(8); // Chop "/Script/" from the name
	}
	
	// Not a native module!
	return FString();
}

FString GetFieldPlugin(const UField* InField)
{
	static const TMap<FName, FString> ModuleNameToPluginMap = []()
	{
		IPluginManager& PluginManager = IPluginManager::Get();

		// Build up a map of plugin modules -> plugin names
		TMap<FName, FString> PluginModules;
		{
			TArray<TSharedRef<IPlugin>> Plugins = PluginManager.GetDiscoveredPlugins();
			for (const TSharedRef<IPlugin>& Plugin : Plugins)
			{
				for (const FModuleDescriptor& PluginModule : Plugin->GetDescriptor().Modules)
				{
					PluginModules.Add(PluginModule.Name, Plugin->GetName());
				}
			}
		}
		return PluginModules;
	}();

	const FString* FieldPluginNamePtr = ModuleNameToPluginMap.Find(*GetFieldModule(InField));
	return FieldPluginNamePtr ? *FieldPluginNamePtr : FString();
}

FString GetModulePythonName(const FName InModuleName, const bool bIncludePrefix)
{
	// Some modules are mapped to others in Python
	static const FName PythonModuleMappings[][2] = {
		{ TEXT("CoreUObject"), TEXT("Core") },
		{ TEXT("SlateCore"), TEXT("Slate") },
		{ TEXT("UnrealEd"), TEXT("Editor") },
		{ TEXT("PythonScriptPlugin"), TEXT("Python") },
	};

	FName MappedModuleName = InModuleName;
	for (const auto& PythonModuleMapping : PythonModuleMappings)
	{
		if (InModuleName == PythonModuleMapping[0])
		{
			MappedModuleName = PythonModuleMapping[1];
			break;
		}
	}

	const FString ModulePythonName = MappedModuleName.ToString().ToLower();
	return bIncludePrefix ? FString::Printf(TEXT("_unreal_%s"), *ModulePythonName) : ModulePythonName;
}

bool GetFieldPythonNameFromMetaDataImpl(const FFieldVariant& InField, const FName InMetaDataKey, FString& OutFieldName)
{
	// See if we have a name override in the meta-data
	if (!InMetaDataKey.IsNone())
	{
		OutFieldName = InField.IsUObject() ? InField.Get<UField>()->GetMetaData(InMetaDataKey) : InField.Get<FField>()->GetMetaData(InMetaDataKey);

		// This may be a semi-colon separated list - the first item is the one we want for the current name
		if (!OutFieldName.IsEmpty())
		{
			int32 SemiColonIndex = INDEX_NONE;
			if (OutFieldName.FindChar(TEXT(';'), SemiColonIndex))
			{
				OutFieldName.RemoveAt(SemiColonIndex, OutFieldName.Len() - SemiColonIndex, EAllowShrinking::No);
			}

			FText ValidationError;
			if (!IsValidName(OutFieldName, &ValidationError))
			{
				if ( InField.IsUObject() )
				{
					const FString FieldPathName = InField.IsUObject() ? InField.Get<UField>()->GetPathName() : InField.Get<FField>()->GetPathName();

					REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("'%s' has an invalid '%s' meta-data value '%s': %s."), *FieldPathName, *InMetaDataKey.ToString(), *OutFieldName, *ValidationError.ToString());
				}
				
			}

			return true;
		}
	}

	return false;
}

bool GetDeprecatedFieldPythonNamesFromMetaDataImpl(const FFieldVariant& InField, const FName InMetaDataKey, TArray<TTuple<FSoftObjectPath, FString>>& OutFieldNames)
{
	// See if we have a name override in the meta-data
	if (!InMetaDataKey.IsNone())
	{
		const FString FieldName = InField.IsUObject() ? InField.Get<UField>()->GetMetaData(InMetaDataKey) : InField.Get<FField>()->GetMetaData(InMetaDataKey);

		// This may be a semi-colon separated list - everything but the first item is deprecated
		if (!FieldName.IsEmpty())
		{
			TArray<FString> LocalFieldNames;
			FieldName.ParseIntoArray(LocalFieldNames, TEXT(";"), false);

			// Remove the non-deprecated entry
			if (LocalFieldNames.Num() > 0)
			{
				LocalFieldNames.RemoveAt(0, EAllowShrinking::No);
			}

			// Trim whitespace and remove empty items
			LocalFieldNames.RemoveAll([](FString& InStr)
			{
				InStr.TrimStartAndEndInline();
				return InStr.IsEmpty();
			});

			OutFieldNames.Reset(LocalFieldNames.Num());
			{
				const FString FieldLeafName = InField.GetName();
				const FString FieldPathName = InField.GetPathName();
				for (const FString& FieldNamePart : LocalFieldNames)
				{
					FText ValidationError;
					if (!IsValidName(FieldNamePart, &ValidationError))
					{
						REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("'%s' has an invalid '%s' meta-data value '%s' (from '%s'): %s."), *FieldPathName, *InMetaDataKey.ToString(), *FieldNamePart, *FieldName, *ValidationError.ToString());
					}

					FString DeprecatedFieldPathName = FieldPathName;
					DeprecatedFieldPathName.RemoveFromEnd(FieldLeafName);
					DeprecatedFieldPathName.Append(FieldNamePart);
					OutFieldNames.Add(MakeTuple(FSoftObjectPath(DeprecatedFieldPathName), FieldNamePart));
				}
			}

			return true;
		}
	}

	return false;
}

FString GetFieldPythonNameImpl(const FFieldVariant& InField, const FName InMetaDataKey)
{
	FString FieldName;

	// First see if we have a name override in the meta-data
	if (GetFieldPythonNameFromMetaDataImpl(InField, InMetaDataKey, FieldName))
	{
		return FieldName;
	}

	// Just use the field name if we have no meta-data
	if (FieldName.IsEmpty())
	{
		FieldName = InField.GetName();

		// Strip the "E" prefix from enum names
		if (InField.IsA<UEnum>() && FieldName.Len() >= 2 && FieldName[0] == TEXT('E') && FChar::IsUpper(FieldName[1]))
		{
			FieldName.RemoveAt(0, EAllowShrinking::No);
		}

		// Classes, structs, and enums will no longer have their C++ prefix at this point
		// These prefixes can prevent types that start with numbers from being a compile error in C++
		// Ideally we don't want those prefixes to be used for Python, but we must ensure a valid symbol name
		if ( UObject* FieldObject = InField.ToUObject() )
		{
			if (FieldName.Len() >= 0 && FChar::IsDigit(FieldName[0]))
			{
				if (const UClass* Class = Cast<UClass>(FieldObject))
				{
					FieldName.InsertAt(0, Class->IsChildOf<AActor>() ? TEXT('A') : TEXT('U'));
				}
				else if (FieldObject->IsA<UScriptStruct>())
				{
					FieldName.InsertAt(0, TEXT('F'));
				}
				else if (FieldObject->IsA<UEnum>())
				{
					FieldName.InsertAt(0, TEXT('E'));
				}
			}
		}
	}

	return FieldName;
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedFieldPythonNamesImpl(const FFieldVariant& InField, const FName InMetaDataKey)
{
	TArray<TTuple<FSoftObjectPath, FString>> FieldNames;

	// First see if we have a name override in the meta-data
	if (GetDeprecatedFieldPythonNamesFromMetaDataImpl(InField, InMetaDataKey, FieldNames))
	{
		return FieldNames;
	}

	// Just use the redirects if we have no meta-data
	ECoreRedirectFlags RedirectFlags = ECoreRedirectFlags::None;
	if (InField.IsA<UFunction>())
	{
		RedirectFlags = ECoreRedirectFlags::Type_Function;
	}
	else if (InField.IsA<FProperty>())
	{
		RedirectFlags = ECoreRedirectFlags::Type_Property;
	}
	else if (InField.IsA<UClass>())
	{
		RedirectFlags = ECoreRedirectFlags::Type_Class;
	}
	else if (InField.IsA<UScriptStruct>())
	{
		RedirectFlags = ECoreRedirectFlags::Type_Struct;
	}
	else if (InField.IsA<UEnum>())
	{
		RedirectFlags = ECoreRedirectFlags::Type_Enum;
	}

	const FCoreRedirectObjectName CurrentName = FCoreRedirectObjectName(InField.GetPathName());
	TArray<FCoreRedirectObjectName> PreviousNames;
	FCoreRedirects::FindPreviousNames(RedirectFlags, CurrentName, PreviousNames);

	FieldNames.Reserve(PreviousNames.Num());
	for (const FCoreRedirectObjectName& PreviousName : PreviousNames)
	{
		// Redirects can be used to redirect outers
		// We want to skip those redirects as we only care about changes within the current scope
		if (!PreviousName.OuterName.IsNone() && PreviousName.OuterName != CurrentName.OuterName)
		{
			continue;
		}

		// Redirects can often keep the same name when updating the path
		// We want to skip those redirects as we only care about name changes
		if (PreviousName.ObjectName == CurrentName.ObjectName)
		{
			continue;
		}
		
		// If the previous name contains invalid characters, then skip it
		// It is likely redirecting from a BP, which has weaker naming restrictions than C++ or Python
		if (!IsValidName(FNameBuilder(PreviousName.ObjectName).ToView()))
		{
			continue;
		}

		// Class/Struct redirects may be used to point a parent type to a child type (eg, to enforce a specific derived implementation)
		// We want to skip those redirects as the parent type still exists in this case
		if (InField.IsA<UClass>() || InField.IsA<UScriptStruct>())
		{
			auto DoesRedirectFromParentType = [&InField, &PreviousName]()
			{
				UStruct* TargetStruct = InField.Get<UStruct>();
				for (UStruct* SuperStruct = TargetStruct->GetSuperStruct(); SuperStruct; SuperStruct = SuperStruct->GetSuperStruct())
				{
					if (SuperStruct->GetFName() == PreviousName.ObjectName)
					{
						return true;
					}
				}
				return false;
			};

			if (DoesRedirectFromParentType())
			{
				continue;
			}
		}

		// Redirects may be from a live field
		// We want to skip those redirects as the real field still exists
		if (InField.IsA<UClass>() || InField.IsA<UScriptStruct>() || InField.IsA<UEnum>())
		{
			UObject* PreviousType = nullptr;
			if (!PreviousName.PackageName.IsNone())
			{
				// Try a qualified search first...
				PreviousType = StaticFindObject(InField.Get<UObject>()->GetClass(), nullptr, *PreviousName.ToString());
			}
			if (!PreviousType)
			{
				// Try an unqualified search too, as the redirector might be broken or incorrect...
				PreviousType = StaticFindFirstObject(InField.Get<UObject>()->GetClass(), *PreviousName.ObjectName.ToString(), EFindFirstObjectOptions::ExactClass | EFindFirstObjectOptions::NativeFirst);
			}
			if (PreviousType)
			{
				continue;
			}
		}
		else if (InField.IsA<UFunction>())
		{
			if (const UClass* OwnerClass = InField.Get<UFunction>()->GetOwnerClass())
			{
				if (const UFunction* ExistingFunction = OwnerClass->FindFunctionByName(PreviousName.ObjectName);
					ExistingFunction && ShouldExportFunction(ExistingFunction))
				{
					continue;
				}
			}
		}
		else if (InField.IsA<FProperty>())
		{
			if (const UStruct* OwnerStruct = InField.Get<FProperty>()->GetOwnerStruct())
			{
				if (const FProperty* ExistingProperty = OwnerStruct->FindPropertyByName(PreviousName.ObjectName);
					ExistingProperty && ShouldExportProperty(ExistingProperty))
				{
					continue;
				}
			}
		}

		FString FieldName = PreviousName.ObjectName.ToString();

		// Strip the "E" prefix from enum names
		if (InField.IsA<UEnum>() && FieldName.Len() >= 2 && FieldName[0] == TEXT('E') && FChar::IsUpper(FieldName[1]))
		{
			FieldName.RemoveAt(0, EAllowShrinking::No);
		}

		// Only add a single redirector for each previous name
		if (FieldNames.ContainsByPredicate([&FieldName](TTuple<FSoftObjectPath, FString>& FieldNamePair) { return FieldNamePair.Value == FieldName; }))
		{
			continue;
		}

		// If PackageName is empty then assume that we're still using the same package as the current field
		FCoreRedirectObjectName PreviousNameWithPackage = PreviousName;
		if (PreviousNameWithPackage.PackageName.IsNone())
		{
			PreviousNameWithPackage.PackageName = InField.GetOutermost()->GetFName();
		}

		FieldNames.Add(MakeTuple(FSoftObjectPath(PreviousNameWithPackage.ToString()), MoveTemp(FieldName)));
	}

	return FieldNames;
}

FString GetClassPythonName(const UClass* InClass)
{
	return GetFieldPythonNameImpl(InClass, ScriptNameMetaDataKey);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedClassPythonNames(const UClass* InClass)
{
	return GetDeprecatedFieldPythonNamesImpl(InClass, ScriptNameMetaDataKey);
}

FString GetStructPythonName(const UScriptStruct* InStruct)
{
	return GetFieldPythonNameImpl(InStruct, ScriptNameMetaDataKey);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedStructPythonNames(const UScriptStruct* InStruct)
{
	return GetDeprecatedFieldPythonNamesImpl(InStruct, ScriptNameMetaDataKey);
}

FString GetEnumPythonName(const UEnum* InEnum)
{
	return GetFieldPythonNameImpl(InEnum, ScriptNameMetaDataKey);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedEnumPythonNames(const UEnum* InEnum)
{
	return GetDeprecatedFieldPythonNamesImpl(InEnum, ScriptNameMetaDataKey);
}

FString GetEnumEntryPythonName(const UEnum* InEnum, const int32 InEntryIndex)
{
	FString EnumEntryName;

	// First see if we have a name override in the meta-data
	{
		EnumEntryName = InEnum->GetMetaData(TEXT("ScriptName"), InEntryIndex);

		// This may be a semi-colon separated list - the first item is the one we want for the current name
		if (!EnumEntryName.IsEmpty())
		{
			int32 SemiColonIndex = INDEX_NONE;
			if (EnumEntryName.FindChar(TEXT(';'), SemiColonIndex))
			{
				EnumEntryName.RemoveAt(SemiColonIndex, EnumEntryName.Len() - SemiColonIndex, EAllowShrinking::No);
			}

			FText ValidationError;
			if (!IsValidName(EnumEntryName, &ValidationError))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("'%s' has an invalid 'ScriptName' meta-data value '%s': %s."), *InEnum->GetPathName(), *EnumEntryName, *ValidationError.ToString());
			}
		}
	}
	
	// Blueprint enums have horrible internal names, so try and create a valid identifier from the display name meta-data
	if (IsBlueprintGeneratedEnum(InEnum))
	{
		EnumEntryName = InEnum->GetAuthoredNameStringByIndex(InEntryIndex);
		if (FCString::IsPureAnsi(*EnumEntryName))
		{
			// Sanitize out any invalid characters in the name
			for (TCHAR& Char : EnumEntryName)
			{
				if (!FChar::IsAlnum(Char))
				{
					Char = TEXT('_');
				}
			}
		}
		else
		{
			// If the name isn't pure-ANSI then don't attempt to sanitize it as the result will be very mangled
			EnumEntryName.Reset();
		}
	}

	// Just use the entry name if we have no meta-data
	if (EnumEntryName.IsEmpty())
	{
		EnumEntryName = InEnum->GetNameStringByIndex(InEntryIndex);
	}

	return PythonizeName(EnumEntryName, EPythonizeNameCase::Upper);
}

TArray<FString> GetDeprecatedEnumEntryPythonNames(const UEnum* InEnum, const int32 InEntryIndex)
{
	TArray<FString> EnumEntryNames;

	// First see if we have a name override in the meta-data
	{
		const FString EnumEntryName = InEnum->GetMetaData(TEXT("ScriptName"), InEntryIndex);

		// This may be a semi-colon separated list - everything but the first item is deprecated
		if (!EnumEntryName.IsEmpty())
		{
			EnumEntryName.ParseIntoArray(EnumEntryNames, TEXT(";"), false);

			// Remove the non-deprecated entry
			if (EnumEntryNames.Num() > 0)
			{
				EnumEntryNames.RemoveAt(0, EAllowShrinking::No);
			}

			// Trim whitespace and remove empty items
			EnumEntryNames.RemoveAll([](FString& InStr)
			{
				InStr.TrimStartAndEndInline();
				return InStr.IsEmpty();
			});

			for (FString& EnumEntryNamePart : EnumEntryNames)
			{
				FText ValidationError;
				if (!IsValidName(EnumEntryNamePart, &ValidationError))
				{
					REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("'%s' has an invalid 'ScriptName' meta-data value '%s' (from '%s'): %s."), *InEnum->GetPathName(), *EnumEntryNamePart, *EnumEntryName, *ValidationError.ToString());
				}

				EnumEntryNamePart = PythonizeName(EnumEntryNamePart, EPythonizeNameCase::Upper);
			}
		}
	}

	// TODO: Support reading deprecated names from CoreRedirects?

	return EnumEntryNames;
}

FString GetDelegatePythonName(const UFunction* InDelegateSignature)
{
	return InDelegateSignature->GetName().LeftChop(19); // Trim the "__DelegateSignature" suffix from the name
}

FString GetFunctionPythonName(const UFunction* InFunc)
{
	FString FuncName = GetFieldPythonNameImpl(InFunc, ScriptNameMetaDataKey);
	return PythonizeName(FuncName, EPythonizeNameCase::Lower);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedFunctionPythonNames(const UFunction* InFunc)
{
	const UClass* FuncOwner = InFunc->GetOwnerClass();
	check(FuncOwner);

	TArray<TTuple<FSoftObjectPath, FString>> FuncNames = GetDeprecatedFieldPythonNamesImpl(InFunc, ScriptNameMetaDataKey);
	if (FuncNames.Num() > 0)
	{
		const FString FuncPythonName = GetFunctionPythonName(InFunc);
		for (auto FuncNamesIt = FuncNames.CreateIterator(); FuncNamesIt; ++FuncNamesIt)
		{
			FString& FuncName = FuncNamesIt->Value;

			// Remove any deprecated names that clash with an existing Python exposed function, 
			// unless that function is ourself, and the Pythonized name is different
			// This can happen when fixing case, eg LODs -> Lods producing lods rather than lo_ds
			const UFunction* DeprecatedFunc = FuncOwner->FindFunctionByName(*FuncName);
			FuncName = PythonizeName(FuncName, EPythonizeNameCase::Lower);
			if (DeprecatedFunc && ShouldExportFunction(DeprecatedFunc) && (DeprecatedFunc != InFunc || FuncName.Equals(FuncPythonName)))
			{
				FuncNamesIt.RemoveCurrent();
				continue;
			}
		}
	}

	return FuncNames;
}

FString GetScriptMethodPythonName(const UFunction* InFunc)
{
	FString ScriptMethodName;
	if (GetFieldPythonNameFromMetaDataImpl(InFunc, ScriptMethodMetaDataKey, ScriptMethodName))
	{
		return PythonizeName(ScriptMethodName, EPythonizeNameCase::Lower);
	}
	return GetFunctionPythonName(InFunc);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedScriptMethodPythonNames(const UFunction* InFunc)
{
	TArray<TTuple<FSoftObjectPath, FString>> ScriptMethodNames;
	if (GetDeprecatedFieldPythonNamesFromMetaDataImpl(InFunc, ScriptMethodMetaDataKey, ScriptMethodNames))
	{
		for (TTuple<FSoftObjectPath, FString>& ScriptMethodNamePair : ScriptMethodNames)
		{
			FString& ScriptMethodName = ScriptMethodNamePair.Value;
			ScriptMethodName = PythonizeName(ScriptMethodName, EPythonizeNameCase::Lower);
		}
		return ScriptMethodNames;
	}
	return GetDeprecatedFunctionPythonNames(InFunc);
}

FString GetScriptConstantPythonName(const UFunction* InFunc)
{
	FString ScriptConstantName;
	if (!GetFieldPythonNameFromMetaDataImpl(InFunc, ScriptConstantMetaDataKey, ScriptConstantName))
	{
		ScriptConstantName = GetFieldPythonNameImpl(InFunc, ScriptNameMetaDataKey);
	}
	return PythonizeName(ScriptConstantName, EPythonizeNameCase::Upper);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedScriptConstantPythonNames(const UFunction* InFunc)
{
	TArray<TTuple<FSoftObjectPath, FString>> ScriptConstantNames;
	if (!GetDeprecatedFieldPythonNamesFromMetaDataImpl(InFunc, ScriptConstantMetaDataKey, ScriptConstantNames))
	{
		ScriptConstantNames = GetDeprecatedFieldPythonNamesImpl(InFunc, ScriptNameMetaDataKey);
	}
	for (TTuple<FSoftObjectPath, FString>& ScriptConstantNamePair : ScriptConstantNames)
	{
		FString& ScriptConstantName = ScriptConstantNamePair.Value;
		ScriptConstantName = PythonizeName(ScriptConstantName, EPythonizeNameCase::Upper);
	}
	return ScriptConstantNames;
}

FString GetPropertyPythonName(const FProperty* InProp)
{
	FString PropName = GetFieldPythonNameImpl(InProp, ScriptNameMetaDataKey);
	return PythonizePropertyName(PropName, EPythonizeNameCase::Lower);
}

TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedPropertyPythonNames(const FProperty* InProp)
{
	const UStruct* PropOwner = InProp->GetOwnerStruct();
	check(PropOwner);

	TArray<TTuple<FSoftObjectPath, FString>> PropNames = GetDeprecatedFieldPythonNamesImpl(InProp, ScriptNameMetaDataKey);
	if (PropNames.Num() > 0)
	{
		const FString PropPythonName = GetPropertyPythonName(InProp);
		for (auto PropNamesIt = PropNames.CreateIterator(); PropNamesIt; ++PropNamesIt)
		{
			FString& PropName = PropNamesIt->Value;

			// Remove any deprecated names that clash with an existing Python exposed property, 
			// unless that property is ourself, and the Pythonized name is different
			// This can happen when fixing case, eg LODs -> Lods producing lods rather than lo_ds
			const FProperty* DeprecatedProp = PropOwner->FindPropertyByName(*PropName);
			PropName = PythonizeName(PropName, EPythonizeNameCase::Lower);
			if (DeprecatedProp && ShouldExportProperty(DeprecatedProp) && (DeprecatedProp != InProp || PropName.Equals(PropPythonName)))
			{
				PropNamesIt.RemoveCurrent();
				continue;
			}
		}
	}

	return PropNames;
}

FString GetPropertyTypePythonName(const FProperty* InProp, const bool InIncludeUnrealNamespace, const bool InIsForDocString, EPythonizeFlags InPythonizeFlags)
{
	const TCHAR* UnrealNamespace = InIncludeUnrealNamespace ? TEXT("unreal.") : TEXT("");

	auto GetTypeWithCoercionHint = [InPythonizeFlags, UnrealNamespace](const TCHAR* TypeName) -> FString
	{
		if (EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeCoercion))
		{
			// Supports using a python string in place of a unreal.Name() or unreal.Text().
			return FString::Printf(TEXT("Union[%s%s, str]"), UnrealNamespace, TypeName);
		}
		return FString::Printf(TEXT("%s%s"), UnrealNamespace, TypeName);
	};

#define GET_PROPERTY_TYPE(TYPE, VALUE)				\
		if (CastField<const TYPE>(InProp) != nullptr)	\
		{											\
			return (VALUE);							\
		}

	GET_PROPERTY_TYPE(FBoolProperty, TEXT("bool"))
	GET_PROPERTY_TYPE(FInt8Property, InIsForDocString ? TEXT("int8") : TEXT("int"))
	GET_PROPERTY_TYPE(FInt16Property, InIsForDocString ? TEXT("int16") : TEXT("int"))
	GET_PROPERTY_TYPE(FUInt16Property, InIsForDocString ? TEXT("uint16") : TEXT("int"))
	GET_PROPERTY_TYPE(FIntProperty, InIsForDocString ? TEXT("int32") : TEXT("int"))
	GET_PROPERTY_TYPE(FUInt32Property, InIsForDocString ? TEXT("uint32") : TEXT("int"))
	GET_PROPERTY_TYPE(FInt64Property, InIsForDocString ? TEXT("int64") : TEXT("int"))
	GET_PROPERTY_TYPE(FUInt64Property, InIsForDocString ? TEXT("uint64") : TEXT("int"))
	GET_PROPERTY_TYPE(FFloatProperty, TEXT("float"))
	GET_PROPERTY_TYPE(FDoubleProperty, InIsForDocString ? TEXT("double") : TEXT("float"))
	GET_PROPERTY_TYPE(FStrProperty, TEXT("str"))
	GET_PROPERTY_TYPE(FNameProperty, GetTypeWithCoercionHint(TEXT("Name")))
	GET_PROPERTY_TYPE(FTextProperty, GetTypeWithCoercionHint(TEXT("Text")))
	GET_PROPERTY_TYPE(FFieldPathProperty, InIncludeUnrealNamespace ? TEXT("unreal.FieldPath") : TEXT("FieldPath"))

	if (const FByteProperty* ByteProp = CastField<const FByteProperty>(InProp))
	{
		if (ByteProp->Enum)
		{
			return FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetEnumPythonName(ByteProp->Enum));
		}
		else
		{
			return InIsForDocString ? TEXT("uint8") : TEXT("int");
		}
	}
	if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProp))
	{
		return FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetEnumPythonName(EnumProp->GetEnum()));
	}
	if (const FClassProperty* ClassProp = CastField<const FClassProperty>(InProp))
	{
		if (InIsForDocString)
		{
			return FString::Printf(TEXT("type(%s%s)"), UnrealNamespace, *GetClassPythonName(ClassProp->PropertyClass));
		}
		else if (EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeCoercion))
		{
			// If type hinting is enabled and the context allows type coercion (for an input parameters), the implementation support taking a 'type' in place of a 'Class'.
			return FString::Printf(TEXT("Union[%sClass, type]"), UnrealNamespace);
		}
	}
	if (const FObjectPropertyBase* ObjProp = CastField<const FObjectPropertyBase>(InProp))
	{
		return (EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithOptionalType))
			? FString::Printf(TEXT("Optional[%s%s]"), UnrealNamespace, *GetClassPythonName(ObjProp->PropertyClass)) // None can be assigned to object.
			: FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetClassPythonName(ObjProp->PropertyClass));
	}
	if (const FInterfaceProperty* InterfaceProp = CastField<const FInterfaceProperty>(InProp))
	{
		return (EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithOptionalType))
			? FString::Printf(TEXT("Optional[%s%s]"), UnrealNamespace, *GetClassPythonName(InterfaceProp->InterfaceClass)) // None can be assigned to Interface.
			: FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetClassPythonName(InterfaceProp->InterfaceClass));
	}
	if (const FOptionalProperty* OptionalProperty = CastField<const FOptionalProperty>(InProp))
	{
		return FString::Printf(TEXT("Optional[%s]"), *GetPropertyTypePythonName(OptionalProperty->GetValueProperty(), InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags));
	}
	if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProp))
	{
		// If type hinting is enabled and the context allows type coercion (for an input parameters), the implementation support several types in place of the struct.
		return (EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeCoercion))
			? FString::Printf(TEXT("Union[%s%s, Iterable[Any], Mapping[str, Any]]"), UnrealNamespace, *GetStructPythonName(StructProp->Struct))
			: FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetStructPythonName(StructProp->Struct));
	}
	if (const FDelegateProperty* DelegateProp = CastField<const FDelegateProperty>(InProp))
	{
		return FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetDelegatePythonName(DelegateProp->SignatureFunction));
	}
	if (const FMulticastDelegateProperty* MulticastDelegateProp = CastField<const FMulticastDelegateProperty>(InProp))
	{
		return FString::Printf(TEXT("%s%s"), UnrealNamespace, *GetDelegatePythonName(MulticastDelegateProp->SignatureFunction));
	}
	if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(InProp))
	{
		// If type hinting is enabled and the context allows type coercion (for an input parameters), the implementation support any Iterable[T] in place of an Array[T]
		return EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeCoercion)
			? FString::Printf(TEXT("Iterable[%s]"), *GetPropertyTypePythonName(ArrayProperty->Inner, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags))
			: FString::Printf(TEXT("Array[%s]"), *GetPropertyTypePythonName(ArrayProperty->Inner, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags));

	}
	if (const FSetProperty* SetProperty = CastField<const FSetProperty>(InProp))
	{
		// If type hinting is enabled and the context allows type coercion (for an input parameters), the implementation support any Iterable[T] in place of a Set[T]
		return EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeCoercion)
			? FString::Printf(TEXT("Iterable[%s]"), *GetPropertyTypePythonName(SetProperty->ElementProp, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags))
			: FString::Printf(TEXT("%sSet[%s]"), UnrealNamespace, *GetPropertyTypePythonName(SetProperty->ElementProp, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags));
	}
	if (const FMapProperty* MapProperty = CastField<const FMapProperty>(InProp))
	{
		// If type hinting is enabled and the context allows type coercion (for an input parameters).
		if (EnumHasAnyFlags(InPythonizeFlags, EPythonizeFlags::WithTypeCoercion))
		{
			// When an unreal.Map() is expected, the implementation will successfully parse Mapping, list of 2-element tuples or even list of list.
			FString KeyTypeDecl   = GetPropertyTypePythonName(MapProperty->KeyProp, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags);
			FString ValueTypeDecl = GetPropertyTypePythonName(MapProperty->ValueProp, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags);
			return FString::Printf(TEXT("Union[Mapping[%s, %s], Iterable[Tuple[%s, %s]], List[List[Union[%s, %s]]]]"), *KeyTypeDecl, *ValueTypeDecl, *KeyTypeDecl, *ValueTypeDecl, *KeyTypeDecl, *ValueTypeDecl);
		}
		else
		{
			return FString::Printf(TEXT("%sMap[%s, %s]"), UnrealNamespace,
				*GetPropertyTypePythonName(MapProperty->KeyProp, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags),
				*GetPropertyTypePythonName(MapProperty->ValueProp, InIncludeUnrealNamespace, InIsForDocString, InPythonizeFlags));
		}
	}

	return InIsForDocString ? TEXT("'undefined'") : TEXT("type");

#undef GET_PROPERTY_TYPE
}

FString GetPropertyPythonType(const FProperty* InProp)
{
	FString RetStr;
	AppendPropertyPythonType(InProp, RetStr);
	return RetStr;
}

void AppendPropertyPythonType(const FProperty* InProp, FString& OutStr)
{
	OutStr += GetPropertyTypePythonName(InProp);
}

void AppendPropertyPythonReadWriteState(const FProperty* InProp, FString& OutStr, const uint64 InReadOnlyFlags)
{
	OutStr += (InProp->HasAnyPropertyFlags(InReadOnlyFlags) ? TEXT(" [Read-Only]") : TEXT(" [Read-Write]"));
}

template <typename FieldType>
FString GetFieldTooltipImpl(const FieldType* InField)
{
	// We use the source string here as the culture may change while the editor is running, and also because some versions 
	// of Python (<3.4) can't override the default encoding to UTF-8 so produce errors when trying to print the help docs
	return *FTextInspector::GetSourceString(InField->GetToolTipText());
}

FString GetFieldTooltip(const UField* InField)
{
	return GetFieldTooltipImpl(InField);
}

FString GetFieldTooltip(const FField* InField)
{
	return GetFieldTooltipImpl(InField);
}

FString GetEnumEntryTooltip(const UEnum* InEnum, const int64 InEntryIndex)
{
	// We use the source string here as the culture may change while the editor is running, and also because some versions 
	// of Python (<3.4) can't override the default encoding to UTF-8 so produce errors when trying to print the help docs
	return *FTextInspector::GetSourceString(InEnum->GetToolTipTextByIndex(InEntryIndex));
}

FString BuildSourceInformationDocString(const UField* InOwnerType)
{
	FString Str;
	AppendSourceInformationDocString(InOwnerType, Str);
	return Str;
}

void AppendSourceInformationDocString(const UField* InOwnerType, FString& OutStr)
{
	if (!InOwnerType)
	{
		return;
	}

	if (!OutStr.IsEmpty())
	{
		if (OutStr[OutStr.Len() - 1] != TEXT('\n'))
		{
			OutStr += LINE_TERMINATOR;
		}
	}

	const UPackage* OwnerPackage = InOwnerType->GetOutermost();
	const FString OwnerPackageName = OwnerPackage->GetName();

	if (FPackageName::IsScriptPackage(OwnerPackageName))
	{
		static const FName ModuleRelativePathMetaDataKey = "ModuleRelativePath";

		const FString TypePlugin = GetFieldPlugin(InOwnerType);
		const FString TypeModule = GetFieldModule(InOwnerType);
		const FString TypeFile = FPaths::GetCleanFilename(InOwnerType->GetMetaData(ModuleRelativePathMetaDataKey));

		OutStr += LINE_TERMINATOR TEXT("**C++ Source:**") LINE_TERMINATOR;
		if (!TypePlugin.IsEmpty())
		{
			OutStr += LINE_TERMINATOR TEXT("- **Plugin**: ");
			OutStr += TypePlugin;
		}
		OutStr += LINE_TERMINATOR TEXT("- **Module**: ");
		OutStr += TypeModule;
		OutStr += LINE_TERMINATOR TEXT("- **File**: ");
		OutStr += TypeFile;
		OutStr += LINE_TERMINATOR;
	}
	else
	{
		OutStr += LINE_TERMINATOR TEXT("**Asset Source:**") LINE_TERMINATOR;
		OutStr += LINE_TERMINATOR TEXT("- **Package**: ");
		OutStr += OwnerPackageName;
	}
}

bool SaveGeneratedTextFile(const TCHAR* InFilename, FStringView InFileContents, const bool InForceWrite)
{
	bool bWriteFile = InForceWrite;

	if (!bWriteFile)
	{
		FString CurrentFileContents;
		if (FFileHelper::LoadFileToString(CurrentFileContents, InFilename))
		{
			// Only write the file if the contents differ
			bWriteFile = !InFileContents.Equals(CurrentFileContents, ESearchCase::CaseSensitive);
		}
		else
		{
			// Failed to load the file, assume it's missing so write it
			bWriteFile = true;
		}
	}

	if (bWriteFile)
	{
		return FFileHelper::SaveStringToFile(InFileContents, InFilename, FFileHelper::EEncodingOptions::ForceUTF8);
	}

	// File up-to-date, return success
	return true;
}

void SetTypeHintingMode(ETypeHintingMode InMode)
{
	GTypeHintingMode = InMode;
}

ETypeHintingMode GetTypeHintingMode()
{
	if (!IsTypeHintingEnabled()) // Can be disable if the Python version is too old.
	{
		return ETypeHintingMode::Off;
	}
	return GTypeHintingMode;
}

bool IsTypeHintingEnabled()
{
	// In 3.7, a feature allowing usage of non-declared types in type hints was added (from __future__ import annotations) and the engine
	// doesn't output types in a strict declaration/usage order, so this is required to correctly type.
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 7
	return GTypeHintingMode != ETypeHintingMode::Off;
#else
	return false;
#endif
}

}

#endif	// WITH_PYTHON
