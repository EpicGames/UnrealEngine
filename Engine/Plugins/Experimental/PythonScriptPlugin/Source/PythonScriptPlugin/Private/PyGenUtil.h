// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"
#include "PyConstant.h"
#include "PyMethodWithClosure.h"
#include "PythonScriptPluginSettings.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/PropertyAccessUtil.h"
#include "UObject/WeakFieldPtr.h"
#include "Async/Mutex.h"

#if WITH_PYTHON

struct FOutParmRec;
struct FPyWrapperBaseMetaData;

struct FReportPythonGenerationIssue_Private
{
	static FORCEINLINE void ConditionalBreak(const ELogVerbosity::Type InVerbosity)
	{
		if (InVerbosity <= ELogVerbosity::Error && FPlatformMisc::IsDebuggerPresent())
		{
			UE_DEBUG_BREAK();
		}
	}
};

#define REPORT_PYTHON_GENERATION_ISSUE(Verbosity, Format, ...)	\
	UE_LOG(LogPython, Verbosity, Format, ##__VA_ARGS__);		\
	FReportPythonGenerationIssue_Private::ConditionalBreak(ELogVerbosity::Verbosity);

namespace PyGenUtil
{
	extern const FName ScriptNameMetaDataKey;
	extern const FName ScriptNoExportMetaDataKey;
	extern const FName ScriptMethodMetaDataKey;
	extern const FName ScriptMethodSelfReturnMetaDataKey;
	extern const FName ScriptMethodMutableMetaDataKey;
	extern const FName ScriptOperatorMetaDataKey;
	extern const FName ScriptConstantMetaDataKey;
	extern const FName ScriptConstantHostMetaDataKey;
	extern const FName ScriptDefaultMakeMetaDataKey;
	extern const FName ScriptDefaultBreakMetaDataKey;
	extern const FName BlueprintTypeMetaDataKey;
	extern const FName NotBlueprintTypeMetaDataKey;
	extern const FName BlueprintSpawnableComponentMetaDataKey;
	extern const FName BlueprintGetterMetaDataKey;
	extern const FName BlueprintSetterMetaDataKey;
	extern const FName BlueprintInternalUseOnlyMetaDataKey;
	extern const FName BlueprintInternalUseOnlyHierarchicalMetaDataKey;
	extern const FName CustomThunkMetaDataKey;
	extern const FName HasNativeMakeMetaDataKey;
	extern const FName HasNativeBreakMetaDataKey;
	extern const FName NativeBreakFuncMetaDataKey;
	extern const FName NativeMakeFuncMetaDataKey;
	extern const FName DeprecatedPropertyMetaDataKey;
	extern const FName DeprecatedFunctionMetaDataKey;
	extern const FName DeprecationMessageMetaDataKey;

	/** Name used by the Python equivalent of PostInitProperties */
	static const char* PostInitFuncName = "_post_init";

	/** Buffer for storing the UTF-8 strings used by Python types */
	typedef TArray<char> FUTF8Buffer;

	/** Case sensitive hashing function for TMap */
	template <typename ValueType>
	struct FCaseSensitiveStringMapFuncs : BaseKeyFuncs<ValueType, FString, /*bInAllowDuplicateKeys*/false>
	{
		static FORCEINLINE const FString& GetSetKey(const TPair<FString, ValueType>& Element)
		{
			return Element.Key;
		}
		static FORCEINLINE bool Matches(const FString& A, const FString& B)
		{
			return A.Equals(B, ESearchCase::CaseSensitive);
		}
		static FORCEINLINE uint32 GetKeyHash(const FString& Key)
		{
			return FCrc::StrCrc32<TCHAR>(*Key);
		}
	};

	/** Context used during multithreaded wrapper generation. */
	struct FMultithreadedGenerationContext
	{
		FMultithreadedGenerationContext();
		~FMultithreadedGenerationContext();

		static bool IsMultithreadedGenerationActive();
		static void EnqueueGameThreadTask(TFunction<void()>&& Task);
		static void ExecuteGameThreadTask(TFunctionRef<void()> Task);
		
		bool ExecuteGameThreadTasks();
		void ExecuteGameThreadTasksAndWait();

	private:
		static bool& GetMultithreadedGenerationRef();
	};

	/** Stores the information about a native Python module */
	struct FNativePythonModule
	{
		FNativePythonModule()
			: PyModule(nullptr)
			, PyModuleMethods(nullptr)
			, PyModuleTypes()
		{
		}

		FNativePythonModule(FNativePythonModule&&) = default;
		FNativePythonModule(const FNativePythonModule&) = default;
		FNativePythonModule& operator=(FNativePythonModule&&) = default;
		FNativePythonModule& operator=(const FNativePythonModule&) = default;

		/** Add the given type to this module (the type must have had PyType_Ready called on it) */
		void AddType(PyTypeObject* InType);

		/** Pointer to the native Python module */
		PyObject* PyModule;

		/** Pointer to a null-terminated array of methods exposed in this module (if any) */
		PyMethodDef* PyModuleMethods;

		/** Array of types exposed in this module (if any) */
		TArray<PyTypeObject*> PyModuleTypes;
	};

	/** Stores the data needed by a runtime generated Python parameter */
	struct FGeneratedWrappedMethodParameter
	{
		FGeneratedWrappedMethodParameter()
			: ParamProp(nullptr)
		{
		}

		FGeneratedWrappedMethodParameter(FGeneratedWrappedMethodParameter&&) = default;
		FGeneratedWrappedMethodParameter(const FGeneratedWrappedMethodParameter&) = default;
		FGeneratedWrappedMethodParameter& operator=(FGeneratedWrappedMethodParameter&&) = default;
		FGeneratedWrappedMethodParameter& operator=(const FGeneratedWrappedMethodParameter&) = default;

		/** The name of the parameter */
		FUTF8Buffer ParamName;

		/** The Unreal property for this parameter */
		const FProperty* ParamProp;

		/** The default Unreal ExportText value of this parameter; parameters with this set are considered optional */
		TOptional<FString> ParamDefaultValue;
	};

	/** Stores the data needed to call an Unreal function via Python */
	struct FGeneratedWrappedFunction
	{
		enum ESetFunctionFlags
		{
			SFF_None = 0,
			SFF_CalculateDeprecationState = 1<<0,
			SFF_ExtractParameters = 1<<1,
			SFF_All = SFF_CalculateDeprecationState | SFF_ExtractParameters,
		};

		FGeneratedWrappedFunction()
			: Func(nullptr)
		{
		}

		FGeneratedWrappedFunction(FGeneratedWrappedFunction&&) = default;
		FGeneratedWrappedFunction(const FGeneratedWrappedFunction&) = default;
		FGeneratedWrappedFunction& operator=(FGeneratedWrappedFunction&&) = default;
		FGeneratedWrappedFunction& operator=(const FGeneratedWrappedFunction&) = default;

		/** Set the function, optionally also calculating some data about it */
		void SetFunction(const UFunction* InFunc, const uint32 InSetFuncFlags = SFF_All);

		/** The Unreal function to call (static dispatch) */
		TObjectPtr<const UFunction> Func;

		/** Array of input parameters associated with the function */
		TArray<FGeneratedWrappedMethodParameter> InputParams;

		/** Array of output (including return) parameters associated with the function */
		TArray<FGeneratedWrappedMethodParameter> OutputParams;

		/** Set if this function is deprecated and using it should emit a deprecation warning */
		TOptional<FString> DeprecationMessage;
	};

	/** Stores the data needed by a runtime generated Python method */
	struct FGeneratedWrappedMethod
	{
		FGeneratedWrappedMethod()
			: MethodCallback(nullptr)
			, MethodFlags(0)
		{
		}

		FGeneratedWrappedMethod(FGeneratedWrappedMethod&&) = default;
		FGeneratedWrappedMethod(const FGeneratedWrappedMethod&) = default;
		FGeneratedWrappedMethod& operator=(FGeneratedWrappedMethod&&) = default;
		FGeneratedWrappedMethod& operator=(const FGeneratedWrappedMethod&) = default;

		/** Convert this wrapper type to its Python type */
		void ToPython(FPyMethodWithClosureDef& OutPyMethod) const;

		/** The name of the method */
		FUTF8Buffer MethodName;

		/** The doc string of the method */
		FUTF8Buffer MethodDoc;

		/** The Unreal function for this method */
		FGeneratedWrappedFunction MethodFunc;

		/* The C function this method should call */
		PyCFunctionWithClosure MethodCallback;

		/* The METH_ flags for this method */
		int MethodFlags;
	};

	/** Stores the data needed for a set of runtime generated Python methods */
	struct FGeneratedWrappedMethods
	{
		FGeneratedWrappedMethods() = default;
		FGeneratedWrappedMethods(FGeneratedWrappedMethods&&) = default;
		FGeneratedWrappedMethods(const FGeneratedWrappedMethods&) = delete;
		FGeneratedWrappedMethods& operator=(FGeneratedWrappedMethods&&) = default;
		FGeneratedWrappedMethods& operator=(const FGeneratedWrappedMethods&) = delete;

		/** Called to ready the generated methods with Python */
		void Finalize();

		/** Array of methods associated from the wrapped type */
		TArray<FGeneratedWrappedMethod> TypeMethods;

		/** The Python methods that were generated from TypeMethods (in Finalize) */
		TArray<FPyMethodWithClosureDef> PyMethods;
	};

	/** Stores the data needed by a runtime generated Python method that is dynamically created and registered post-finalize of its owner type (for hoisting util functions onto structs) */
	struct FGeneratedWrappedDynamicMethod : public FGeneratedWrappedMethod
	{
		FGeneratedWrappedDynamicMethod() = default;
		FGeneratedWrappedDynamicMethod(FGeneratedWrappedDynamicMethod&&) = default;
		FGeneratedWrappedDynamicMethod(const FGeneratedWrappedDynamicMethod&) = default;
		FGeneratedWrappedDynamicMethod& operator=(FGeneratedWrappedDynamicMethod&&) = default;
		FGeneratedWrappedDynamicMethod& operator=(const FGeneratedWrappedDynamicMethod&) = default;

		/** The 'self' parameter information (this parameter is set to the instance that calls the method) */
		FGeneratedWrappedMethodParameter SelfParam;

		/** The 'self' return information (this value overwrites the instance that calls the method) */
		FGeneratedWrappedMethodParameter SelfReturn;
	};

	/** Stores the data needed by a runtime generated Python method that is dynamically created and registered post-finalize of its owner struct type (for hoisting util functions onto other types) */
	struct FGeneratedWrappedDynamicMethodWithClosure : public FGeneratedWrappedDynamicMethod
	{
		FGeneratedWrappedDynamicMethodWithClosure() = default;
		FGeneratedWrappedDynamicMethodWithClosure(FGeneratedWrappedDynamicMethodWithClosure&&) = default;
		FGeneratedWrappedDynamicMethodWithClosure(const FGeneratedWrappedDynamicMethodWithClosure&) = delete;
		FGeneratedWrappedDynamicMethodWithClosure& operator=(FGeneratedWrappedDynamicMethodWithClosure&&) = default;
		FGeneratedWrappedDynamicMethodWithClosure& operator=(const FGeneratedWrappedDynamicMethodWithClosure&) = delete;

		/** Called to ready the generated method for Python */
		void Finalize();

		/** Python method that was generated from this method */
		FPyMethodWithClosureDef PyMethod;
	};

	/** Mixin used to add dynamic method support to a type (base for common non-templated functionality) */
	struct FGeneratedWrappedDynamicMethodsMixinBase
	{
		FGeneratedWrappedDynamicMethodsMixinBase() = default;
		FGeneratedWrappedDynamicMethodsMixinBase(FGeneratedWrappedDynamicMethodsMixinBase&&) = default;
		FGeneratedWrappedDynamicMethodsMixinBase(const FGeneratedWrappedDynamicMethodsMixinBase&) = delete;
		FGeneratedWrappedDynamicMethodsMixinBase& operator=(FGeneratedWrappedDynamicMethodsMixinBase&&) = default;
		FGeneratedWrappedDynamicMethodsMixinBase& operator=(const FGeneratedWrappedDynamicMethodsMixinBase&) = delete;

		/** Called to add a dynamic method to the Python type (this should only be called post-finalize) */
		void AddDynamicMethodImpl(FGeneratedWrappedDynamicMethod&& InDynamicMethod, PyTypeObject* InPyType);

		/** Array of dynamic methods associated with this type (call AddDynamicMethod to add methods) */
		TArray<TSharedRef<FGeneratedWrappedDynamicMethodWithClosure>> DynamicMethods;
	};

	/** Mixin used to add dynamic method support to a type */
	template <typename MixedIntoType>
	struct TGeneratedWrappedDynamicMethodsMixin : public FGeneratedWrappedDynamicMethodsMixinBase
	{
		TGeneratedWrappedDynamicMethodsMixin() = default;
		TGeneratedWrappedDynamicMethodsMixin(TGeneratedWrappedDynamicMethodsMixin&&) = default;
		TGeneratedWrappedDynamicMethodsMixin(const TGeneratedWrappedDynamicMethodsMixin&) = delete;
		TGeneratedWrappedDynamicMethodsMixin& operator=(TGeneratedWrappedDynamicMethodsMixin&&) = default;
		TGeneratedWrappedDynamicMethodsMixin& operator=(const TGeneratedWrappedDynamicMethodsMixin&) = delete;

		/** Called to add a dynamic method to the Python type (this should only be called post-finalize) */
		void AddDynamicMethod(FGeneratedWrappedDynamicMethod&& InDynamicMethod)
		{
			AddDynamicMethodImpl(MoveTemp(InDynamicMethod), &static_cast<MixedIntoType*>(this)->PyType);
		}
	};

	/** Operator types supported by Python */
	enum class EGeneratedWrappedOperatorType : uint8
	{
		Bool = 0,			// bool
		Equal,				// ==
		NotEqual,			// !=
		Less,				// <
		LessEqual,			// <=
		Greater,			// >
		GreaterEqual,		// >=
		Add,				// +
		InlineAdd,			// +=
		Subtract,			// -
		InlineSubtract,		// -=
		Multiply,			// *
		InlineMultiply,		// *=
		Divide,				// /
		InlineDivide,		// /=
		Modulus,			// %
		InlineModulus,		// %=
		And,				// &
		InlineAnd,			// &=
		Or,					// |
		InlineOr,			// |=
		Xor,				// ^
		InlineXor,			// ^=
		RightShift,			// >>
		InlineRightShift,	// >>=
		LeftShift,			// <<
		InlineLeftShift,	// <<=
		Negated,			// -obj prefix operator
		Num,
	};

	/** Stores the signature definition of a runtime generated Python operator stack function */
	struct FGeneratedWrappedOperatorSignature
	{
		enum class EType
		{
			/** This operator type should be void */
			None,
			/** This operator type can be anything */
			Any,
			/** This operator type should be a struct (of the same type as 'self') */
			Struct,
			/** This operator type should be a bool */
			Bool
		};

		FGeneratedWrappedOperatorSignature()
			: OpType(EGeneratedWrappedOperatorType::Num)
			, OpTypeStr(TEXT(""))
			, PyFuncName(TEXT(""))
			, ReturnType(EType::None)
			, OtherType(EType::None)
		{
		}

		FGeneratedWrappedOperatorSignature(const EGeneratedWrappedOperatorType InOpType, const TCHAR* InOpTypeStr, const TCHAR* InPyFuncName, const EType InReturnType, const EType InOtherType)
			: OpType(InOpType)
			, OpTypeStr(InOpTypeStr)
			, PyFuncName(InPyFuncName)
			, ReturnType(InReturnType)
			, OtherType(InOtherType)
		{
		}

		/**
		 * Given an operator type, return its signature
		 */
		static const FGeneratedWrappedOperatorSignature& OpTypeToSignature(const EGeneratedWrappedOperatorType InOpType);

		/**
		 * Given a potential operator string, try and convert it to a known operator signature
		 * @return true if the conversion was a success, false otherwise
		 */
		static bool StringToSignature(const TCHAR* InStr, FGeneratedWrappedOperatorSignature& OutSignature);

		/**
		 * Validate that the given parameter is compatible with the expected type.
		 */
		static bool ValidateParam(const FGeneratedWrappedMethodParameter& InParam, const EType InType, const UScriptStruct* InStructType, FString* OutError = nullptr);

		/**
		 * Get the number of input parameter that are expected by this operator.
		 */
		int32 GetInputParamCount() const;

		/**
		 * Get the number of output parameter that are expected by this operator.
		 */
		int32 GetOutputParamCount() const;

		/** The operator enum value */
		EGeneratedWrappedOperatorType OpType;

		/** The string representation of this operator (see the comments on EGeneratedWrappedOperatorType) */
		const TCHAR* OpTypeStr;

		/** The name of the function used in Python for this operator) */
		const TCHAR* PyFuncName;

		/** The type that is required by the return value of this operator (a struct return implies a 'self' return) */
		EType ReturnType;

		/** The type that is required by the other value of this operator (for binary operators) */
		EType OtherType;
	};

	/** Stores the data needed by a runtime generated Python operator stack function */
	struct FGeneratedWrappedOperatorFunction
	{
		FGeneratedWrappedOperatorFunction() = default;
		FGeneratedWrappedOperatorFunction(FGeneratedWrappedOperatorFunction&&) = default;
		FGeneratedWrappedOperatorFunction(const FGeneratedWrappedOperatorFunction&) = default;
		FGeneratedWrappedOperatorFunction& operator=(FGeneratedWrappedOperatorFunction&&) = default;
		FGeneratedWrappedOperatorFunction& operator=(const FGeneratedWrappedOperatorFunction&) = default;

		/** Set the function, validating that it's compatible with the given signature */
		bool SetFunction(const UFunction* InFunc, const FGeneratedWrappedOperatorSignature& InSignature, FString* OutError = nullptr);
		bool SetFunction(const FGeneratedWrappedFunction& InFuncDef, const FGeneratedWrappedOperatorSignature& InSignature, FString* OutError = nullptr);

		/** The Unreal function to call (static dispatch) */
		const UFunction* Func;

		/** The 'self' parameter information (this parameter is set to the instance that calls the function) */
		FGeneratedWrappedMethodParameter SelfParam;

		/** The 'self' return information (this value overwrites the instance that calls the function) */
		FGeneratedWrappedMethodParameter SelfReturn;

		/** The 'other' parameter information (for binary operators) */
		FGeneratedWrappedMethodParameter OtherParam;

		/** The return parameter information */
		FGeneratedWrappedMethodParameter ReturnParam;

		/** Array of additional defaulted input parameters */
		TArray<FGeneratedWrappedMethodParameter> AdditionalParams;
	};

	/** Stores the data needed by a runtime generated Python operator stack that is dynamically created and registered post-finalize of its owner type (for hoisting operators onto other types) */
	struct FGeneratedWrappedOperatorStack
	{
		FGeneratedWrappedOperatorStack() = default;
		FGeneratedWrappedOperatorStack(FGeneratedWrappedOperatorStack&&) = default;
		FGeneratedWrappedOperatorStack(const FGeneratedWrappedOperatorStack&) = default;
		FGeneratedWrappedOperatorStack& operator=(FGeneratedWrappedOperatorStack&&) = default;
		FGeneratedWrappedOperatorStack& operator=(const FGeneratedWrappedOperatorStack&) = default;

		/** Array of operator functions associated with this operator stack */
		TArray<FGeneratedWrappedOperatorFunction> Funcs;
	};

	/** Stores the data needed to access an Unreal property via Python */
	struct FGeneratedWrappedProperty
	{
		enum ESetPropertyFlags
		{
			SPF_None = 0,
			SPF_CalculateDeprecationState = 1<<0,
			SPF_All = SPF_CalculateDeprecationState,
		};

		FGeneratedWrappedProperty()
			: Prop(nullptr)
		{
		}

		FGeneratedWrappedProperty(FGeneratedWrappedProperty&&) = default;
		FGeneratedWrappedProperty(const FGeneratedWrappedProperty&) = default;
		FGeneratedWrappedProperty& operator=(FGeneratedWrappedProperty&&) = default;
		FGeneratedWrappedProperty& operator=(const FGeneratedWrappedProperty&) = default;

		/** Set the property, optionally also calculating some data about it */
		void SetProperty(const FProperty* InProp, const uint32 InSetPropFlags = SPF_All);

		/** The Unreal property to access */
		const FProperty* Prop;

		/** Set if this property is deprecated and using it should emit a deprecation warning */
		TOptional<FString> DeprecationMessage;
	};

	/** Stores the data needed by a runtime generated Python get/set */
	struct FGeneratedWrappedGetSet
	{
		FGeneratedWrappedGetSet()
			: GetCallback(nullptr)
			, SetCallback(nullptr)
		{
		}

		FGeneratedWrappedGetSet(FGeneratedWrappedGetSet&&) = default;
		FGeneratedWrappedGetSet(const FGeneratedWrappedGetSet&) = default;
		FGeneratedWrappedGetSet& operator=(FGeneratedWrappedGetSet&&) = default;
		FGeneratedWrappedGetSet& operator=(const FGeneratedWrappedGetSet&) = default;

		/** Convert this wrapper type to its Python type */
		void ToPython(PyGetSetDef& OutPyGetSet) const;

		/** The name of the get/set */
		FUTF8Buffer GetSetName;

		/** The doc string of the get/set */
		FUTF8Buffer GetSetDoc;

		/** The Unreal property for this get/set */
		FGeneratedWrappedProperty Prop;

		/** The Unreal function for the get (if any) */
		FGeneratedWrappedFunction GetFunc;

		/** The Unreal function for the set (if any) */
		FGeneratedWrappedFunction SetFunc;

		/* The C function that should be called for get */
		getter GetCallback;

		/* The C function that should be called for set */
		setter SetCallback;
	};

	/** Stores the data needed for a set of runtime generated Python get/sets */
	struct FGeneratedWrappedGetSets
	{
		FGeneratedWrappedGetSets() = default;
		FGeneratedWrappedGetSets(FGeneratedWrappedGetSets&&) = default;
		FGeneratedWrappedGetSets(const FGeneratedWrappedGetSets&) = delete;
		FGeneratedWrappedGetSets& operator=(FGeneratedWrappedGetSets&&) = default;
		FGeneratedWrappedGetSets& operator=(const FGeneratedWrappedGetSets&) = delete;

		/** Called to ready the generated get/sets with Python */
		void Finalize();

		/** Array of get/sets from the wrapped type */
		TArray<FGeneratedWrappedGetSet> TypeGetSets;

		/** The Python get/sets that were generated from TypeGetSets (in Finalize) */
		TArray<PyGetSetDef> PyGetSets;
	};

	/** Stores the data needed by a runtime generated Python constant */
	struct FGeneratedWrappedConstant
	{
		FGeneratedWrappedConstant() = default;
		FGeneratedWrappedConstant(FGeneratedWrappedConstant&&) = default;
		FGeneratedWrappedConstant(const FGeneratedWrappedConstant&) = default;
		FGeneratedWrappedConstant& operator=(FGeneratedWrappedConstant&&) = default;
		FGeneratedWrappedConstant& operator=(const FGeneratedWrappedConstant&) = default;

		/** Convert this wrapper type to its Python type */
		void ToPython(FPyConstantDef& OutPyConstant) const;

		/** The name of the constant */
		FUTF8Buffer ConstantName;

		/** The doc string of the constant */
		FUTF8Buffer ConstantDoc;

		/** The Unreal function for getting the constant data */
		FGeneratedWrappedFunction ConstantFunc;
	};

	/** Stores the data needed for a set of runtime generated Python constants */
	struct FGeneratedWrappedConstants
	{
		FGeneratedWrappedConstants() = default;
		FGeneratedWrappedConstants(FGeneratedWrappedConstants&&) = default;
		FGeneratedWrappedConstants(const FGeneratedWrappedConstants&) = delete;
		FGeneratedWrappedConstants& operator=(FGeneratedWrappedConstants&&) = default;
		FGeneratedWrappedConstants& operator=(const FGeneratedWrappedConstants&) = delete;

		/** Called to ready the generated constants with Python */
		void Finalize();

		/** Array of constants from the wrapped type */
		TArray<FGeneratedWrappedConstant> TypeConstants;

		/** The Python constants that were generated from TypeConstants (in Finalize) */
		TArray<FPyConstantDef> PyConstants;
	};

	/** Stores the data needed by a runtime generated Python constant that is dynamically created and registered post-finalize of its owner type (for hoisting constant functions onto other types) */
	struct FGeneratedWrappedDynamicConstantWithClosure : public FGeneratedWrappedConstant
	{
		FGeneratedWrappedDynamicConstantWithClosure() = default;
		FGeneratedWrappedDynamicConstantWithClosure(FGeneratedWrappedDynamicConstantWithClosure&&) = default;
		FGeneratedWrappedDynamicConstantWithClosure(const FGeneratedWrappedDynamicConstantWithClosure&) = delete;
		FGeneratedWrappedDynamicConstantWithClosure& operator=(FGeneratedWrappedDynamicConstantWithClosure&&) = default;
		FGeneratedWrappedDynamicConstantWithClosure& operator=(const FGeneratedWrappedDynamicConstantWithClosure&) = delete;

		/** Called to ready the generated constant for Python */
		void Finalize();

		/** Python constant that was generated from this data */
		FPyConstantDef PyConstant;
	};

	/** Mixin used to add dynamic constant support to a type (base for common non-templated functionality) */
	struct FGeneratedWrappedDynamicConstantsMixinBase
	{
		FGeneratedWrappedDynamicConstantsMixinBase() = default;
		FGeneratedWrappedDynamicConstantsMixinBase(FGeneratedWrappedDynamicConstantsMixinBase&&) = default;
		FGeneratedWrappedDynamicConstantsMixinBase(const FGeneratedWrappedDynamicConstantsMixinBase&) = delete;
		FGeneratedWrappedDynamicConstantsMixinBase& operator=(FGeneratedWrappedDynamicConstantsMixinBase&&) = default;
		FGeneratedWrappedDynamicConstantsMixinBase& operator=(const FGeneratedWrappedDynamicConstantsMixinBase&) = delete;

		/** Called to add a dynamic constant to the Python type (this should only be called post-finalize) */
		void AddDynamicConstantImpl(FGeneratedWrappedConstant&& InDynamicConstant, PyTypeObject* InPyType);

		/** Array of dynamic constants associated with this type (call AddDynamicConstant to add constants) */
		TArray<TSharedRef<FGeneratedWrappedDynamicConstantWithClosure>> DynamicConstants;
	};

	/** Mixin used to add dynamic constant support to a type */
	template <typename MixedIntoType>
	struct TGeneratedWrappedDynamicConstantsMixin : public FGeneratedWrappedDynamicConstantsMixinBase
	{
		TGeneratedWrappedDynamicConstantsMixin() = default;
		TGeneratedWrappedDynamicConstantsMixin(TGeneratedWrappedDynamicConstantsMixin&&) = default;
		TGeneratedWrappedDynamicConstantsMixin(const TGeneratedWrappedDynamicConstantsMixin&) = delete;
		TGeneratedWrappedDynamicConstantsMixin& operator=(TGeneratedWrappedDynamicConstantsMixin&&) = default;
		TGeneratedWrappedDynamicConstantsMixin& operator=(const TGeneratedWrappedDynamicConstantsMixin&) = delete;

		/** Called to add a dynamic constant to the Python type (this should only be called post-finalize) */
		void AddDynamicConstant(FGeneratedWrappedConstant&& InDynamicConstant)
		{
			AddDynamicConstantImpl(MoveTemp(InDynamicConstant), &static_cast<MixedIntoType*>(this)->PyType);
		}
	};

	/** Stores the data needed to generate a Python doc string for editor exposed properties */
	struct FGeneratedWrappedPropertyDoc
	{
		explicit FGeneratedWrappedPropertyDoc(const FProperty* InProp);

		FGeneratedWrappedPropertyDoc(FGeneratedWrappedPropertyDoc&&) = default;
		FGeneratedWrappedPropertyDoc(const FGeneratedWrappedPropertyDoc&) = default;
		FGeneratedWrappedPropertyDoc& operator=(FGeneratedWrappedPropertyDoc&&) = default;
		FGeneratedWrappedPropertyDoc& operator=(const FGeneratedWrappedPropertyDoc&) = default;

		/** Util function to sort an array of doc structs based on the Pythonized property name */
		static bool SortPredicate(const FGeneratedWrappedPropertyDoc& InOne, const FGeneratedWrappedPropertyDoc& InTwo);

		/** Util function to convert an array of doc structs into a combined doc string (the array should have been sorted before calling this) */
		static FString BuildDocString(const TArray<FGeneratedWrappedPropertyDoc>& InDocs);

		/** Util function to convert an array of doc structs into a combined doc string (the array should have been sorted before calling this) */
		static void AppendDocString(const TArray<FGeneratedWrappedPropertyDoc>& InDocs, FString& OutStr);

		/** Pythonized name of the property */
		FString PythonPropName;

		/** Pythonized doc string of the property */
		FString DocString;

		/** Pythonized editor doc string of the property */
		FString EditorDocString;
	};

	/** Utility for tracking and logging field conflicts when exported to Python */
	struct FGeneratedWrappedFieldTracker
	{
		FGeneratedWrappedFieldTracker() = default;
		FGeneratedWrappedFieldTracker(FGeneratedWrappedFieldTracker&&) = default;
		FGeneratedWrappedFieldTracker(const FGeneratedWrappedFieldTracker&) = default;
		FGeneratedWrappedFieldTracker& operator=(FGeneratedWrappedFieldTracker&&) = default;
		FGeneratedWrappedFieldTracker& operator=(const FGeneratedWrappedFieldTracker&) = default;

		/** Register a Python field name, and detect if a name conflict has occurred */
		void RegisterPythonFieldName(const FString& InPythonFieldName, const FFieldVariant& InUnrealField);

		/** Map from the Python wrapped field name to the Unreal field it was generated from (for conflict detection) */
		typedef TMap<FString, TWeakObjectPtr<const UField>, FDefaultSetAllocator, FCaseSensitiveStringMapFuncs<TWeakObjectPtr<const UField>>> FCaseSensitiveStringToFieldMap;
		typedef TMap<FString, TWeakFieldPtr<const FField>, FDefaultSetAllocator, FCaseSensitiveStringMapFuncs<TWeakFieldPtr<const FField>>> FCaseSensitiveStringToFFieldMap;
		FCaseSensitiveStringToFieldMap PythonWrappedFieldNameToUnrealField;
		FCaseSensitiveStringToFFieldMap PythonWrappedFieldNameToFField;
	};

	/** Stores the minimal data needed by a runtime generated Python type */
	struct FGeneratedWrappedType
	{
		enum class EFinalizedState : uint8
		{
			Initial,
			Reset,
			Finalized,
		};

		FGeneratedWrappedType()
		{
			PyType = { PyVarObject_HEAD_INIT(nullptr, 0) };
		}

		virtual ~FGeneratedWrappedType() = default;

		FGeneratedWrappedType(FGeneratedWrappedType&&) = delete;
		FGeneratedWrappedType(const FGeneratedWrappedType&) = delete;
		FGeneratedWrappedType& operator=(FGeneratedWrappedType&&) = delete;
		FGeneratedWrappedType& operator=(const FGeneratedWrappedType&) = delete;

		/** Called to ready the generated type with Python */
		bool Finalize();

		/** Called to reset this type back to its clean state */
		void Reset();

		/** Internal version of Finalize, called before readying the type with Python */
		virtual void Finalize_PreReady();

		/** Internal version of Finalize, called after readying the type with Python */
		virtual void Finalize_PostReady();

		/** Internal version of Reset, called to remove data added to the Python type */
		virtual void Reset_CleansePyType();

		/** Internal version of Reset, called to reset the data on this type */
		virtual void Reset_CleanseSelf();

		/** The name of the type */
		FUTF8Buffer TypeName;

		/** The doc string of the type */
		FUTF8Buffer TypeDoc;

		/** The meta-data associated with this type */
		TSharedPtr<FPyWrapperBaseMetaData> MetaData;

		/* The Python type that was generated */
		PyTypeObject PyType;

		/** What is the current finalization state of this generated type? */
		EFinalizedState FinalizedState = EFinalizedState::Initial;

		/* Per-object lock used during the construction phase */
		struct FDebuggableMutex
		{
			// We should never go above 1 lock per stack because
			// going recursive on those lock will create a very deadlock
			// prone situation when run under multiple thread that will
			// be hard to repro depending on the order in which things
			// are recursed into.
			static int32& GetThreadLocal()
			{
				static thread_local int32 StackLockCount = 0;
				return StackLockCount;
			}

			UE::FMutex Mutex;
			uint32     OwnerThreadId = 0;

			void Lock()
			{
				check(GetThreadLocal() == 0);
				Mutex.Lock();
				GetThreadLocal()++;
				check(OwnerThreadId == 0);
				OwnerThreadId = FPlatformTLS::GetCurrentThreadId();
			}

			void Unlock()
			{
				OwnerThreadId = 0;
				check(GetThreadLocal() == 1);
				GetThreadLocal()--;
				Mutex.Unlock();
			}
		} Lock;

		/* Used to atomically publish readyness of this object */
		std::atomic<bool> bIsReadyForUse;
	};

	/** Stores the data needed by a runtime generated Python struct type */
	struct FGeneratedWrappedStructType : public FGeneratedWrappedType, public TGeneratedWrappedDynamicMethodsMixin<FGeneratedWrappedStructType>, public TGeneratedWrappedDynamicConstantsMixin<FGeneratedWrappedStructType>
	{
		FGeneratedWrappedStructType() = default;
		FGeneratedWrappedStructType(FGeneratedWrappedStructType&&) = delete;
		FGeneratedWrappedStructType(const FGeneratedWrappedStructType&) = delete;
		FGeneratedWrappedStructType& operator=(FGeneratedWrappedStructType&&) = delete;
		FGeneratedWrappedStructType& operator=(const FGeneratedWrappedStructType&) = delete;

		/** Internal version of Finalize, called before readying the type with Python */
		virtual void Finalize_PreReady() override;

		/** Get/sets associated with this type */
		FGeneratedWrappedGetSets GetSets;

		/** The doc string data for the properties associated with this type */
		TArray<FGeneratedWrappedPropertyDoc> PropertyDocs;

		/** Tracks and logs field conflicts on this type */
		FGeneratedWrappedFieldTracker FieldTracker;
	};

	/** Stores the data needed by a runtime generated Python class type */
	struct FGeneratedWrappedClassType : public FGeneratedWrappedType, public TGeneratedWrappedDynamicMethodsMixin<FGeneratedWrappedStructType>, public TGeneratedWrappedDynamicConstantsMixin<FGeneratedWrappedStructType>
	{
		FGeneratedWrappedClassType() = default;
		FGeneratedWrappedClassType(FGeneratedWrappedClassType&&) = delete;
		FGeneratedWrappedClassType(const FGeneratedWrappedClassType&) = delete;
		FGeneratedWrappedClassType& operator=(FGeneratedWrappedClassType&&) = delete;
		FGeneratedWrappedClassType& operator=(const FGeneratedWrappedClassType&) = delete;

		/** Internal version of Finalize, called before readying the type with Python */
		virtual void Finalize_PreReady() override;

		/** Internal version of Finalize, called after readying the type with Python */
		virtual void Finalize_PostReady() override;

		/** Methods associated with this type */
		FGeneratedWrappedMethods Methods;

		/** Get/sets associated with this type */
		FGeneratedWrappedGetSets GetSets;

		/** Constants associated with this type */
		FGeneratedWrappedConstants Constants;

		/** The doc string data for the properties associated with this type */
		TArray<FGeneratedWrappedPropertyDoc> PropertyDocs;

		/** Tracks and logs field conflicts on this type */
		FGeneratedWrappedFieldTracker FieldTracker;
	};

	/** Stores the data needed by a runtime generated Python enum entry */
	struct FGeneratedWrappedEnumEntry
	{
		FGeneratedWrappedEnumEntry() = default;
		FGeneratedWrappedEnumEntry(FGeneratedWrappedEnumEntry&&) = delete;
		FGeneratedWrappedEnumEntry(const FGeneratedWrappedEnumEntry&) = default;
		FGeneratedWrappedEnumEntry& operator=(FGeneratedWrappedEnumEntry&&) = delete;
		FGeneratedWrappedEnumEntry& operator=(const FGeneratedWrappedEnumEntry&) = default;

		/** Array of deprecated aliases for this enum entry, if any */
		TArray<FUTF8Buffer> EntryDeprecatedAliases;

		/** The name of the entry */
		FUTF8Buffer EntryName;

		/** The doc string of the entry */
		FUTF8Buffer EntryDoc;

		/** The value of the entry */
		int64 EntryValue = 0;
	};

	/** Stores the data needed by a runtime generated Python enum type */
	struct FGeneratedWrappedEnumType : public FGeneratedWrappedType
	{
		FGeneratedWrappedEnumType() = default;
		FGeneratedWrappedEnumType(FGeneratedWrappedEnumType&&) = delete;
		FGeneratedWrappedEnumType(const FGeneratedWrappedEnumType&) = delete;
		FGeneratedWrappedEnumType& operator=(FGeneratedWrappedEnumType&&) = delete;
		FGeneratedWrappedEnumType& operator=(const FGeneratedWrappedEnumType&) = delete;

		/** Internal version of Finalize, called after readying the type with Python */
		virtual void Finalize_PostReady() override;

		/** Internal version of Reset, called to remove data added to the Python type */
		virtual void Reset_CleansePyType() override;

		/** Internal version of Reset, called to reset the data on this type */
		virtual void Reset_CleanseSelf() override;

		/** Called to extract the enum entries array from the given enum */
		void ExtractEnumEntries(const UEnum* InEnum);

		/** Array of entries associated with this enum */
		TArray<FGeneratedWrappedEnumEntry> EnumEntries;
	};

	/** Definition data for an Unreal property generated from a Python type */
	struct FPropertyDef
	{
		FPropertyDef() = default;
		FPropertyDef(FPropertyDef&&) = default;
		FPropertyDef(const FPropertyDef&) = delete;
		FPropertyDef& operator=(FPropertyDef&&) = default;
		FPropertyDef& operator=(const FPropertyDef&) = delete;

		/** Data needed to re-wrap this property for Python */
		FGeneratedWrappedGetSet GeneratedWrappedGetSet;

		/** Definition of the re-wrapped property for Python */
		PyGetSetDef PyGetSet;
	};

	/** Definition data for an Unreal function generated from a Python type */
	struct FFunctionDef
	{
		FFunctionDef() = default;
		FFunctionDef(FFunctionDef&&) = default;
		FFunctionDef(const FFunctionDef&) = delete;
		FFunctionDef& operator=(FFunctionDef&&) = default;
		FFunctionDef& operator=(const FFunctionDef&) = delete;

		/** Data needed to re-wrap this function for Python */
		FGeneratedWrappedMethod GeneratedWrappedMethod;

		/** Definition of the re-wrapped function for Python */
		FPyMethodWithClosureDef PyMethod;

		/** Python function to call when invoking this re-wrapped function */
		FPyObjectPtr PyFunction;

		/** Is this a function hidden from Python? (eg, internal getter/setter function) */
		bool bIsHidden = false;
	};

	/** How should PythonizeName adjust the final name? */
	enum EPythonizeNameCase : uint8
	{
		/** lower_snake_case */
		Lower,
		/** UPPER_SNAKE_CASE */
		Upper,
	};

	/** Flags controlling the behavior of PythonizeValue/PythonizeMethodParam/PythonizeMethodReturnType. */
	enum class EPythonizeFlags
	{
		None = 0,
		IncludeUnrealNamespace = 1<<0,
		UseStrictTyping = 1<<1,
		DefaultConstructStructs = 1<<2,
		DefaultConstructDateTime = 1<<3,

		/** Type hinting is enabled. */
		WithTypeHinting = 1<<4,

		/** If the type of a value being hinted can be None, wrap the type in Optional[T], which is an alias to Union[T, None]. */
		WithOptionalType = 1<<5,

		/** Declare the type along with it known type coercion. For example [unreal.Name, str] because a string can be passed in place of a Name. */
		WithTypeCoercion = 1<<6,

		/** In method declaration, add '...' as default value even if no default value was found. Used in stub generation to workaround 'non-defaulted param after defaulted param' coming from the 'malformed' UFUNCTION. */
		AddMissingDefaultValueEllipseWorkaround = 1<<7,
	};
	ENUM_CLASS_FLAGS(EPythonizeFlags)

	/** Context information passed to PythonizeTooltip */
	struct FPythonizeTooltipContext
	{
		FPythonizeTooltipContext()
			: Prop(nullptr)
			, Func(nullptr)
			, ReadOnlyFlags(PropertyAccessUtil::RuntimeReadOnlyFlags)
		{
		}

		explicit FPythonizeTooltipContext(const FProperty* InProp, const uint64 InReadOnlyFlags = PropertyAccessUtil::RuntimeReadOnlyFlags);

		explicit FPythonizeTooltipContext(const UFunction* InFunc, const TSet<FName>& InParamsToIgnore = TSet<FName>());

		/** Optional property that should be used when converting property tooltips */
		const FProperty* Prop;

		/** Optional function that should be used when converting function tooltips */
		const UFunction* Func;

		/** Flags that dictate whether the property should be considered read-only */
		uint64 ReadOnlyFlags;

		/** Optional deprecation message for the property or function */
		FString DeprecationMessage;

		/** Optional set of parameters that we should ignore when generating function tooltips */
		TSet<FName> ParamsToIgnore;
	};

	/** Parsed tooltip data used by PythonizeTooltip */
	struct FParsedTooltip
	{
		struct FTokenString
		{
			FTokenString() = default;
			FTokenString(FTokenString&&) = default;
			FTokenString& operator=(FTokenString&&) = default;

			bool operator==(const FTokenString& InOther) const
			{
				return GetValue() == InOther.GetValue();
			}

			bool operator!=(const FTokenString& InOther) const
			{
				return GetValue() != InOther.GetValue();
			}

			FStringView GetValue() const
			{
				return SimpleValue.Len() > 0
					? SimpleValue
					: FStringView(ComplexValue);
			}

			void SetValue(FStringView InValue)
			{
				SimpleValue = InValue;
				ComplexValue.Reset();
			}

			void SetValue(FString&& InValue)
			{
				SimpleValue.Reset();
				ComplexValue = MoveTemp(InValue);
			}

			FStringView SimpleValue;
			FString ComplexValue;
		};

		struct FMiscToken
		{
			FMiscToken() = default;
			FMiscToken(FMiscToken&&) = default;
			FMiscToken& operator=(FMiscToken&&) = default;

			FTokenString TokenName;
			FTokenString TokenValue;
		};

		struct FParamToken
		{
			FParamToken() = default;
			FParamToken(FParamToken&&) = default;
			FParamToken& operator=(FParamToken&&) = default;

			FTokenString ParamName;
			FTokenString ParamType;
			FTokenString ParamComment;
		};

		typedef TArray<FMiscToken, TInlineAllocator<4>> FMiscTokensArray;
		typedef TArray<FParamToken, TInlineAllocator<8>> FParamTokensArray;

		int32 SourceTooltipLen = 0;

		FString BasicTooltipText;
		FMiscTokensArray MiscTokens;
		FParamTokensArray ParamTokens;
		FParamToken ReturnToken;
	};

	/** Convert a TCHAR to a UTF-8 buffer */
	FUTF8Buffer TCHARToUTF8Buffer(const TCHAR* InStr);

	/** Get the PostInit function from this Python type */
	PyObject* GetPostInitFunc(PyTypeObject* InPyType);

	/** Add a struct init parameter to the given array of method parameters */
	void AddStructInitParam(const FProperty* InUnrealProp, const TCHAR* InPythonAttrName, TArray<FGeneratedWrappedMethodParameter>& OutInitParams);

	/** Given a function, extract all of its parameter information (input and output) */
	void ExtractFunctionParams(const UFunction* InFunc, TArray<FGeneratedWrappedMethodParameter>& OutInputParams, TArray<FGeneratedWrappedMethodParameter>& OutOutputParams);

	/** Apply default values to function arguments */
	void ApplyParamDefaults(void* InBaseParamsAddr, const TArray<FGeneratedWrappedMethodParameter>& InParamDef);

	/** Parse an array Python objects from the args and keywords based on the generated method parameter data */
	bool ParseMethodParameters(PyObject* InArgs, PyObject* InKwds, const TArray<FGeneratedWrappedMethodParameter>& InParamDef, const char* InPyMethodName, TArray<PyObject*>& OutPyParams);

	/** Given a set of return values and the struct data associated with them, pack them appropriately for returning to Python */
	PyObject* PackReturnValues(const void* InBaseParamsAddr, const TArray<FGeneratedWrappedMethodParameter>& InOutputParams, const TCHAR* InErrorCtxt, const TCHAR* InCallingCtxt);

	/** Given a Python return value, unpack the values into the struct data associated with them */
	bool UnpackReturnValues(PyObject* InRetVals, const FOutParmRec* InOutputParms, const TCHAR* InErrorCtxt, const TCHAR* InCallingCtxt);

	/** Invoke a Python callable from an Unreal function thunk (ie, DEFINE_FUNCTION), ensuring that the calling convention is correct for both native and script function callers */
	bool InvokePythonCallableFromUnrealFunctionThunk(FPyObjectPtr InSelf, PyObject* InCallable, const UFunction* InFunc, UObject* Context, FFrame& Stack, RESULT_DECL);

	/** Get the current value of the given property from the given struct */
	PyObject* GetPropertyValue(const UStruct* InStruct, const void* InStructData, const FGeneratedWrappedProperty& InPropDef, const char *InAttributeName, PyObject* InOwnerPyObject, const TCHAR* InErrorCtxt);

	/** Set the current value of the given property from the given struct */
	int SetPropertyValue(const UStruct* InStruct, void* InStructData, PyObject* InValue, const FGeneratedWrappedProperty& InPropDef, const char *InAttributeName, const FPropertyAccessChangeNotify* InChangeNotify, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const TCHAR* InErrorCtxt, const TConstArrayView<void*>& InArchetypeInstContainers);

	/** Build a Python doc string for the given function and arguments list */
	FString BuildFunctionDocString(const UFunction* InFunc, const FString& InFuncPythonName, const TArray<FGeneratedWrappedMethodParameter>& InInputParams, const TArray<FGeneratedWrappedMethodParameter>& InOutputParams, const bool* InStaticOverride = nullptr);

	/** Is the given class generated by Blueprints? */
	bool IsBlueprintGeneratedClass(const UClass* InClass);

	/** Is the given struct generated by Blueprints? */
	bool IsBlueprintGeneratedStruct(const UScriptStruct* InStruct);

	/** Is the given enum generated by Blueprints? */
	bool IsBlueprintGeneratedEnum(const UEnum* InEnum);

	/** Is the given class marked as deprecated? */
	bool IsDeprecatedClass(const UClass* InClass, FString* OutDeprecationMessage = nullptr);

	/** Is the given property marked as deprecated? */
	bool IsDeprecatedProperty(const FProperty* InProp, FString* OutDeprecationMessage = nullptr);

	/** Is the given function marked as deprecated? */
	bool IsDeprecatedFunction(const UFunction* InFunc, FString* OutDeprecationMessage = nullptr);

	/** Given a class, get all the interface classes related to it that should be included when exporting it to Python */
	TArray<const UClass*> GetExportedInterfacesForClass(const UClass* InClass);

	/** Should the given class be exported to Python? */
	bool ShouldExportClass(const UClass* InClass);

	/** Should the given struct be exported to Python? */
	bool ShouldExportStruct(const UScriptStruct* InStruct);

	/** Should the given enum be exported to Python? */
	bool ShouldExportEnum(const UEnum* InEnum);

	/** Should the given enum entry be exported to Python? */
	bool ShouldExportEnumEntry(const UEnum* InEnum, int32 InEnumEntryIndex);

	/** Should the given property be exported to Python? */
	bool ShouldExportProperty(const FProperty* InProp);

	/** Should the given property be exported to Python as editor-only data? */
	bool ShouldExportEditorOnlyProperty(const FProperty* InProp);

	/** Should the given function be exported to Python? */
	bool ShouldExportFunction(const UFunction* InFunc);

	/** Check that the given name will be valid for Python */
	bool IsValidName(FStringView InName, FText* OutError = nullptr);

	/** Given a CamelCase name, convert it to snake_case */
	FString PythonizeName(FStringView InName, const EPythonizeNameCase InNameCase);

	/** Given a CamelCase property name, convert it to snake_case (may remove some superfluous parts of the property name) */
	FString PythonizePropertyName(FStringView InName, const EPythonizeNameCase InNameCase);

	/** Given a property tooltip, convert it to a doc string */
	FString PythonizePropertyTooltip(const FParsedTooltip& InTooltip, const FProperty* InProp, const uint64 InReadOnlyFlags = PropertyAccessUtil::RuntimeReadOnlyFlags);

	/** Given a function tooltip, convert it to a doc string */
	FString PythonizeFunctionTooltip(const FParsedTooltip& InTooltip, const UFunction* InFunc, const TSet<FName>& ParamsToIgnore = TSet<FName>());

	/** Given a parsed tooltip, convert it to a doc string */
	FString PythonizeTooltip(const FParsedTooltip& InTooltip, const FPythonizeTooltipContext& InContext = FPythonizeTooltipContext());

	/**
	 * Given a tooltip, parse it into something that can be converted into a doc string
	 * @note The parsed result may referenced sub-strings of the given view, so InTooltip must exist as long as the result does.
	 */
	FParsedTooltip ParseTooltip(FStringView InTooltip);

	/** Given a property and its value, convert it into something that could be used in a Python script */
	FString PythonizeValue(const FProperty* InProp, const void* InPropValue, const EPythonizeFlags InPythonizeFlags = EPythonizeFlags::None);

	/** Given a property and its default value, convert it into something that could be used in a Python script */
	FString PythonizeDefaultValue(const FProperty* InProp, const FString& InDefaultValue, const EPythonizeFlags InPythonizeFlags = EPythonizeFlags::None);

	/** Given a generated method parameter, extract the name, type and default value and format a parameter declaration usable in a Python method signature according to the specified flags. */
	FString PythonizeMethodParam(const FGeneratedWrappedMethodParameter& InMethodParam, EPythonizeFlags InPythonizeFlags = EPythonizeFlags::None, const TFunction<bool(const FString&)>& ShouldEllipseDefaultValue = TFunction<bool(const FString&)>());

	/** Given a property and a name, format a parameter declaration usable in a Python method signature according to the specified flags. */
	FString PythonizeMethodParam(const FString& InParamName, const FProperty* ParamProp, EPythonizeFlags InPythonizeFlags = EPythonizeFlags::None);

	/** Given a param name and optional param type and default value, format a parameter declaration usable in a Python method signature. */
	FString PythonizeMethodParam(const FString& InParamName, const FString& InParamType = FString(), const FString& InDefaultValue = FString());

	/** Given list of return parameter, format the return type declaration usable in the Python method signature according to the flags. */
	FString PythonizeMethodReturnType(const TArray<PyGenUtil::FGeneratedWrappedMethodParameter>& InOutputParams, EPythonizeFlags InPythonizeFlags = EPythonizeFlags::None);

	/** Given a property, format a return type declaration usable in the Python method signature according to the flags. */
	FString PythonizeMethodReturnType(const FProperty* InProp, EPythonizeFlags InPythonizeFlags = EPythonizeFlags::None);

	/** Get the type that should be used with the Python type registry for an asset (eg, a Blueprint asset should use its generated class) */
	const UObject* GetAssetTypeRegistryType(const UObject* InObj);

	/** Get the native module the given field belongs to */
	FString GetFieldModule(const UField* InField);

	/** Get the plugin module the given field belongs to (if any) */
	FString GetFieldPlugin(const UField* InField);

	/** Given a native module name, get the Python module we should use */
	FString GetModulePythonName(const FName InModuleName, const bool bIncludePrefix = true);

	/** Get the Python name of the given class */
	FString GetClassPythonName(const UClass* InClass);

	/** Get the deprecated Python names of the given class */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedClassPythonNames(const UClass* InClass);

	/** Get the Python name of the given struct */
	FString GetStructPythonName(const UScriptStruct* InStruct);

	/** Get the deprecated Python names of the given struct */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedStructPythonNames(const UScriptStruct* InStruct);

	/** Get the Python name of the given enum */
	FString GetEnumPythonName(const UEnum* InEnum);

	/** Get the deprecated Python names of the given enum */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedEnumPythonNames(const UEnum* InEnum);

	/** Get the Python name of the given enum entry */
	FString GetEnumEntryPythonName(const UEnum* InEnum, const int32 InEntryIndex);

	/** Get the deprecated Python names of the given enum entry */
	TArray<FString> GetDeprecatedEnumEntryPythonNames(const UEnum* InEnum, const int32 InEntryIndex);

	/** Get the Python name of the given delegate signature */
	FString GetDelegatePythonName(const UFunction* InDelegateSignature);

	/** Get the Python name of the given function */
	FString GetFunctionPythonName(const UFunction* InFunc);

	/** Get the deprecated Python names of the given function */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedFunctionPythonNames(const UFunction* InFunc);

	/** Get the Python name of the given function when it's hoisted as a script method */
	FString GetScriptMethodPythonName(const UFunction* InFunc);

	/** Get the deprecated Python names of the given function it's hoisted as a script method */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedScriptMethodPythonNames(const UFunction* InFunc);

	/** Get the Python name of the given function when it's hoisted as a script constant */
	FString GetScriptConstantPythonName(const UFunction* InFunc);

	/** Get the deprecated Python names of the given function it's hoisted as a script constant */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedScriptConstantPythonNames(const UFunction* InFunc);

	/** Get the Python name of the given property */
	FString GetPropertyPythonName(const FProperty* InProp);

	/** Get the deprecated Python names of the given property */
	TArray<TTuple<FSoftObjectPath, FString>> GetDeprecatedPropertyPythonNames(const FProperty* InProp);

	/** Get the Python name of the given property */
	FString GetPropertyTypePythonName(const FProperty* InProp, const bool InIncludeUnrealNamespace = false, const bool InIsForDocString = true, const EPythonizeFlags PythonizeFlags = EPythonizeFlags::None);

	/** Get the Python type of the given property */
	FString GetPropertyPythonType(const FProperty* InProp);

	/** Append the Python type of the given property to the given string */
	void AppendPropertyPythonType(const FProperty* InProp, FString& OutStr);

	/** Append the given Python property read /write state to the given string */
	void AppendPropertyPythonReadWriteState(const FProperty* InProp, FString& OutStr, const uint64 InReadOnlyFlags = PropertyAccessUtil::RuntimeReadOnlyFlags);

	/** Get the tooltip for the given field */
	FString GetFieldTooltip(const UField* InField);
	FString GetFieldTooltip(const FField* InField);

	/** Get the tooltip for the given enum entry */
	FString GetEnumEntryTooltip(const UEnum* InEnum, const int64 InEntryIndex);

	/** Get the doc string for the C++ or asset source information of the given type */
	FString BuildSourceInformationDocString(const UField* InOwnerType);

	/** Append the doc string for the C++ or asset source information of the given type to the given string */
	void AppendSourceInformationDocString(const UField* InOwnerType, FString& OutStr);

	/** Save a generated text file to disk as UTF-8 (only writes the file if the contents differs, unless forced) */
	bool SaveGeneratedTextFile(const TCHAR* InFilename, FStringView InFileContents, const bool InForceWrite = false);

	/**
	 * Whether Python type hints should be generated. This is only effective if Python interpreter version has the support. This
	 * affects the Python stub and docstrings. The function must be called before Python glue and the stub are generated. It
	 * can be set from the user settings or forced to a value for a specific purpose, like generating the online doc.
	 */
	void SetTypeHintingMode(ETypeHintingMode InMode);

	/**
	 * Return the effecting type hinting mode.
	 * @see SetTypeHintingMode.
	 */
	ETypeHintingMode GetTypeHintingMode();

	/**
	 * Whether the stub and docstrings are type hinted during glue/stub generation. Default to false. This doesn't read
	 * the user settings, but rather the value set with SetTypeHintingMode(). The function always returns false if
	 * the linked Python interpreter version is older than 3.7.
	 */
	bool IsTypeHintingEnabled();
}

#endif	// WITH_PYTHON
