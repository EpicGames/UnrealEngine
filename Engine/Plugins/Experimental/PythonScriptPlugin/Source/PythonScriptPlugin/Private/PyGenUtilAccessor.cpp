// Copyright Epic Games, Inc. All Rights Reserved.


#include "PyGenUtilAccessor.h"


#include "PyGenUtil.h"


#if WITH_PYTHON


void PyGenUtilAccessor::ExtractFunctionParamsAsProperties(const UFunction* InFunc, TArray<const FProperty*>& OutInputParams, TArray<const FProperty*>& OutOutputParams)
{
	OutInputParams.Empty();
	OutOutputParams.Empty();

	TArray<PyGenUtil::FGeneratedWrappedMethodParameter> InputParams;
	TArray<PyGenUtil::FGeneratedWrappedMethodParameter> OutputParams;
	PyGenUtil::ExtractFunctionParams(InFunc, InputParams, OutputParams);

	for (const PyGenUtil::FGeneratedWrappedMethodParameter& InputParam : InputParams)
	{
		OutInputParams.Add(InputParam.ParamProp);
	}
	for (const PyGenUtil::FGeneratedWrappedMethodParameter& OutputParam : OutputParams)
	{
		OutOutputParams.Add(OutputParam.ParamProp);
	}
}


FString PyGenUtilAccessor::GetClassPythonName(const UClass* InClass)
{
	return PyGenUtil::GetClassPythonName(InClass);	
}


FString PyGenUtilAccessor::GetFunctionPythonName(const UFunction* InFunc)
{
	return PyGenUtil::GetFunctionPythonName(InFunc);	
}


FString PyGenUtilAccessor::GetPropertyPythonName(const FProperty* InProp)
{
	return PyGenUtil::GetPropertyPythonName(InProp);
}


FString PyGenUtilAccessor::GetPropertyPythonType(const FProperty* InProp)
{
	return PyGenUtil::GetPropertyPythonType(InProp);
}


bool PyGenUtilAccessor::ShouldExportClass(const UClass* InClass)
{
	return PyGenUtil::ShouldExportClass(InClass);
}


bool PyGenUtilAccessor::ShouldExportFunction(const UFunction* InFunc)
{
	return PyGenUtil::ShouldExportFunction(InFunc);
}


bool PyGenUtilAccessor::ShouldExportProperty(const FProperty* InProp)
{
	return PyGenUtil::ShouldExportProperty(InProp);
}


bool PyGenUtilAccessor::ShouldExportEditorOnlyProperty(const FProperty* InProp)
{
	return PyGenUtil::ShouldExportEditorOnlyProperty(InProp);
}


#endif
