// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyMethodWithClosure.h"
#include "PyUtil.h"
#include "PyPtr.h"

#if WITH_PYTHON

/** Free list of FPyCFunctionWithClosureObject instances to avoid allocation churn (chained via SelfArg) */
struct FPyCFunctionWithClosureObjectFreeList
{
public:
	FPyCFunctionWithClosureObjectFreeList()
		: Head(nullptr)
		, Count(0)
	{
	}

	~FPyCFunctionWithClosureObjectFreeList()
	{
		while (Head)
		{
			FPyCFunctionWithClosureObject* Obj = Head;
			Head = (FPyCFunctionWithClosureObject*)Obj->SelfArg;
			PyObject_GC_Del(Obj);
		}
	}

	/** Push an entry onto the free list if there's space */
	bool Push(FPyCFunctionWithClosureObject* InObj)
	{
		static const int32 MaxCount = 256;
		if (Count >= MaxCount)
		{
			return false;
		}
		
		InObj->SelfArg = (PyObject*)Head;
		Head = InObj;
		++Count;
		return true;
	}

	/** Pop an entry from the free list (if any) */
	FPyCFunctionWithClosureObject* Pop()
	{
		FPyCFunctionWithClosureObject* Obj = Head;
		if (Obj)
		{
			Head = (FPyCFunctionWithClosureObject*)Obj->SelfArg;
			PyObject_INIT(Obj, &PyCFunctionWithClosureType);
			--Count;
		}
		return Obj;
	}

private:
	FPyCFunctionWithClosureObject* Head;
	int32 Count;
};

FPyCFunctionWithClosureObjectFreeList* PyCFunctionWithClosureObjectFreeList = nullptr;

bool InitializePyMethodWithClosure()
{
	if (!ensure(PyType_Ready(&PyCFunctionWithClosureType) == 0))
	{
		PyUtil::LogPythonError();
		UE_LOGF(LogPython, Error, "PyType_Ready(&PyCFunctionWithClosureType) failed! Python will be disabled.")
		return false;
	}

	if (!ensure(PyType_Ready(&PyMethodWithClosureDescrType) == 0))
	{
		PyUtil::LogPythonError();
		UE_LOGF(LogPython, Error, "PyType_Ready(&PyMethodWithClosureDescrType) failed! Python will be disabled.")
		return false;
	}
	
	if (!ensure(PyType_Ready(&PyClassMethodWithClosureDescrType) == 0))
	{
		PyUtil::LogPythonError();
		UE_LOGF(LogPython, Error, "PyType_Ready(&PyClassMethodWithClosureDescrType) failed! Python will be disabled.")
		return false;
	}

	PyCFunctionWithClosureObjectFreeList = new FPyCFunctionWithClosureObjectFreeList();
	return true;
}

void ShutdownPyMethodWithClosure()
{
	delete PyCFunctionWithClosureObjectFreeList;
	PyCFunctionWithClosureObjectFreeList = nullptr;
}


bool FPyMethodWithClosureDef::AddMethod(FPyMethodWithClosureDef* InMethod, PyTypeObject* InType)
{
	if (!(InMethod->MethodFlags & METH_COEXIST) && PyDict_GetItemString(InType->tp_dict, InMethod->MethodName))
	{
		// Skip this method, but don't fail
		return true;
	}

	FPyObjectPtr Descr = FPyObjectPtr::StealReference(NewMethodDescriptor(InType, InMethod));
	if (!Descr)
	{
		return false;
	}

	if (PyDict_SetItemString(InType->tp_dict, InMethod->MethodName, Descr) != 0)
	{
		return false;
	}

	return true;
}

bool FPyMethodWithClosureDef::AddMethods(FPyMethodWithClosureDef* InMethods, PyTypeObject* InType)
{
	for (FPyMethodWithClosureDef* MethodDef = InMethods; MethodDef->MethodName; ++MethodDef)
	{
		if (!AddMethod(MethodDef, InType))
		{
			return false;
		}
	}

	return true;
}

PyObject* FPyMethodWithClosureDef::NewMethodDescriptor(PyTypeObject* InType, FPyMethodWithClosureDef* InDef)
{
	FPyObjectPtr Descr;

	if (InDef->MethodFlags & METH_CLASS)
	{
		if (InDef->MethodFlags & METH_STATIC)
		{
			PyErr_Format(PyExc_ValueError, "method '%s' cannot be both class and static", InDef->MethodName);
			return nullptr;
		}

		Descr = FPyObjectPtr::StealReference((PyObject*)FPyMethodWithClosureDescrObject::NewClassMethod(InType, InDef));
	}
	else if (InDef->MethodFlags & METH_STATIC)
	{
		FPyObjectPtr FuncObj = FPyObjectPtr::StealReference((PyObject*)FPyCFunctionWithClosureObject::New(InDef, nullptr, nullptr));
		if (FuncObj)
		{
			Descr = FPyObjectPtr::StealReference(PyStaticMethod_New(FuncObj));
		}
	}
	else
	{
		Descr = FPyObjectPtr::StealReference((PyObject*)FPyMethodWithClosureDescrObject::NewMethod(InType, InDef));
	}

	return Descr.Release();
}

PyObject* FPyMethodWithClosureDef::Call(FPyMethodWithClosureDef* InDef, PyObject* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	const int FuncFlags = InDef->MethodFlags & ~(METH_CLASS | METH_STATIC | METH_COEXIST);

	// Keywords take precedence
	if (FuncFlags & METH_KEYWORDS)
	{
		return (*(PyCFunctionWithClosureAndKeywords)(void*)InDef->MethodCallback)(InSelf, InArgs, InKwds, InDef->MethodClosure);
	}

	// If this method wasn't flagged with METH_KEYWORDS then we shouldn't have been passed any keyword arguments
	if (InKwds && PyDict_Size(InKwds) != 0)
	{
		PyErr_Format(PyExc_TypeError, "%s() takes no keyword arguments", InDef->MethodName);
		return nullptr;
	}

	// Handle the other calling conventions
	const Py_ssize_t ArgsCount = PyTuple_GET_SIZE(InArgs);
	switch (FuncFlags)
	{
	case METH_VARARGS:
		return (*InDef->MethodCallback)(InSelf, InArgs, InDef->MethodClosure);

	case METH_NOARGS:
		if (ArgsCount == 0)
		{
			return (*(PyCFunctionWithClosureAndNoArgs)(void*)InDef->MethodCallback)(InSelf, InDef->MethodClosure);
		}

		PyErr_Format(PyExc_TypeError, "%s() takes no arguments (%d given)", InDef->MethodName, (int32)ArgsCount);
		return nullptr;

	case METH_O:
		if (ArgsCount == 1)
		{
			return (*InDef->MethodCallback)(InSelf, PyTuple_GET_ITEM(InArgs, 0), InDef->MethodClosure);
		}

		PyErr_Format(PyExc_TypeError, "%s() takes exactly one argument (%d given)", InDef->MethodName, (int32)ArgsCount);
		return nullptr;

	default:
		break;
	}

	PyErr_BadInternalCall();
	return nullptr;
}


FPyCFunctionWithClosureObject* FPyCFunctionWithClosureObject::New(FPyMethodWithClosureDef* InMethod, PyObject* InSelf, PyObject* InModule)
{
	FPyCFunctionWithClosureObject* Self = PyCFunctionWithClosureObjectFreeList ? PyCFunctionWithClosureObjectFreeList->Pop() : nullptr;
	if (!Self)
	{
		Self = PyObject_GC_New(FPyCFunctionWithClosureObject, &PyCFunctionWithClosureType);
	}

	if (Self)
	{
		Self->MethodDef = InMethod;

		Py_XINCREF(InSelf);
		Self->SelfArg = InSelf;

		Py_XINCREF(InModule);
		Self->ModuleAttr = InModule;

		PyObject_GC_Track(Self);
	}

	return Self;
}

void FPyCFunctionWithClosureObject::Free(FPyCFunctionWithClosureObject* InSelf)
{
	PyObject_GC_UnTrack(InSelf);

	InSelf->MethodDef = nullptr;

	Py_XDECREF(InSelf->SelfArg);
	InSelf->SelfArg = nullptr;

	Py_XDECREF(InSelf->ModuleAttr);
	InSelf->ModuleAttr = nullptr;

	if (!PyCFunctionWithClosureObjectFreeList || !PyCFunctionWithClosureObjectFreeList->Push(InSelf))
	{
		PyObject_GC_Del(InSelf);
	}
}

int FPyCFunctionWithClosureObject::GCTraverse(FPyCFunctionWithClosureObject* InSelf, visitproc InVisit, void* InArg)
{
	// Aliases for Py_VISIT
	visitproc visit = InVisit;
	void* arg = InArg;

	Py_VISIT(InSelf->SelfArg);
	Py_VISIT(InSelf->ModuleAttr);
	return 0;
}

int FPyCFunctionWithClosureObject::GCClear(FPyCFunctionWithClosureObject* InSelf)
{
	Py_CLEAR(InSelf->SelfArg);
	Py_CLEAR(InSelf->ModuleAttr);
	return 0;
}

PyObject* FPyCFunctionWithClosureObject::Str(FPyCFunctionWithClosureObject* InSelf)
{
	if (InSelf->SelfArg)
	{
		return PyUnicode_FromFormat("<built-in method %s of %s object at %p>", InSelf->MethodDef->MethodName, InSelf->SelfArg->ob_type->tp_name, InSelf->SelfArg);
	}
	return PyUnicode_FromFormat("<built-in function %s>", InSelf->MethodDef->MethodName);
}


FPyMethodWithClosureDescrObject* FPyMethodWithClosureDescrObject::NewMethod(PyTypeObject* InType, FPyMethodWithClosureDef* InMethod)
{
	return NewImpl(&PyMethodWithClosureDescrType, InType, InMethod);
}

FPyMethodWithClosureDescrObject* FPyMethodWithClosureDescrObject::NewClassMethod(PyTypeObject* InType, FPyMethodWithClosureDef* InMethod)
{
	return NewImpl(&PyClassMethodWithClosureDescrType, InType, InMethod);
}

FPyMethodWithClosureDescrObject* FPyMethodWithClosureDescrObject::NewImpl(PyTypeObject* InDescrType, PyTypeObject* InType, FPyMethodWithClosureDef* InMethod)
{
	FPyMethodWithClosureDescrObject* Self = PyObject_GC_New(FPyMethodWithClosureDescrObject, InDescrType);
	if (Self)
	{
		Py_XINCREF(InType);
		Self->OwnerType = InType;

		Self->MethodName = PyUnicode_FromString(InMethod->MethodName);

		Self->MethodDef = InMethod;

		PyObject_GC_Track(Self);
	}
	return Self;
}

void FPyMethodWithClosureDescrObject::Free(FPyMethodWithClosureDescrObject* InSelf)
{
	PyObject_GC_UnTrack(InSelf);

	Py_XDECREF(InSelf->OwnerType);
	InSelf->OwnerType = nullptr;

	Py_XDECREF(InSelf->MethodName);
	InSelf->MethodName = nullptr;

	InSelf->MethodDef = nullptr;

	PyObject_GC_Del(InSelf);
}

int FPyMethodWithClosureDescrObject::GCTraverse(FPyMethodWithClosureDescrObject* InSelf, visitproc InVisit, void* InArg)
{
	// Aliases for Py_VISIT
	visitproc visit = InVisit;
	void* arg = InArg;

	Py_VISIT(InSelf->OwnerType);
	Py_VISIT(InSelf->MethodName);
	return 0;
}

int FPyMethodWithClosureDescrObject::GCClear(FPyMethodWithClosureDescrObject* InSelf)
{
	Py_CLEAR(InSelf->OwnerType);
	Py_CLEAR(InSelf->MethodName);
	return 0;
}

PyObject* FPyMethodWithClosureDescrObject::Str(FPyMethodWithClosureDescrObject* InSelf)
{
	return PyUnicode_FromFormat("<method '%s' of '%s' objects>", TCHAR_TO_UTF8(*GetMethodName(InSelf)), InSelf->OwnerType->tp_name);
}

FString FPyMethodWithClosureDescrObject::GetMethodName(FPyMethodWithClosureDescrObject* InSelf)
{
	FString Name;
	if (InSelf->MethodName)
	{
		Name = PyUtil::PyObjectToUEString(InSelf->MethodName);
	}
	if (Name.IsEmpty())
	{
		Name = TEXT("?");
	}
	return Name;
}


PyTypeObject InitializePyCFunctionWithClosureType()
{
	struct FFuncs
	{
		static void Dealloc(FPyCFunctionWithClosureObject* InSelf)
		{
			FPyCFunctionWithClosureObject::Free(InSelf);
		}

		static int Traverse(FPyCFunctionWithClosureObject* InSelf, visitproc InVisit, void* InArg)
		{
			return FPyCFunctionWithClosureObject::GCTraverse(InSelf, InVisit, InArg);
		}

		static int Clear(FPyCFunctionWithClosureObject* InSelf)
		{
			return FPyCFunctionWithClosureObject::GCClear(InSelf);
		}

		static PyObject* Str(FPyCFunctionWithClosureObject* InSelf)
		{
			return FPyCFunctionWithClosureObject::Str(InSelf);
		}

		static PyObject* RichCmp(FPyCFunctionWithClosureObject* InSelf, PyObject* InOther, int InOp)
		{
			if (InOp != Py_EQ && InOp != Py_NE)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (PyObject_IsInstance(InOther, (PyObject*)&PyCFunctionWithClosureType) != 1)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			FPyCFunctionWithClosureObject* Other = (FPyCFunctionWithClosureObject*)InOther;

			bool bResult = InSelf->SelfArg == Other->SelfArg && InSelf->MethodDef->MethodCallback == Other->MethodDef->MethodCallback && InSelf->MethodDef->MethodClosure == Other->MethodDef->MethodClosure;
			if (InOp == Py_NE)
			{
				bResult = !bResult;
			}

			if (bResult)
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyUtil::FPyHashType Hash(FPyCFunctionWithClosureObject* InSelf)
		{
			PyUtil::FPyHashType PyHash = InSelf->SelfArg ? (PyUtil::FPyHashType)PyObject_Hash(InSelf->SelfArg) : 0;
			if (PyHash == -1)
			{
				return -1;
			}

			PyHash = (PyUtil::FPyHashType)HashCombine((uint32)PyHash, GetTypeHash((void*)InSelf->MethodDef->MethodCallback));
			if (InSelf->MethodDef->MethodClosure)
			{
				PyHash = (PyUtil::FPyHashType)HashCombine((uint32)PyHash, GetTypeHash(InSelf->MethodDef->MethodClosure));
			}
			return PyHash != -1 ? PyHash : 0;
		}

		static PyObject* Call(FPyCFunctionWithClosureObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			return FPyMethodWithClosureDef::Call(InSelf->MethodDef, InSelf->SelfArg, InArgs, InKwds);
		}
	};

	struct FGetSets
	{
		static PyObject* GetDoc(FPyCFunctionWithClosureObject* InSelf, void* InClosure)
		{
			if (InSelf->MethodDef->MethodDoc)
			{
				return PyUnicode_FromString(InSelf->MethodDef->MethodDoc);
			}

			Py_RETURN_NONE;
		}

		static PyObject* GetName(FPyCFunctionWithClosureObject* InSelf, void* InClosure)
		{
			return PyUnicode_FromString(InSelf->MethodDef->MethodName);
		}

		static PyObject* GetSelf(FPyCFunctionWithClosureObject* InSelf, void* InClosure)
		{
			PyObject* Self = InSelf->SelfArg ? InSelf->SelfArg : Py_None;
			Py_INCREF(Self);
			return Self;
		}
	};

	static PyMemberDef PyMembers[] = {
		{ PyCStrCast("__module__"), T_OBJECT, STRUCT_OFFSET(FPyCFunctionWithClosureObject, ModuleAttr), PY_WRITE_RESTRICTED, nullptr },
		{ nullptr, 0, 0, 0, nullptr }
	};

	static PyGetSetDef PyGetSets[] = {
		{ PyCStrCast("__doc__"), (getter)&FGetSets::GetDoc, nullptr, nullptr, nullptr },
		{ PyCStrCast("__name__"), (getter)&FGetSets::GetName, nullptr, nullptr, nullptr },
		{ PyCStrCast("__self__"), (getter)&FGetSets::GetSelf, nullptr, nullptr, nullptr },
		{ nullptr, nullptr, nullptr, nullptr, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"builtin_function_or_method_with_closure", /* tp_name */
		sizeof(FPyCFunctionWithClosureObject), /* tp_basicsize */
	};

	PyType.tp_base = &PyCFunction_Type; // This is a hack so that the doc gen code will think this is a built-in function

	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_traverse = (traverseproc)&FFuncs::Traverse;
	PyType.tp_clear = (inquiry)&FFuncs::Clear;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;
	PyType.tp_call = (ternaryfunc)&FFuncs::Call;
	PyType.tp_getattro = (getattrofunc)&PyObject_GenericGetAttr;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_HAVE_GC;

	PyType.tp_members = PyMembers;
	PyType.tp_getset = PyGetSets;

	return PyType;
}


void InitializePyMethodWithClosureDescrTypeMembers(PyTypeObject& PyType)
{
	static PyMemberDef PyMembers[] = {
		{ PyCStrCast("__objclass__"), T_OBJECT, STRUCT_OFFSET(FPyMethodWithClosureDescrObject, OwnerType), READONLY, nullptr },
		{ PyCStrCast("__name__"), T_OBJECT, STRUCT_OFFSET(FPyMethodWithClosureDescrObject, MethodName), READONLY, nullptr },
		{ nullptr, 0, 0, 0, nullptr }
	};

	PyType.tp_members = PyMembers;
}

void InitializePyMethodWithClosureDescrTypeGetSet(PyTypeObject& PyType)
{
	struct FGetSets
	{
		static PyObject* GetDoc(FPyMethodWithClosureDescrObject* InSelf, void* InClosure)
		{
			if (InSelf->MethodDef->MethodDoc)
			{
				return PyUnicode_FromString(InSelf->MethodDef->MethodDoc);
			}

			Py_RETURN_NONE;
		}
	};

	static PyGetSetDef PyGetSets[] = {
		{ PyCStrCast("__doc__"), (getter)&FGetSets::GetDoc, nullptr, nullptr, nullptr },
		{ nullptr, nullptr, nullptr, nullptr, nullptr }
	};

	PyType.tp_getset = PyGetSets;
}

PyTypeObject InitializePyMethodWithClosureDescrType()
{
	struct FFuncs
	{
		static void Dealloc(FPyMethodWithClosureDescrObject* InSelf)
		{
			FPyMethodWithClosureDescrObject::Free(InSelf);
		}

		static int Traverse(FPyMethodWithClosureDescrObject* InSelf, visitproc InVisit, void* InArg)
		{
			return FPyMethodWithClosureDescrObject::GCTraverse(InSelf, InVisit, InArg);
		}

		static int Clear(FPyMethodWithClosureDescrObject* InSelf)
		{
			return FPyMethodWithClosureDescrObject::GCClear(InSelf);
		}

		static PyObject* Str(FPyMethodWithClosureDescrObject* InSelf)
		{
			return FPyMethodWithClosureDescrObject::Str(InSelf);
		}

		static PyObject* Call(FPyMethodWithClosureDescrObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			check(PyTuple_Check(InArgs));

			Py_ssize_t ArgCount = PyTuple_GET_SIZE(InArgs);
			if (ArgCount < 1)
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' of '%s' object needs an argument", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name);
				return nullptr;
			}

			PyObject* Self = PyTuple_GET_ITEM(InArgs, 0);
			if (PyObject_IsSubclass((PyObject*)Py_TYPE(Self), (PyObject*)InSelf->OwnerType) != 1)
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' requires a '%s' object but received a '%s'", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name, Self->ob_type->tp_name);
				return nullptr;
			}

			FPyObjectPtr Args = FPyObjectPtr::StealReference(PyTuple_GetSlice(InArgs, 1, ArgCount));
			return FPyMethodWithClosureDef::Call(InSelf->MethodDef, Self, Args, InKwds);
		}

		static PyObject* DescrGet(FPyMethodWithClosureDescrObject* InSelf, PyObject* InObj, PyObject* InType)
		{
			if (!InObj)
			{
				Py_INCREF(InSelf);
				return (PyObject*)InSelf;
			}

			if (!PyObject_TypeCheck(InObj, InSelf->OwnerType))
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' for '%s' objects doesn't apply to '%s' object", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name, InObj->ob_type->tp_name);
				return nullptr;
			}

			return (PyObject*)FPyCFunctionWithClosureObject::New(InSelf->MethodDef, InObj, nullptr);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"methodwithclosure_descriptor", /* tp_name */
		sizeof(FPyMethodWithClosureDescrObject), /* tp_basicsize */
	};

	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_traverse = (traverseproc)&FFuncs::Traverse;
	PyType.tp_clear = (inquiry)&FFuncs::Clear;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_call = (ternaryfunc)&FFuncs::Call;
	PyType.tp_descr_get = (descrgetfunc)&FFuncs::DescrGet;
	PyType.tp_getattro = (getattrofunc)&PyObject_GenericGetAttr;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_HAVE_GC;

	InitializePyMethodWithClosureDescrTypeMembers(PyType);
	InitializePyMethodWithClosureDescrTypeGetSet(PyType);

	return PyType;
}

PyTypeObject InitializePyClassMethodWithClosureDescrType()
{
	struct FFuncs
	{
		static void Dealloc(FPyMethodWithClosureDescrObject* InSelf)
		{
			FPyMethodWithClosureDescrObject::Free(InSelf);
		}

		static int Traverse(FPyMethodWithClosureDescrObject* InSelf, visitproc InVisit, void* InArg)
		{
			return FPyMethodWithClosureDescrObject::GCTraverse(InSelf, InVisit, InArg);
		}

		static int Clear(FPyMethodWithClosureDescrObject* InSelf)
		{
			return FPyMethodWithClosureDescrObject::GCClear(InSelf);
		}

		static PyObject* Str(FPyMethodWithClosureDescrObject* InSelf)
		{
			return FPyMethodWithClosureDescrObject::Str(InSelf);
		}

		static PyObject* Call(FPyMethodWithClosureDescrObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			check(PyTuple_Check(InArgs));

			Py_ssize_t ArgCount = PyTuple_GET_SIZE(InArgs);
			if (ArgCount < 1)
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' of '%s' object needs an argument", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name);
				return nullptr;
			}

			PyObject* Self = PyTuple_GET_ITEM(InArgs, 0);
			if (!PyType_Check(Self))
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' requires a type but received a '%s'", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), Self->ob_type->tp_name);
				return nullptr;
			}

			if (!PyType_IsSubtype((PyTypeObject*)Self, InSelf->OwnerType))
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' requires a subtype of '%s' but received '%s", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name, Self->ob_type->tp_name);
				return nullptr;
			}

			FPyObjectPtr Args = FPyObjectPtr::StealReference(PyTuple_GetSlice(InArgs, 1, ArgCount));
			return FPyMethodWithClosureDef::Call(InSelf->MethodDef, Self, Args, InKwds);
		}

		static PyObject* DescrGet(FPyMethodWithClosureDescrObject* InSelf, PyObject* InObj, PyObject* InType)
		{
			PyObject* Type = InType;
			if (!Type && InObj)
			{
				Type = (PyObject*)InObj->ob_type;
			}

			if (!Type)
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' for type '%s' needs either an object or a type", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name);
				return nullptr;
			}

			if (!PyType_Check(Type))
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' for type '%s' needs a type, not a '%s' as arg 2", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name, Type->ob_type->tp_name);
				return nullptr;
			}

			if (!PyType_IsSubtype((PyTypeObject*)Type, InSelf->OwnerType))
			{
				PyErr_Format(PyExc_TypeError, "descriptor '%s' for type '%s' doesn't apply to type '%s'", TCHAR_TO_UTF8(*FPyMethodWithClosureDescrObject::GetMethodName(InSelf)), InSelf->OwnerType->tp_name, ((PyTypeObject*)Type)->tp_name);
				return nullptr;
			}

			return (PyObject*)FPyCFunctionWithClosureObject::New(InSelf->MethodDef, Type, nullptr);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"classmethodwithclosure_descriptor", /* tp_name */
		sizeof(FPyMethodWithClosureDescrObject), /* tp_basicsize */
	};

	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_traverse = (traverseproc)&FFuncs::Traverse;
	PyType.tp_clear = (inquiry)&FFuncs::Clear;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_call = (ternaryfunc)&FFuncs::Call;
	PyType.tp_descr_get = (descrgetfunc)&FFuncs::DescrGet;
	PyType.tp_getattro = (getattrofunc)&PyObject_GenericGetAttr;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_HAVE_GC;

	InitializePyMethodWithClosureDescrTypeMembers(PyType);
	InitializePyMethodWithClosureDescrTypeGetSet(PyType);

	return PyType;
}

PyTypeObject PyCFunctionWithClosureType = InitializePyCFunctionWithClosureType();
PyTypeObject PyMethodWithClosureDescrType = InitializePyMethodWithClosureDescrType();
PyTypeObject PyClassMethodWithClosureDescrType = InitializePyClassMethodWithClosureDescrType();

#endif	// WITH_PYTHON
