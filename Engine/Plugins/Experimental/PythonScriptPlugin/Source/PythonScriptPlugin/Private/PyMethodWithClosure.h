// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "CoreMinimal.h"

#if WITH_PYTHON

/** Python type for FPyCFunctionWithClosureObject */
extern PyTypeObject PyCFunctionWithClosureType;

/** Python type for FPyMethodWithClosureDescrObject */
extern PyTypeObject PyMethodWithClosureDescrType;

/** Python type for FPyMethodWithClosureDescrObject (for class methods) */
extern PyTypeObject PyClassMethodWithClosureDescrType;

/** Initialize the PyMethodWithClosure types */
bool InitializePyMethodWithClosure();

/** Shutdown the PyMethodWithClosure types */
void ShutdownPyMethodWithClosure();

/** C function pointers used by FPyMethodWithClosureDef */
typedef PyObject*(*PyCFunctionWithClosure)(PyObject*, PyObject*, void*);
typedef PyObject*(*PyCFunctionWithClosureAndKeywords)(PyObject*, PyObject*, PyObject*, void*);
typedef PyObject*(*PyCFunctionWithClosureAndNoArgs)(PyObject*, void*);

/** Cast a function pointer to PyCFunctionWithClosure (via a void* to avoid a compiler warning) */
#define PyCFunctionWithClosureCast(FUNCPTR) (PyCFunctionWithClosure)(void*)(FUNCPTR)

/** Definition for a method with a closure (equivalent to PyMethodDef) */
struct FPyMethodWithClosureDef
{
	/* The name of the method */
	const char* MethodName;

	/* The C function this method should call */
	PyCFunctionWithClosure MethodCallback;

	/* Combination of METH_ flags describing how the C function should be called */
	int MethodFlags;

	/* The method documentation, or nullptr */
	const char* MethodDoc;

	/** The closure passed to the C function, or nullptr */
	void* MethodClosure;

	/** Add a singular method to the given type */
	static bool AddMethod(FPyMethodWithClosureDef* InMethod, PyTypeObject* InType);

	/** Add the given null-terminated table of methods to the given type */
	static bool AddMethods(FPyMethodWithClosureDef* InMethods, PyTypeObject* InType);

	/** Create a method descriptor for the given definition */
	static PyObject* NewMethodDescriptor(PyTypeObject* InType, FPyMethodWithClosureDef* InDef);

	/** Call the method with the given self and arguments */
	static PyObject* Call(FPyMethodWithClosureDef* InDef, PyObject* InSelf, PyObject* InArgs, PyObject* InKwds);
};

/** Python object for the bound callable form of method with a closure (equivalent to PyCFunctionObject) */
struct FPyCFunctionWithClosureObject
{
	/** Common Python Object */
	PyObject_HEAD
	
	/* Definition of the method to call */
	FPyMethodWithClosureDef* MethodDef;
	
	/* The 'self' argument passed to the C function, or nullptr */
    PyObject* SelfArg;
	
	/* The __module__ attribute, can be anything */
    PyObject* ModuleAttr;

	/** New this instance */
	static FPyCFunctionWithClosureObject* New(FPyMethodWithClosureDef* InMethod, PyObject* InSelf, PyObject* InModule);

	/** Free this instance */
	static void Free(FPyCFunctionWithClosureObject* InSelf);

	/** Handle a Python GC traverse */
	static int GCTraverse(FPyCFunctionWithClosureObject* InSelf, visitproc InVisit, void* InArg);
	
	/** Handle a Python GC clear */
	static int GCClear(FPyCFunctionWithClosureObject* InSelf);
	
	/** Get a string representing this instance */
	static PyObject* Str(FPyCFunctionWithClosureObject* InSelf);
};

/** Python object for the descriptor of a method with a closure (equivalent to PyMethodDescrObject) */
struct FPyMethodWithClosureDescrObject
{
	/** Common Python Object */
	PyObject_HEAD

	/** Type of the owner object */
	PyTypeObject* OwnerType;

	/** Name of the method being described */
	PyObject* MethodName;

	/* Definition of the method to call */
	FPyMethodWithClosureDef* MethodDef;

	/** New a method */
	static FPyMethodWithClosureDescrObject* NewMethod(PyTypeObject* InType, FPyMethodWithClosureDef* InMethod);

	/** New a class method */
	static FPyMethodWithClosureDescrObject* NewClassMethod(PyTypeObject* InType, FPyMethodWithClosureDef* InMethod);

	/** New this instance */
	static FPyMethodWithClosureDescrObject* NewImpl(PyTypeObject* InDescrType, PyTypeObject* InType, FPyMethodWithClosureDef* InMethod);

	/** Free this instance */
	static void Free(FPyMethodWithClosureDescrObject* InSelf);

	/** Handle a Python GC traverse */
	static int GCTraverse(FPyMethodWithClosureDescrObject* InSelf, visitproc InVisit, void* InArg);

	/** Handle a Python GC clear */
	static int GCClear(FPyMethodWithClosureDescrObject* InSelf);

	/** Get a string representing this instance */
	static PyObject* Str(FPyMethodWithClosureDescrObject* InSelf);

	/** Safely get the name of the method (if known) */
	static FString GetMethodName(FPyMethodWithClosureDescrObject* InSelf);
};

#endif	// WITH_PYTHON
