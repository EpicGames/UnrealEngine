// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyOnlineDocsWriter.h"
#include "IPythonScriptPlugin.h"
#include "PyUtil.h"
#include "PyGenUtil.h"
#include "HAL/FileManager.h"
#include "Logging/MessageLog.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Runtime/Launch/Resources/Version.h"

#if WITH_PYTHON

class FPyDeleteUnreferencedFilesVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	void ReferenceFile(const FString& InFilename)
	{
		ReferencedFiles.Add(FPaths::ConvertRelativePathToFull(InFilename));
	}

	bool IsReferencedFile(const FString& InFilename) const
	{
		return ReferencedFiles.Contains(FPaths::ConvertRelativePathToFull(InFilename));
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		if (!bIsDirectory && !IsReferencedFile(FilenameOrDirectory))
		{
			IFileManager::Get().Delete(FilenameOrDirectory, false, true, true);
		}

		return true;
	}

private:
	/** Set of referenced files (absolute paths) */
	TSet<FString> ReferencedFiles;
};


void FPyOnlineDocsModule::AccumulateFunction(const TCHAR* InFunctionName)
{
	FunctionNames.Add(InFunctionName);
}


void FPyOnlineDocsSection::AccumulateClass(const TCHAR* InTypeName)
{
	TypeNames.Add(InTypeName);
}


TSharedRef<FPyOnlineDocsModule> FPyOnlineDocsWriter::CreateModule(const FString& InModuleName)
{
	return Modules.Add_GetRef(MakeShared<FPyOnlineDocsModule>(InModuleName));
}

TSharedRef<FPyOnlineDocsSection> FPyOnlineDocsWriter::CreateSection(const FString& InSectionName)
{
	return Sections.Add_GetRef(MakeShared<FPyOnlineDocsSection>(InSectionName));
}

FString FPyOnlineDocsWriter::GetSphinxDocsPath() const
{
	return FPaths::ConvertRelativePathToFull(FPaths::EnginePluginsDir()) / TEXT("Experimental/PythonScriptPlugin/SphinxDocs");
}

FString FPyOnlineDocsWriter::GetSourcePath() const
{
	return GetSphinxDocsPath() / TEXT("source");
}

FString FPyOnlineDocsWriter::GetBuildPath() const
{
	return GetSphinxDocsPath() / TEXT("build");
}

FString FPyOnlineDocsWriter::GetTemplatePath() const
{
	return GetSourcePath() / TEXT("_templates");
}

void FPyOnlineDocsWriter::GenerateConfigFile()
{
	// Load up conf.py template
	const FString ConfigTemplatePath = GetTemplatePath() / TEXT("conf.py");
	FString ConfigTemplate;
	if (!FFileHelper::LoadFileToString(ConfigTemplate, *ConfigTemplatePath))
	{
		UE_LOGF(LogPython, Warning, "Documentation generation template file '%ls' failed to load!", *ConfigTemplatePath);
		return;
	}

	// Replace {{Version}} with the actual version number
	FString ConfigText = ConfigTemplate;
	ConfigText.ReplaceInline(TEXT("{{Version}}"), VERSION_STRINGIFY(ENGINE_MAJOR_VERSION) TEXT(".") VERSION_STRINGIFY(ENGINE_MINOR_VERSION), ESearchCase::CaseSensitive);

	// Save out config file
	const FString ConfigPath = GetSourcePath() / TEXT("conf.py");
	if (!PyGenUtil::SaveGeneratedTextFile(*ConfigPath, ConfigText))
	{
		UE_LOGF(LogPython, Warning, "Documentation generation file '%ls' failed to write!", *ConfigPath);
	}
}

void FPyOnlineDocsWriter::GenerateIndexFile()
{
	// Load up index.rst template
	const FString IndexTemplatePath = GetTemplatePath() / TEXT("index.rst");
	FString IndexTemplate;
	if (!FFileHelper::LoadFileToString(IndexTemplate, *IndexTemplatePath))
	{
		UE_LOGF(LogPython, Warning, "Documentation generation template file '%ls' failed to load!", *IndexTemplatePath);
		return;
	}

	// Sort the items in each module and keep the modules in the order they were created
	for (const TSharedRef<FPyOnlineDocsModule>& Module : Modules)
	{
		Module->FunctionNames.StableSort();
	}

	// Sort the items in each section and keep the sections in the order they were created
	for (const TSharedRef<FPyOnlineDocsSection>& Section : Sections)
	{
		Section->TypeNames.StableSort();
	}

	FString SectionList;
	FString TableOfContents;

	// Accumulate all the modules into the table of contents
	if (Modules.Num() > 0)
	{
		SectionList += TEXT("* :ref:`Modules`") LINE_TERMINATOR;

		TableOfContents +=
			TEXT(".. _Modules:") LINE_TERMINATOR
			TEXT(".. toctree::") LINE_TERMINATOR
			TEXT("    :maxdepth: 1") LINE_TERMINATOR
			TEXT("    :caption: Modules") LINE_TERMINATOR LINE_TERMINATOR;

		for (const TSharedRef<FPyOnlineDocsModule>& Module : Modules)
		{
			TableOfContents += FString::Printf(TEXT("    module/%s") LINE_TERMINATOR, *Module->Name);
		}
	}

	// Accumulate all the classes for each section into the table of contents
	for (const TSharedRef<FPyOnlineDocsSection>& Section : Sections)
	{
		FString SectionRef(Section->Name);
		SectionRef.ReplaceInline(TEXT(" "), TEXT("-"));

		SectionList += TEXT("* :ref:`");
		SectionList += SectionRef;
		SectionList += TEXT("`") LINE_TERMINATOR;

		TableOfContents +=
			LINE_TERMINATOR
			TEXT(".. _");
		TableOfContents += SectionRef;
		TableOfContents +=
			TEXT(":") LINE_TERMINATOR
			TEXT(".. toctree::") LINE_TERMINATOR
			TEXT("    :maxdepth: 1") LINE_TERMINATOR
			TEXT("    :caption: ");
		TableOfContents += Section->Name;
		TableOfContents += LINE_TERMINATOR LINE_TERMINATOR;

		for (const FString& TypeName : Section->TypeNames)
		{
			TableOfContents += FString::Printf(TEXT("    class/%s") LINE_TERMINATOR, *TypeName);
		}
	}

	// Replace {{SectionList}} with actual list
	FString IndexText = IndexTemplate;
	IndexText.ReplaceInline(TEXT("{{SectionList}}"), *SectionList, ESearchCase::CaseSensitive);

	// Replace {{TableOfContents}} with actual list
	IndexText.ReplaceInline(TEXT("{{TableOfContents}}"), *TableOfContents, ESearchCase::CaseSensitive);

	// Save out index file
	const FString IndexPath = GetSourcePath() / TEXT("index.rst");
	if (!PyGenUtil::SaveGeneratedTextFile(*IndexPath, IndexText))
	{
		UE_LOGF(LogPython, Warning, "Documentation generation file '%ls' failed to write!", *IndexPath);
	}
}

void FPyOnlineDocsWriter::GenerateModuleFiles()
{
	// Load up Module.rst template
	const FString ModuleTemplatePath = GetTemplatePath() / TEXT("Module.rst");
	FString ModuleTemplate;
	if (!FFileHelper::LoadFileToString(ModuleTemplate, *ModuleTemplatePath))
	{
		UE_LOGF(LogPython, Warning, "Documentation generation template file '%ls' failed to load!", *ModuleTemplatePath);
		return;
	}

	// Keep track of referenced files so we can delete any stale ones
	FPyDeleteUnreferencedFilesVisitor DeleteUnreferencedFilesVisitor;

	// Create page for each module
	const FString ModuleSourcePath = GetSourcePath() / TEXT("module");
	for (const TSharedRef<FPyOnlineDocsModule>& Module : Modules)
	{
		FString ModuleFunctions;
		for (const FString& FunctionName : Module->FunctionNames)
		{
			ModuleFunctions += FString::Printf(TEXT(".. autofunction:: unreal.%s") LINE_TERMINATOR, *FunctionName);
		}

		// Replace {{Module}} with actual module name
		// Replace {{ModuleFunctions}} with actual module functions list
		FString ModuleText = ModuleTemplate;
		ModuleText.ReplaceInline(TEXT("{{Module}}"), *Module->Name, ESearchCase::CaseSensitive);
		ModuleText.ReplaceInline(TEXT("{{ModuleFunctions}}"), *ModuleFunctions, ESearchCase::CaseSensitive);

		// Write out module file
		const FString ModulePath = ModuleSourcePath / Module->Name + TEXT(".rst");
		if (PyGenUtil::SaveGeneratedTextFile(*ModulePath, ModuleText))
		{
			DeleteUnreferencedFilesVisitor.ReferenceFile(ModulePath);
		}
		else
		{
			UE_LOGF(LogPython, Warning, "Documentation generation file '%ls' failed to write!", *ModulePath);
		}
	}

	// Remove any stale files
	IFileManager::Get().IterateDirectory(*ModuleSourcePath, DeleteUnreferencedFilesVisitor);
}

void FPyOnlineDocsWriter::GenerateClassFiles()
{
	// Load up Class.rst template
	const FString ClassTemplatePath = GetTemplatePath() / TEXT("Class.rst");
	FString ClassTemplate;
	if (!FFileHelper::LoadFileToString(ClassTemplate, *ClassTemplatePath))
	{
		UE_LOGF(LogPython, Warning, "Documentation generation template file '%ls' failed to load!", *ClassTemplatePath);
		return;
	}

	// Keep track of referenced files so we can delete any stale ones
	FPyDeleteUnreferencedFilesVisitor DeleteUnreferencedFilesVisitor;

	// Create page for each class in each section
	const FString ClassSourcePath = GetSourcePath() / TEXT("class");
	for (const TSharedRef<FPyOnlineDocsSection>& Section : Sections)
	{
		for (const FString& TypeName : Section->TypeNames)
		{
			// Replace {{Class}} with actual class name
			FString ClassText = ClassTemplate;
			ClassText.ReplaceInline(TEXT("{{Class}}"), *TypeName, ESearchCase::CaseSensitive);

			// Write out class file
			const FString ClassPath = ClassSourcePath / TypeName + TEXT(".rst");
			if (PyGenUtil::SaveGeneratedTextFile(*ClassPath, ClassText))
			{
				DeleteUnreferencedFilesVisitor.ReferenceFile(*ClassPath);
			}
			else
			{
				UE_LOGF(LogPython, Warning, "Documentation generation file '%ls' failed to write!", *ClassPath);
			}
		}
	}

	// Remove any stale files
	IFileManager::Get().IterateDirectory(*ClassSourcePath, DeleteUnreferencedFilesVisitor);
}

void FPyOnlineDocsWriter::GenerateFiles(const FString& InPythonStubPath)
{
	UE_LOGF(LogPython, Display, "Generating Python API online docs used by Sphinx to generate static HTML...");

	const FString PythonStubDestPath = GetSphinxDocsPath() / TEXT("modules") / FPaths::GetCleanFilename(InPythonStubPath);
	
	// Copy generated unreal module stub file to PythonScriptPlugin/SphinxDocs/modules
	{
		FString SourceFileContents;
		if (FFileHelper::LoadFileToString(SourceFileContents, *InPythonStubPath))
		{
			if (!PyGenUtil::SaveGeneratedTextFile(*PythonStubDestPath, SourceFileContents))
			{
				UE_LOGF(LogPython, Warning, "Documentation generation file '%ls' failed to write!", *PythonStubDestPath);
			}
		}
		else
		{
			UE_LOGF(LogPython, Warning, "Documentation generation file '%ls' failed to load!", *InPythonStubPath);
		}
	}

	GenerateConfigFile();
	GenerateIndexFile();
	GenerateModuleFiles();
	GenerateClassFiles();

	UE_LOGF(LogPython, Display, "  ... finished generating Sphinx files.");

	bool bIsEnginePython = false;
	const FString PythonPath = PyUtil::GetInterpreterExecutablePath(&bIsEnginePython);
	
	// We only use Sphinx on the version embedded in UE rather than as an executed external 
	// process since other installs, paths and environment variables may act in unexpected ways.
	FString Commandline = FCommandLine::Get();
	bool bUseSphinx = bIsEnginePython && Commandline.Contains(TEXT("-NoHTML")) == false;

	// Validate the unreal generated stub file before we let Sphinx try and process it
	{
		FString PyCommandStr;

		// We do this as a subprocess call to the Python interpreter to avoid the generated stub file from breaking the log redirection that the non-stub unreal module configures
		PyCommandStr.Append(TEXT("import subprocess\n"));
		PyCommandStr.Appendf(TEXT("subprocess.check_call(['%s', '%s'])\n"), *PythonPath, *PythonStubDestPath);

		UE_LOGF(LogPython, Display, "Validating the unreal generated stub file...");
		if (!IPythonScriptPlugin::Get()->ExecPythonCommand(*PyCommandStr))
		{
			UE_LOGF(LogPython, Error, "The unreal generated stub file failed to import. Aborting!");
			return;
		}
		UE_LOGF(LogPython, Display, "  ... finished validating file.");
	}

	if (bUseSphinx)
	{
		// Call Sphinx to generate online Python API docs. (Default)

		// Could potentially use Python C++ API calls rather than Python scripts, though this keeps it clear 
		// and if the calls evolve over time the vast number of examples online are in Python rather than C++.

		// Update pip and then install Sphinx if needed. If Sphinx and its dependencies are already
		// installed then it will determine that quickly and move on to using it.
		// More info on using pip within Python here:
		//   https://pip.pypa.io/en/stable/user_guide/#using-pip-from-your-program

		// Update pip and install Sphinx
		{
			FString PyCommandStr;

			PyCommandStr.Append(TEXT("import subprocess\n"));
			PyCommandStr.Appendf(TEXT("subprocess.call(['%s', '-m', 'pip', 'install', '-q', '-U', 'pip'])\n"), *PythonPath);	// This may fail if UE is read-only, and on failure somehow completely breaks pip...
			PyCommandStr.Appendf(TEXT("subprocess.check_call(['%s', '-m', 'ensurepip'])\n"), *PythonPath);						// ... so this call ensures that pip is definitely available, even if the above call broke it
			PyCommandStr.Appendf(TEXT("subprocess.check_call(['%s', '-m', 'pip', 'install', '-q', '--no-warn-script-location', 'sphinx'])\n"), *PythonPath);

			// Alternate technique calling pip as a Python command though above is recommended by pip.
			//PyCommandStr.Append(TEXT("import pip\n"));
			//PyCommandStr.Append(TEXT("pip.main(['install', 'sphinx'])\n"));

			UE_LOGF(LogPython, Display, "Installing Sphinx...");
			IPythonScriptPlugin::Get()->ExecPythonCommand(*PyCommandStr);
			UE_LOGF(LogPython, Display, "  ... finished installing Sphinx.");
		}

		// Run Sphinx to build the docs
		{
			FString PyCommandStr;

			// Need to remove the C++ generated module so that Sphinx can import the generated stub file instead
			PyCommandStr.Append(TEXT("import sys\n"));
			PyCommandStr.Append(TEXT("del unreal\n"));
			PyCommandStr.Append(TEXT("del sys.modules['unreal']\n"));

			PyCommandStr.Append(TEXT("import sphinx.cmd.build\n"));
			PyCommandStr.Appendf(TEXT("sphinx.cmd.build.build_main(['-b', 'html', '%s', '%s'])\n"), *GetSourcePath(), *GetBuildPath());

#if !NO_LOGGING
			bool bLogSphinx = Commandline.Contains(TEXT("-HTMLLog"));
			ELogVerbosity::Type OldVerbosity = LogPython.GetVerbosity();

			if (!bLogSphinx)
			{
				// Disable Python logging (default)
				LogPython.SetVerbosity(ELogVerbosity::NoLogging);
			}
#endif // !NO_LOGGING

			// Run the Python commands
			UE_LOGF(LogPython, Display, "Calling Sphinx in PythonPlugin/SphinxDocs to generate the HTML. This can take a long time - 25+ minutes for full build on test system...");
			bool PyRunSuccess = IPythonScriptPlugin::Get()->ExecPythonCommand(*PyCommandStr);

#if !NO_LOGGING
			if (!bLogSphinx)
			{
				// Re-enable Python logging
				LogPython.SetVerbosity(OldVerbosity);
			}
#endif // !NO_LOGGING

			// The running of the Python commands seem to think there are errors no matter what so not much use to make this notification.
			//if (PyRunSuccess)
			//{
			//	UE_LOG(LogPython, Display, TEXT(
			//		"\n"
			//		"  There was an internal Python error while generating the docs, though it may not be an issue.\n\n"
			//		"  You could call the commandlet again with the '-HTMLLog' option to see more information.\n"));
			//}

			UE_LOGF(LogPython, Display, "  ... finished generating Python API online docs!\n\n"
				"Find them in the following directory:\n"
				"  %ls\n\n"
				"See additional instructions and information in:\n"
				"  PythonScriptPlugin/SphinxDocs/PythonAPI_docs_readme.txt\n",
				*GetBuildPath());
		}
	}
	else
	{
		// Prompt to manually call Sphinx to generate online Python API docs.
		UE_LOGF(LogPython, Display, "To build the Python API online docs manually follow the instructions in:\n"
			"  PythonScriptPlugin/SphinxDocs/PythonAPI_docs_readme.txt\n\n"
			"And then call:"
			"  PythonScriptPlugin/SphinxDocs/sphinx-build -b html source/ build/");
	}
}

#endif	// WITH_PYTHON
