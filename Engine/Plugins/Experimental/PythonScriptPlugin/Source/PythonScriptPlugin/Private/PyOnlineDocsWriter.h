// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumClassFlags.h"

#if WITH_PYTHON

/**
 * Flags controlling which data is included in the Python API online docs.
 */
enum class EPyOnlineDocsFilterFlags : uint8
{
	IncludeNone = 0,
	IncludeEngine = 1<<0,
	IncludeEnterprise UE_DEPRECATED(5.6, "Enterprise is now considered a part of engine, please update usage to the engine enum flag.") = IncludeEngine,
	IncludeInternal = 1<<2,
	IncludeProject = 1<<3,
	IncludeAll = IncludeEngine | IncludeInternal | IncludeProject,
};
ENUM_CLASS_FLAGS(EPyOnlineDocsFilterFlags);

/**
 * A single module in the Python API online docs.
 * Hosts a series of function names that belong to this module and will be used for indexing purposes later.
 */
class FPyOnlineDocsModule
{
	friend class FPyOnlineDocsWriter;

public:
	FPyOnlineDocsModule(const FString& InModuleName)
		: Name(InModuleName)
	{
	}

	/** Store function name in this module to generate files later. */
	void AccumulateFunction(const TCHAR* InFunctionName);

private:
	/** Module name. */
	FString Name;

	/** Accumulated Python function names to write out to API docs. */
	TArray<FString> FunctionNames;
};

/**
 * A single section in the Python API online docs.
 * Hosts a series of type names that belong to this section and will be used for indexing purposes later.
 */
class FPyOnlineDocsSection
{
	friend class FPyOnlineDocsWriter;

public:
	FPyOnlineDocsSection(const FString& InSectionName)
		: Name(InSectionName)
	{
	}

	/** Store class name in this section to generate files later. */
	void AccumulateClass(const TCHAR* InTypeName);

	//** Get name of section. */
	const FString& GetName() const	{ return Name; }

private:
	/** Section name. */
	FString Name;

	/** Accumulated Python type names to write out to API docs. */
	TArray<FString> TypeNames;
};

/**
 * Utility class to help format and write Python API online docs in reStructuredText files
 * used by Sphinx to generate static HTML.
 * See PythonScriptPlugin\SphinxDocs\PythonAPI_docs_readme.txt for additional info.
 */
class FPyOnlineDocsWriter
{
public:
	/** Add a new module. */
	TSharedRef<FPyOnlineDocsModule> CreateModule(const FString& InModuleName);

	/** Add a new section. */
	TSharedRef<FPyOnlineDocsSection> CreateSection(const FString& InSectionName);

	/** Get the directory for the Sphinx files. */
	FString GetSphinxDocsPath() const;

	/** Get the directory for the Sphinx source files. */
	FString GetSourcePath() const;

	/** Get the directory for the Sphinx build files. */
	FString GetBuildPath() const;

	/** Get the directory for the Sphinx template files. */
	FString GetTemplatePath() const;

	/** Create Python config for Sphinx based on template. */
	void GenerateConfigFile();

	/** Create index reStructuredText file for Sphinx based on template. */
	void GenerateIndexFile();

	/** Create reStructuredText module files for Sphinx based on template. */
	void GenerateModuleFiles();

	/** Create reStructuredText class files for Sphinx based on template. */
	void GenerateClassFiles();

	/** Create reStructuredText files for Sphinx. */
	void GenerateFiles(const FString& InPythonStubPath);

private:
	/** API doc modules. */
	TArray<TSharedRef<FPyOnlineDocsModule>> Modules;

	/** API doc sections. */
	TArray<TSharedRef<FPyOnlineDocsSection>> Sections;
};

#endif	// WITH_PYTHON
