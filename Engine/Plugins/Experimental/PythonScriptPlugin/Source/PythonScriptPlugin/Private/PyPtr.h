// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "Misc/AssertionMacros.h"

#if WITH_PYTHON

/** Wrapper that handles the ref-counting of the contained Python object */
template <typename TPythonType>
class TPyPtr
{
public:
	/** Create a null pointer */
	TPyPtr()
		: Ptr(nullptr)
	{
	}

	/** Copy a pointer */
	TPyPtr(const TPyPtr& InOther)
		: Ptr(InOther.Ptr)
	{
		Py_XINCREF(Ptr);
	}

	/** Copy a pointer */
	template <typename TOtherPythonType>
	TPyPtr(const TPyPtr<TOtherPythonType>& InOther)
		: Ptr(InOther.Ptr)
	{
		Py_XINCREF(Ptr);
	}

	/** Move a pointer */
	TPyPtr(TPyPtr&& InOther)
		: Ptr(InOther.Ptr)
	{
		InOther.Ptr = nullptr;
	}

	/** Move a pointer */
	template <typename TOtherPythonType>
	TPyPtr(TPyPtr<TOtherPythonType>&& InOther)
		: Ptr(InOther.Ptr)
	{
		InOther.Ptr = nullptr;
	}

	/** Release our reference to this object */
	~TPyPtr()
	{
		Py_XDECREF(Ptr);
	}

	/** Create a pointer from the given object, incrementing its reference count */
	static TPyPtr NewReference(TPythonType* InPtr)
	{
		return TPyPtr(InPtr, true);
	}

	/** Create a pointer from the given object, leaving its reference count alone */
	static TPyPtr StealReference(TPythonType* InPtr)
	{
		return TPyPtr(InPtr, false);
	}

	/** Copy a pointer */
	TPyPtr& operator=(const TPyPtr& InOther)
	{
		if (this != &InOther)
		{
			Py_XDECREF(Ptr);
			Ptr = InOther.Ptr;
			Py_XINCREF(Ptr);
		}
		return *this;
	}

	/** Copy a pointer */
	template <typename TOtherPythonType>
	TPyPtr& operator=(const TPyPtr<TOtherPythonType>& InOther)
	{
		if (this != &InOther)
		{
			Py_XDECREF(Ptr);
			Ptr = InOther.Ptr;
			Py_XINCREF(Ptr);
		}
		return *this;
	}

	/** Move a pointer */
	TPyPtr& operator=(TPyPtr&& InOther)
	{
		if (this != &InOther)
		{
			Py_XDECREF(Ptr);
			Ptr = InOther.Ptr;
			InOther.Ptr = nullptr;
		}
		return *this;
	}

	/** Move a pointer */
	template <typename TOtherPythonType>
	TPyPtr& operator=(TPyPtr<TOtherPythonType>&& InOther)
	{
		if (this != &InOther)
		{
			Py_XDECREF(Ptr);
			Ptr = InOther.Ptr;
			InOther.Ptr = nullptr;
		}
		return *this;
	}

	explicit operator bool() const
	{
		return Ptr != nullptr;
	}

	bool IsValid() const
	{
		return Ptr != nullptr;
	}

	operator TPythonType*() const
	{
		return Ptr;
	}

	TPythonType& operator*()
	{
		check(Ptr);
		return *Ptr;
	}

	const TPythonType& operator*() const
	{
		check(Ptr);
		return *Ptr;
	}

	TPythonType* operator->()
	{
		check(Ptr);
		return Ptr;
	}

	const TPythonType* operator->() const
	{
		check(Ptr);
		return Ptr;
	}

	TPythonType*& Get()
	{
		return Ptr;
	}

	TPythonType* GetPtr() const
	{
		return Ptr;
	}

	TPythonType* Release()
	{
		TPythonType* RetPtr = Ptr;
		Ptr = nullptr;
		return RetPtr;
	}

	void Reset()
	{
		Py_XDECREF(Ptr);
		Ptr = nullptr;
	}

private:
	TPyPtr(TPythonType* InPtr, const bool bIncRef)
		: Ptr(InPtr)
	{
		if (bIncRef)
		{
			Py_XINCREF(Ptr);
		}
	}

	TPythonType* Ptr;
};

typedef TPyPtr<PyObject> FPyObjectPtr;
typedef TPyPtr<PyTypeObject> FPyTypeObjectPtr;

#endif	// WITH_PYTHON
