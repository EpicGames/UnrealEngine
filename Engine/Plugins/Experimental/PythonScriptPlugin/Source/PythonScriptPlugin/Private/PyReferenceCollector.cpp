// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyReferenceCollector.h"
#include "PyWrapperTypeRegistry.h"
#include "PyWrapperBase.h"
#include "PyWrapperObject.h"
#include "PyWrapperStruct.h"
#include "PyWrapperEnum.h"
#include "PyWrapperDelegate.h"
#include "PyGIL.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PurgingReferenceCollector.h"

#if WITH_EDITOR
#include "CoreGlobals.h"
#include "ObjectTools.h"
#endif

#if WITH_PYTHON

FPyReferenceCollector& FPyReferenceCollector::Get()
{
	static FPyReferenceCollector Instance;
	return Instance;
}

void FPyReferenceCollector::AddWrappedInstance(FPyWrapperBase* InInstance)
{
	PythonWrappedInstances.Add(InInstance);
}

void FPyReferenceCollector::RemoveWrappedInstance(FPyWrapperBase* InInstance)
{
	PythonWrappedInstances.Remove(InInstance);
}

void FPyReferenceCollector::AddReferencedObjects(FReferenceCollector& InCollector)
{
	if (!PythonWrappedInstances.IsEmpty())
	{
		FPyScopedGIL GIL;
		for (FPyWrapperBase* PythonWrappedInstance : PythonWrappedInstances)
		{
			FPyWrapperBaseMetaData* PythonWrappedInstanceMetaData = FPyWrapperBaseMetaData::GetMetaData(PythonWrappedInstance);
			if (PythonWrappedInstanceMetaData)
			{
				PythonWrappedInstanceMetaData->AddInstanceReferencedObjects(PythonWrappedInstance, InCollector);
			}
		}
	}
}

FString FPyReferenceCollector::GetReferencerName() const
{
	return TEXT("FPyReferenceCollector");
}

void FPyReferenceCollector::PurgeUnrealObjectReferences(const UObject* InObject, const bool bIncludeInnerObjects)
{
	PurgeUnrealObjectReferences(TArrayView<const UObject*>(&InObject, 1), bIncludeInnerObjects);
}

void FPyReferenceCollector::PurgeUnrealObjectReferences(const TArrayView<const UObject*>& InObjects, const bool bIncludeInnerObjects)
{
	FPurgingReferenceCollector PurgingReferenceCollector;

	for (const UObject* Object : InObjects)
	{
		PurgingReferenceCollector.AddObjectToPurge(Object);

		if (bIncludeInnerObjects)
		{
			ForEachObjectWithOuter(Object, [&PurgingReferenceCollector](UObject* InnerObject)
			{
				PurgingReferenceCollector.AddObjectToPurge(InnerObject);
			}, EGetObjectsFlags::IncludeNestedObjects);
		}
	}

	if (PurgingReferenceCollector.HasObjectToPurge())
	{
		AddReferencedObjects(PurgingReferenceCollector);
	}
}

void FPyReferenceCollector::PurgeUnrealGeneratedTypes()
{
#if WITH_EDITOR
	// Only do the reference cleaning when the editor is not shuthing down to avoid wasting time.
	if (!IsEngineExitRequested())
	{
		// Preclean to only grab the object that are refered by other systems
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		TArray<UObject*> PythonObjectInstances;
		ForEachObjectOfClass(UPythonGeneratedClass::StaticClass(), [&PythonObjectInstances](UObject* InObject)
			{
				UPythonGeneratedClass* PythonGeneratedClass = CastChecked<UPythonGeneratedClass>(InObject);
				ForEachObjectOfClass(PythonGeneratedClass, [&PythonObjectInstances](UObject* InInnerObject)
					{
						PythonObjectInstances.Add(InInnerObject);
					}, false, RF_NoFlags, EInternalObjectFlags::Native);
			}, false, RF_ClassDefaultObject, EInternalObjectFlags::Native);

		ObjectTools::ForceReplaceReferences(nullptr, PythonObjectInstances);
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
#endif //WITH_EDITOR

	FPurgingReferenceCollector PurgingReferenceCollector;
	TArray<FWeakObjectPtr> WeakReferencesToPurgedObjects;

	TMap<UObject*, UObjectRedirector*> ReverseRedirectorMapping;
	ForEachObjectOfClass(UObjectRedirector::StaticClass(), [&ReverseRedirectorMapping](UObject* InObject)
	{
		UObjectRedirector* Redirector = CastChecked<UObjectRedirector>(InObject);
		if (Redirector->DestinationObject)
		{
			ReverseRedirectorMapping.Add(Redirector->DestinationObject, Redirector);
		}
	});

	auto FlagObjectForPurge = [&PurgingReferenceCollector, &WeakReferencesToPurgedObjects, &ReverseRedirectorMapping](UObject* InObject, const bool bMarkPendingKill)
	{
		check(!InObject->HasAnyInternalFlags(EInternalObjectFlags::Native));

		if (InObject->IsRooted())
		{
			InObject->RemoveFromRoot();
		}
		InObject->ClearFlags(RF_Public | RF_Standalone);

		if (!InObject->HasAnyFlags(RF_ClassDefaultObject))
		{
			if (bMarkPendingKill)
			{
				InObject->MarkAsGarbage();
			}
			WeakReferencesToPurgedObjects.Add(InObject);
		}

		if (UObjectRedirector* Redirector = ReverseRedirectorMapping.FindRef(InObject))
		{
			check(Redirector->DestinationObject == InObject);
			Redirector->DestinationObject = nullptr;
			Redirector->ClearFlags(RF_Public | RF_Standalone);
		}

		PurgingReferenceCollector.AddObjectToPurge(InObject);
	};

	// Clean-up Python generated class types and instances
	// The class types are instances of UPythonGeneratedClass
	{
		ForEachObjectOfClass(UPythonGeneratedClass::StaticClass(), [&FlagObjectForPurge](UObject* InObject)
		{
			UPythonGeneratedClass* PythonGeneratedClass = CastChecked<UPythonGeneratedClass>(InObject);

			// Generated classes are kept alive by the FPyWrapperTypeRegistry ARO, so unregister the type from FPyWrapperTypeRegistry
			PythonGeneratedClass->UnregisterGeneratedType();

			FlagObjectForPurge(InObject, /*bMarkPendingKill*/false);
			ForEachObjectOfClass(PythonGeneratedClass, [&FlagObjectForPurge](UObject* InInnerObject)
			{
				FlagObjectForPurge(InInnerObject, /*bMarkPendingKill*/true);
			}, false, RF_NoFlags, EInternalObjectFlags::Native);
		}, false, RF_ClassDefaultObject, EInternalObjectFlags::Native);
	}

	// Clean-up Python generated struct types
	// The struct types are instances of UPythonGeneratedStruct
	{
		ForEachObjectOfClass(UPythonGeneratedStruct::StaticClass(), [&FlagObjectForPurge](UObject* InObject)
		{
			UPythonGeneratedStruct* PythonGeneratedStruct = CastChecked<UPythonGeneratedStruct>(InObject);

			// Generated structs are kept alive by the FPyWrapperTypeRegistry ARO, so unregister the type from FPyWrapperTypeRegistry
			PythonGeneratedStruct->UnregisterGeneratedType();

			FlagObjectForPurge(InObject, /*bMarkPendingKill*/false);
		}, false, RF_ClassDefaultObject, EInternalObjectFlags::Native);
	}

	// Clean-up Python generated enum types
	// The enum types are instances of UPythonGeneratedEnum
	{
		ForEachObjectOfClass(UPythonGeneratedEnum::StaticClass(), [&FlagObjectForPurge](UObject* InObject)
		{
			UPythonGeneratedEnum* PythonGeneratedEnum = CastChecked<UPythonGeneratedEnum>(InObject);

			// Generated enums are kept alive by the FPyWrapperTypeRegistry ARO, so unregister the type from FPyWrapperTypeRegistry
			PythonGeneratedEnum->UnregisterGeneratedType();

			FlagObjectForPurge(InObject, /*bMarkPendingKill*/false);
		}, false, RF_ClassDefaultObject, EInternalObjectFlags::Native);
	}

	// Clean-up Python callable types and instances
	// The callable types all derive directly from UPythonCallableForDelegate
	{
		TArray<UClass*> PythonCallableClasses;
		GetDerivedClasses(UPythonCallableForDelegate::StaticClass(), PythonCallableClasses, true);

		for (UClass* PythonCallableClass : PythonCallableClasses)
		{
			// The classes themselves don't hold Python resources, so it's not imperative that 
			// they are destroyed, but we don't need them anymore so attempt to let them die
			PythonCallableClass->ClearFlags(RF_Public | RF_Standalone);

			ForEachObjectOfClass(PythonCallableClass, [&FlagObjectForPurge](UObject* InObject)
			{
				FlagObjectForPurge(InObject, /*bMarkPendingKill*/true);
			}, false, RF_NoFlags, EInternalObjectFlags::Native);
		}
	}

	// Clean-up Python object handles
	// The handles are instances of UPythonObjectHandle
	{
		ForEachObjectOfClass(UPythonObjectHandle::StaticClass(), [&FlagObjectForPurge](UObject* InObject)
		{
			FlagObjectForPurge(InObject, /*bMarkPendingKill*/false);
		}, false, RF_ClassDefaultObject, EInternalObjectFlags::Native);
	}

	if (PurgingReferenceCollector.HasObjectToPurge())
	{
		AddReferencedObjects(PurgingReferenceCollector);
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		for (const FWeakObjectPtr& WeakReferencesToPurgedObject : WeakReferencesToPurgedObjects)
		{
			if (UObject* FailedToPurgeObject = WeakReferencesToPurgedObject.Get())
			{
				if (IPythonResourceOwner* PythonResourceOwner = Cast<IPythonResourceOwner>(FailedToPurgeObject))
				{
					UE_LOGF(LogPython, Display, "Object '%ls' was externally referenced when shutting down Python. Forcibly releasing its Python resources!", *FailedToPurgeObject->GetPathName());
					PythonResourceOwner->ReleasePythonResources();
				}
				else
				{
					UE_LOGF(LogPython, Warning, "Object '%ls' was externally referenced when shutting down Python and could not gracefully cleaned-up. This may lead to crashes!", *FailedToPurgeObject->GetPathName());
				}
			}
		}
	}
}

void FPyReferenceCollector::Shutdown()
{
	PythonWrappedInstances.Reset();
}

void FPyReferenceCollector::AddReferencedObjectsFromDelegate(FReferenceCollector& InCollector, FScriptDelegate& InDelegate)
{
	// Keep the delegate object alive if it's using a Python proxy instance
	if (UPythonCallableForDelegate* PythonCallableForDelegate = Cast<UPythonCallableForDelegate>(InDelegate.GetUObject()))
	{
		TWeakObjectPtr<UPythonCallableForDelegate> Ptr{PythonCallableForDelegate};
		InCollector.AddReferencedObject(Ptr);
	}
}

void FPyReferenceCollector::AddReferencedObjectsFromMulticastDelegate(FReferenceCollector& InCollector, const FMulticastScriptDelegate& InDelegate)
{
	// Keep the delegate objects alive if they're using a Python proxy instance
	// We have to use the EvenIfUnreachable variant here as the objects are speculatively marked as unreachable during GC
	for (UObject* DelegateObj : InDelegate.GetAllObjectsEvenIfUnreachable())
	{
		if (UPythonCallableForDelegate* PythonCallableForDelegate = Cast<UPythonCallableForDelegate>(DelegateObj))
		{
			TWeakObjectPtr<UPythonCallableForDelegate> Ptr{PythonCallableForDelegate};
			InCollector.AddReferencedObject(Ptr);
		}
	}
}

void FPyReferenceCollector::AddReferencedObjectsFromStruct(FReferenceCollector& InCollector, const UStruct* InStruct, void* InStructAddr, const EPyReferenceCollectorFlags InFlags)
{
	bool bUnused = false;
	AddReferencedObjectsFromStructInternal(InCollector, InStruct, InStructAddr, InFlags, bUnused);
}

void FPyReferenceCollector::AddReferencedObjectsFromProperty(FReferenceCollector& InCollector, const FProperty* InProp, void* InBaseAddr, const EPyReferenceCollectorFlags InFlags)
{
	bool bUnused = false;
	AddReferencedObjectsFromPropertyInternal(InCollector, InProp, InBaseAddr, InFlags, bUnused);
}

void FPyReferenceCollector::AddReferencedObjectsFromStructInternal(FReferenceCollector& InCollector, const UStruct* InStruct, void* InStructAddr, const EPyReferenceCollectorFlags InFlags, bool& OutValueChanged)
{
	for (TFieldIterator<const FProperty> PropIt(InStruct); PropIt; ++PropIt)
	{
		AddReferencedObjectsFromPropertyInternal(InCollector, *PropIt, InStructAddr, InFlags, OutValueChanged);
	}
}

void FPyReferenceCollector::AddReferencedObjectsFromPropertyInternal(FReferenceCollector& InCollector, const FProperty* InProp, void* InBaseAddr, const EPyReferenceCollectorFlags InFlags, bool& OutValueChanged)
{
	if (const FObjectProperty* CastProp = CastField<FObjectProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeObjects))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				void* ObjValuePtr = CastProp->ContainerPtrToValuePtr<void>(InBaseAddr, ArrIndex);
				UObject* CurObjVal = CastProp->GetObjectPropertyValue(ObjValuePtr);
				if (CurObjVal)
				{
					TObjectPtr<UObject> NewObjVal = CurObjVal;
					InCollector.AddReferencedObject(NewObjVal);

					if (NewObjVal != CurObjVal)
					{
						OutValueChanged = true;
						CastProp->SetObjectPropertyValue(ObjValuePtr, NewObjVal);
					}
				}
			}
		}
		return;
	}

	if (const FInterfaceProperty* CastProp = CastField<FInterfaceProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeInterfaces))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				void* ValuePtr = CastProp->ContainerPtrToValuePtr<void>(InBaseAddr, ArrIndex);
				UObject* CurObjVal = CastProp->GetPropertyValue(ValuePtr).GetObject();
				if (CurObjVal)
				{
					TObjectPtr<UObject> NewObjVal = CurObjVal;
					InCollector.AddReferencedObject(NewObjVal);

					if (NewObjVal != CurObjVal)
					{
						OutValueChanged = true;
						CastProp->SetPropertyValue(ValuePtr, FScriptInterface(NewObjVal, NewObjVal ? NewObjVal->GetInterfaceAddress(CastProp->InterfaceClass) : nullptr));
					}
				}
			}
		}
		return;
	}

	if (const FStructProperty* CastProp = CastField<FStructProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeStructs))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				AddReferencedObjectsFromStructInternal(InCollector, CastProp->Struct, CastProp->ContainerPtrToValuePtr<void>(InBaseAddr, ArrIndex), InFlags, OutValueChanged);
			}
		}
		return;
	}

	if (const FDelegateProperty* CastProp = CastField<FDelegateProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeDelegates))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				FScriptDelegate* Value = CastProp->GetPropertyValuePtr(CastProp->ContainerPtrToValuePtr<void>(InBaseAddr, ArrIndex));
				AddReferencedObjectsFromDelegate(InCollector, *Value);
			}
		}
		return;
	}

	if (const FMulticastDelegateProperty* CastProp = CastField<FMulticastDelegateProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeDelegates))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				if (const FMulticastScriptDelegate* Value = CastProp->GetMulticastDelegate(CastProp->ContainerPtrToValuePtr<void>(InBaseAddr, ArrIndex)))
				{
					AddReferencedObjectsFromMulticastDelegate(InCollector, *Value);
				}
			}
		}
		return;
	}

	if (const FArrayProperty* CastProp = CastField<FArrayProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeArrays))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				FScriptArrayHelper_InContainer ScriptArrayHelper(CastProp, InBaseAddr, ArrIndex);

				const int32 ElementCount = ScriptArrayHelper.Num();
				for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
				{
					AddReferencedObjectsFromPropertyInternal(InCollector, CastProp->Inner, ScriptArrayHelper.GetRawPtr(ElementIndex), InFlags, OutValueChanged);
				}
			}
		}
		return;
	}

	if (const FSetProperty* CastProp = CastField<FSetProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeSets))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				bool bSetValuesChanged = false;
				FScriptSetHelper_InContainer ScriptSetHelper(CastProp, InBaseAddr, ArrIndex);
				for (FScriptSetHelper::FIterator It(ScriptSetHelper); It; ++It)
				{
					AddReferencedObjectsFromPropertyInternal(InCollector, ScriptSetHelper.GetElementProperty(), ScriptSetHelper.GetElementPtr(It), InFlags, bSetValuesChanged);
				}

				if (bSetValuesChanged)
				{
					OutValueChanged = true;
					ScriptSetHelper.Rehash();
				}
			}
		}
		return;
	}

	if (const FMapProperty* CastProp = CastField<FMapProperty>(InProp))
	{
		if (EnumHasAnyFlags(InFlags, EPyReferenceCollectorFlags::IncludeMaps))
		{
			for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
			{
				bool bMapKeysChanged = false;
				bool bMapValuesChanged = false;
				FScriptMapHelper_InContainer ScriptMapHelper(CastProp, InBaseAddr, ArrIndex);

				for (FScriptMapHelper::FIterator It(ScriptMapHelper); It; ++It)
				{
					// Note: We use the pair pointer below as AddReferencedObjectsFromPropertyInternal expects a base address and the key/value property will apply the correct offset from the base
					AddReferencedObjectsFromPropertyInternal(InCollector, ScriptMapHelper.GetKeyProperty(), ScriptMapHelper.GetPairPtr(It), InFlags, bMapKeysChanged);
					AddReferencedObjectsFromPropertyInternal(InCollector, ScriptMapHelper.GetValueProperty(), ScriptMapHelper.GetPairPtr(It), InFlags, bMapValuesChanged);
				}

				if (bMapKeysChanged || bMapValuesChanged)
				{
					OutValueChanged = true;
					if (bMapKeysChanged)
					{
						ScriptMapHelper.Rehash();
					}
				}
			}
		}
		return;
	}

	if (const FOptionalProperty* CastProp = CastField<FOptionalProperty>(InProp))
	{
		for (int32 ArrIndex = 0; ArrIndex < InProp->ArrayDim; ++ArrIndex)
		{
			void* ValuePtr = CastProp->ContainerPtrToValuePtr<void>(InBaseAddr, ArrIndex);
			if (CastProp->IsSet(ValuePtr))
			{
				AddReferencedObjectsFromPropertyInternal(InCollector, CastProp->GetValueProperty(), CastProp->GetValuePointerForReadOrReplace(ValuePtr), InFlags, OutValueChanged);
			}
		}
		return;
	}
}

#endif	// WITH_PYTHON
