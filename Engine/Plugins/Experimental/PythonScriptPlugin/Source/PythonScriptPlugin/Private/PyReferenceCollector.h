// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "UObject/ScriptDelegateFwd.h"
#include "UObject/WeakObjectPtr.h"
#include "Containers/ArrayView.h"

#if WITH_PYTHON

struct FPyWrapperBase;

enum class EPyReferenceCollectorFlags : uint8
{
	None = 0,
	IncludeObjects = 1<<0,
	IncludeInterfaces = 1<<1,
	IncludeStructs = 1<<2,
	IncludeDelegates = 1<<3,
	IncludeArrays = 1<<4,
	IncludeSets = 1<<5,
	IncludeMaps = 1<<6,
	IncludeContainers = IncludeArrays | IncludeSets | IncludeMaps,
	IncludeAll = IncludeObjects | IncludeInterfaces | IncludeStructs | IncludeDelegates | IncludeContainers,
};
ENUM_CLASS_FLAGS(EPyReferenceCollectorFlags)

/** Singleton instance that keeps UObject instances alive while they're being referenced by Python */
class FPyReferenceCollector : public FGCObject
{
public:
	/** Access the singleton instance */
	static FPyReferenceCollector& Get();

	/** Add a wrapped instance so it can be reference collected */
	void AddWrappedInstance(FPyWrapperBase* InInstance);

	/** Remove a wrapped instance */
	void RemoveWrappedInstance(FPyWrapperBase* InInstance);

	/** Purge any Python references to the given UObject instance */
	void PurgeUnrealObjectReferences(const UObject* InObject, const bool bIncludeInnerObjects);

	/** Purge any Python references to the given UObject instances */
	void PurgeUnrealObjectReferences(const TArrayView<const UObject*>& InObjects, const bool bIncludeInnerObjects);

	/** Purge any Python references to any Unreal types generated from or for Python */
	void PurgeUnrealGeneratedTypes();

	void Shutdown();

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;

	/** Utility function to ARO a delegate instance */
	static void AddReferencedObjectsFromDelegate(FReferenceCollector& InCollector, FScriptDelegate& InDelegate);

	/** Utility function to ARO a multicast delegate instance */
	static void AddReferencedObjectsFromMulticastDelegate(FReferenceCollector& InCollector, const FMulticastScriptDelegate& InDelegate);

	/** Utility function to ARO all properties on a struct instance */
	static void AddReferencedObjectsFromStruct(FReferenceCollector& InCollector, const UStruct* InStruct, void* InStructAddr, const EPyReferenceCollectorFlags InFlags = EPyReferenceCollectorFlags::IncludeAll);

	/** Utility function to ARO the given property instance */
	static void AddReferencedObjectsFromProperty(FReferenceCollector& InCollector, const FProperty* InProp, void* InBaseAddr, const EPyReferenceCollectorFlags InFlags = EPyReferenceCollectorFlags::IncludeAll);

private:
	/** Utility function to ARO all properties on a struct instance */
	static void AddReferencedObjectsFromStructInternal(FReferenceCollector& InCollector, const UStruct* InStruct, void* InStructAddr, const EPyReferenceCollectorFlags InFlags, bool& OutValueChanged);

	/** Utility function to ARO the given property instance */
	static void AddReferencedObjectsFromPropertyInternal(FReferenceCollector& InCollector, const FProperty* InProp, void* InBaseAddr, const EPyReferenceCollectorFlags InFlags, bool& OutValueChanged);

	/** Set of Python wrapped instances to ARO */
	TSet<FPyWrapperBase*> PythonWrappedInstances;
};

#endif	// WITH_PYTHON
