// Copyright Epic Games, Inc. All Rights Reserved.

#include "PySlate.h"
#include "PyGIL.h"
#include "PyCore.h"
#include "PyGenUtil.h"
#include "PyConversion.h"
#include "PyWrapperTypeRegistry.h"

#include "UObject/Package.h"
#include "Framework/Application/SlateApplication.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#if WITH_PYTHON

namespace PySlateUtil
{

TArray<FDelegateHandle> PythonPreTickCallbackHandles;
TArray<FDelegateHandle> PythonPostTickCallbackHandles;

FPyDelegateHandle* RegisterSlateTickCallback(FSlateApplication::FSlateTickEvent& InSlateTickEvent, TArray<FDelegateHandle>& InOutPythonCallbackHandles, PyObject* InPyCallable)
{
	// We copy the PyCallable into the lambda to keep the Python object alive as long as the delegate is bound
	FDelegateHandle TickEventDelegateHandle = InSlateTickEvent.AddLambda([PyCallable = FPyAutoGILPtr(FPyObjectPtr::NewReference(InPyCallable))](const float InDeltaTime) mutable
	{
		if (!PyUtil::CanTickPython())
		{
			return;
		}

		FPyScopedGIL GIL;

		FPyObjectPtr PyArgs = FPyObjectPtr::StealReference(PyTuple_New(1));
		PyTuple_SetItem(PyArgs, 0, PyConversion::Pythonize(InDeltaTime)); // SetItem steals the reference

		FPyObjectPtr Result = FPyObjectPtr::StealReference(PyObject_CallObject(PyCallable.Get().GetPtr(), PyArgs));
		if (!Result)
		{
			PyUtil::LogPythonError();
		}
	});

	if (TickEventDelegateHandle.IsValid())
	{
		InOutPythonCallbackHandles.Add(TickEventDelegateHandle);
	}

	return FPyDelegateHandle::CreateInstance(TickEventDelegateHandle);
}

bool UnregisterSlateTickCallback(FSlateApplication::FSlateTickEvent& InSlateTickEvent, TArray<FDelegateHandle>& InOutPythonCallbackHandles, PyObject* InCallbackHandle)
{
	FPyDelegateHandlePtr PyTickEventDelegateHandle = FPyDelegateHandlePtr::StealReference(FPyDelegateHandle::CastPyObject(InCallbackHandle));
	if (!PyTickEventDelegateHandle)
	{
		return false;
	}

	FDelegateHandle& TickEventDelegateHandle = PyTickEventDelegateHandle->Value;
	if (TickEventDelegateHandle.IsValid())
	{
		InOutPythonCallbackHandles.Remove(TickEventDelegateHandle);
		InSlateTickEvent.Remove(TickEventDelegateHandle);
	}

	return true;
}

void ClearSlateTickCallbacks(FSlateApplication::FSlateTickEvent& InSlateTickEvent, TArray<FDelegateHandle>& InOutPythonCallbackHandles)
{
	for (const FDelegateHandle& Handle : InOutPythonCallbackHandles)
	{
		InSlateTickEvent.Remove(Handle);
	}
	InOutPythonCallbackHandles.Empty();
}

} // namespace PySlateUtil

namespace PySlate
{

PyObject* RegisterSlatePreTickCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:register_slate_pre_tick_callback", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (FSlateApplication::IsInitialized())
	{
		return (PyObject*)PySlateUtil::RegisterSlateTickCallback(FSlateApplication::Get().OnPreTick(), PySlateUtil::PythonPreTickCallbackHandles, PyObj);
	}

	return (PyObject*)FPyDelegateHandle::CreateInstance(FDelegateHandle());
}

PyObject* UnregisterSlatePreTickCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:unregister_slate_pre_tick_callback", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (FSlateApplication::IsInitialized() && !PySlateUtil::UnregisterSlateTickCallback(FSlateApplication::Get().OnPreTick(), PySlateUtil::PythonPreTickCallbackHandles, PyObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("unregister_slate_pre_tick_callback"), *FString::Printf(TEXT("Failed to convert argument '%s' to '_DelegateHandle'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	Py_RETURN_NONE;
}

PyObject* RegisterSlatePostTickCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:register_slate_post_tick_callback", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (FSlateApplication::IsInitialized())
	{
		return (PyObject*)PySlateUtil::RegisterSlateTickCallback(FSlateApplication::Get().OnPostTick(), PySlateUtil::PythonPostTickCallbackHandles, PyObj);
	}

	return (PyObject*)FPyDelegateHandle::CreateInstance(FDelegateHandle());
}

PyObject* UnregisterSlatePostTickCallback(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O:unregister_slate_post_tick_callback", &PyObj))
	{
		return nullptr;
	}
	check(PyObj);

	if (FSlateApplication::IsInitialized() && !PySlateUtil::UnregisterSlateTickCallback(FSlateApplication::Get().OnPostTick(), PySlateUtil::PythonPostTickCallbackHandles, PyObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("unregister_slate_post_tick_callback"), *FString::Printf(TEXT("Failed to convert argument '%s' to '_DelegateHandle'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return nullptr;
	}

	Py_RETURN_NONE;
}

PyObject* ParentExternalWindowToSlate(PyObject* InSelf, PyObject* InArgs)
{
	PyObject* PyExternalWindowHandle = nullptr;
	PyObject* PyParentWindowSearchMethod = nullptr;
	if (!PyArg_ParseTuple(InArgs, "O|O:parent_external_window_to_slate", &PyExternalWindowHandle, &PyParentWindowSearchMethod))
	{
		return nullptr;
	}
	check(PyExternalWindowHandle);

	void* ExternalWindowHandle = nullptr;
	if (!PyConversion::Nativize(PyExternalWindowHandle, ExternalWindowHandle))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("parent_external_window_to_slate"), *FString::Printf(TEXT("Failed to convert argument '%s' to 'void*'"), *PyUtil::GetFriendlyTypename(PyExternalWindowHandle)));
		return nullptr;
	}

	static const UEnum* ParentWindowSearchMethodEnum = StaticEnum<ESlateParentWindowSearchMethod>();
	ESlateParentWindowSearchMethod ParentWindowSearchMethod = ESlateParentWindowSearchMethod::ActiveWindow;
	if (PyParentWindowSearchMethod && !PyConversion::NativizeEnumEntry(PyParentWindowSearchMethod, ParentWindowSearchMethodEnum, ParentWindowSearchMethod))
	{
		PyUtil::SetPythonError(PyExc_TypeError, TEXT("parent_external_window_to_slate"), *FString::Printf(TEXT("Failed to convert argument '%s' to 'SlateParentWindowSearchMethod'"), *PyUtil::GetFriendlyTypename(PyParentWindowSearchMethod)));
		return nullptr;
	}

	if (FSlateApplication::IsInitialized())
	{
		const void* SlateParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr, ParentWindowSearchMethod);
		if (SlateParentWindowHandle && ExternalWindowHandle)
		{
#if PLATFORM_WINDOWS
			::SetWindowLongPtr((HWND)ExternalWindowHandle, -8/*GWL_HWNDPARENT*/, (LONG_PTR)SlateParentWindowHandle);
#endif
		}
	}

	Py_RETURN_NONE;
}

PyMethodDef PySlateMethods[] = {
	{ "register_slate_pre_tick_callback", PyCFunctionCast(&RegisterSlatePreTickCallback), METH_VARARGS, "x.register_slate_pre_tick_callback(callable: Union[Callable[[float], None], DelegateBase]) -> object -- register the given callable (taking a single float) as a pre-tick callback in Slate" },
	{ "unregister_slate_pre_tick_callback", PyCFunctionCast(&UnregisterSlatePreTickCallback), METH_VARARGS, "x.unregister_slate_pre_tick_callback(handle: object) -> None -- unregister the given handle from a previous call to register_slate_pre_tick_callback" },
	{ "register_slate_post_tick_callback", PyCFunctionCast(&RegisterSlatePostTickCallback), METH_VARARGS, "x.register_slate_post_tick_callback(callable: Union[Callable[[float], None], DelegateBase]) -> object -- register the given callable (taking a single float) as a pre-tick callback in Slate" },
	{ "unregister_slate_post_tick_callback", PyCFunctionCast(&UnregisterSlatePostTickCallback), METH_VARARGS, "x.unregister_slate_post_tick_callback(handle: object) -> None -- unregister the given handle from a previous call to register_slate_post_tick_callback" },
	{ "parent_external_window_to_slate", PyCFunctionCast(&ParentExternalWindowToSlate), METH_VARARGS, "x.parent_external_window_to_slate(external_window_handle: object, parent_search_method: SlateParentWindowSearchMethod = SlateParentWindowSearchMethod.ACTIVE_WINDOW) -> None -- parent the given OS specific external window handle to a suitable Slate window" },
	{ nullptr, nullptr, 0, nullptr }
};

void InitializeModule()
{
	PyGenUtil::FNativePythonModule NativePythonModule;
	NativePythonModule.PyModuleMethods = PySlateMethods;

	NativePythonModule.PyModule = PyImport_AddModule("_unreal_slate");
	PyModule_AddFunctions(NativePythonModule.PyModule, PySlateMethods);

	FPyWrapperTypeRegistry::Get().RegisterNativePythonModule(MoveTemp(NativePythonModule));
}

void ShutdownModule()
{
	// We need to remove any lingering Python Slate callbacks, as they will crash if called after the interpreter has shutdown
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication& SlateApplication = FSlateApplication::Get();
		PySlateUtil::ClearSlateTickCallbacks(SlateApplication.OnPreTick(), PySlateUtil::PythonPreTickCallbackHandles);
		PySlateUtil::ClearSlateTickCallbacks(SlateApplication.OnPostTick(), PySlateUtil::PythonPostTickCallbackHandles);
	}
}

}

#endif	// WITH_PYTHON
