// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"
#include "CoreMinimal.h"

#if WITH_PYTHON

namespace PySlate
{
	void InitializeModule();
	void ShutdownModule();
}

#endif	// WITH_PYTHON
