// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyTest.h"
#include "PyUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PyTest)

FPyTestStruct::FPyTestStruct()
{
	Bool = false;
	Int = 0;
	Float = 0.0f;
	Enum = EPyTestEnum::One;
	LegacyInt_DEPRECATED = 0;
	BoolInstanceOnly = false;
	BoolDefaultsOnly = false;
}

bool UPyTestStructLibrary::IsBoolSet(const FPyTestStruct& InStruct)
{
	return InStruct.Bool;
}

bool UPyTestStructLibrary::LegacyIsBoolSet(const FPyTestStruct& InStruct)
{
	return IsBoolSet(InStruct);
}

int32 UPyTestStructLibrary::GetConstantValue()
{
	return 10;
}

FPyTestStruct UPyTestStructLibrary::AddInt(const FPyTestStruct& InStruct, const int32 InValue)
{
	FPyTestStruct Result = InStruct;
	Result.Int += InValue;
	return Result;
}

FPyTestStruct UPyTestStructLibrary::AddFloat(const FPyTestStruct& InStruct, const float InValue)
{
	FPyTestStruct Result = InStruct;
	Result.Float += InValue;
	return Result;
}

FPyTestStruct UPyTestStructLibrary::AddStr(const FPyTestStruct& InStruct, const FString& InValue)
{
	FPyTestStruct Result = InStruct;
	Result.String += InValue;
	return Result;
}

void UPyTestStructLibrary::SetBoolMutable(const FPyTestStruct& InStruct)
{
	InStruct.BoolMutable = true;
}

void UPyTestStructLibrary::ClearBoolMutable(const FPyTestStruct& InStruct)
{
	InStruct.BoolMutable = false;
}

void UPyTestStructLibrary::SetBoolMutableViaRef(FPyTestStruct& InStruct)
{
	InStruct.BoolMutable = true;
}

void UPyTestStructLibrary::ClearBoolMutableViaRef(FPyTestStruct& InStruct)
{
	InStruct.BoolMutable = false;
}

UPyTestInterface::UPyTestInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPyTestChildInterface::UPyTestChildInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPyTestOtherInterface::UPyTestOtherInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPyTestObject::UPyTestObject()
{
	StructArray.AddDefaulted();
	StructArray.AddDefaulted();
}

int32 UPyTestObject::FuncBlueprintNative_Implementation(const int32 InValue) const
{
	return InValue;
}

void UPyTestObject::FuncBlueprintNativeRef_Implementation(FPyTestStruct& InOutStruct) const
{
}

int32 UPyTestObject::CallFuncBlueprintImplementable(const int32 InValue) const
{
	return FuncBlueprintImplementable(InValue);
}

bool UPyTestObject::CallFuncBlueprintImplementablePackedGetter(int32& OutValue) const
{
	return FuncBlueprintImplementablePackedGetter(OutValue);
}

int32 UPyTestObject::CallFuncBlueprintNative(const int32 InValue) const
{
	return FuncBlueprintNative(InValue);
}

void UPyTestObject::CallFuncBlueprintNativeRef(FPyTestStruct& InOutStruct) const
{
	return FuncBlueprintNativeRef(InOutStruct);
}

void UPyTestObject::FuncTakingPyTestStruct(const FPyTestStruct& InStruct) const
{
}

void UPyTestObject::FuncTakingPyTestChildStruct(const FPyTestChildStruct& InStruct) const
{
}

void UPyTestObject::LegacyFuncTakingPyTestStruct(const FPyTestStruct& InStruct) const
{
	FuncTakingPyTestStruct(InStruct);
}

void UPyTestObject::FuncTakingPyTestStructDefault(const FPyTestStruct& InStruct)
{
}

int32 UPyTestObject::FuncTakingPyTestDelegate(const FPyTestDelegate& InDelegate, const int32 InValue) const
{
	return InDelegate.IsBound() ? InDelegate.Execute(InValue) : INDEX_NONE;
}

void UPyTestObject::FuncTakingFieldPath(const TFieldPath<FProperty>& InFieldPath)
{
	FieldPath = InFieldPath;
}

int32 UPyTestObject::DelegatePropertyCallback(const int32 InValue) const
{
	if (InValue != Int)
	{
		UE_LOGF(LogPython, Error, "Given value (%d) did not match the Int property value (%d)", InValue, Int);
	}

	return InValue;
}

void UPyTestObject::MulticastDelegatePropertyCallback(FString InStr) const
{
	if (InStr != String)
	{
		UE_LOGF(LogPython, Error, "Given value (%ls) did not match the String property value (%ls)", *InStr, *String);
	}
}

TArray<int32> UPyTestObject::ReturnArray()
{
	TArray<int32> TmpArray;
	TmpArray.Add(10);
	return TmpArray;
}

TSet<int32> UPyTestObject::ReturnSet()
{
	TSet<int32> TmpSet;
	TmpSet.Add(10);
	return TmpSet;
}

TMap<int32, bool> UPyTestObject::ReturnMap()
{
	TMap<int32, bool> TmpMap;
	TmpMap.Add(10, true);
	return TmpMap;
}

TFieldPath<FProperty> UPyTestObject::ReturnFieldPath()
{
	return TFieldPath<FProperty>(UPyTestObject::StaticClass()->FindPropertyByName(TEXT("FieldPath")));
}

void UPyTestObject::EmitScriptError()
{
	FFrame::KismetExecutionMessage(TEXT("EmitScriptError was called"), ELogVerbosity::Error);
}

void UPyTestObject::EmitScriptWarning()
{
	FFrame::KismetExecutionMessage(TEXT("EmitScriptWarning was called"), ELogVerbosity::Warning);
}

int32 UPyTestObject::GetConstantValue()
{
	return 10;
}

int32 UPyTestObject::FuncInterface(const int32 InValue) const
{
	return InValue;
}

int32 UPyTestObject::FuncInterfaceChild(const int32 InValue) const
{
	return InValue;
}

int32 UPyTestObject::FuncInterfaceOther(const int32 InValue) const
{
	return InValue;
}

bool UPyTestObjectLibrary::IsBoolSet(const UPyTestObject* InObj)
{
	return InObj->Bool;
}

int32 UPyTestObjectLibrary::GetOtherConstantValue()
{
	return 20;
}


FString UPyTestTypeHint::GetStringConst()
{
	return FString("Foo");
}

int32 UPyTestTypeHint::GetIntConst()
{
	return 777;
}


UPyTestTypeHint::UPyTestTypeHint()
{
	ObjectProp = NewObject<UPyTestObject>();
}

UPyTestTypeHint::UPyTestTypeHint(bool bParam1, int32 Param2, float Param3, const FString& Param4, const FText& Param5)
	: BoolProp(bParam1)
	, IntProp(Param2)
	, FloatProp(Param3)
	, StringProp(Param4)
	, TextProp(Param5)
{
}

bool UPyTestTypeHint::CheckBoolTypeHints(bool bParam1, bool bParam2, bool bParam3)
{
	return true;
}

int32 UPyTestTypeHint::CheckIntegerTypeHints(uint8 Param1, int32 Param2, int64 Param3)
{
	return 0;
}

double UPyTestTypeHint::CheckFloatTypeHints(float Param1, double Param2, float Param3, double Param4)
{
	return 0.0;
}

EPyTestEnum UPyTestTypeHint::CheckEnumTypeHints(EPyTestEnum Param1, EPyTestEnum Param2)
{
	return EPyTestEnum::One;
}

FString UPyTestTypeHint::CheckStringTypeHints(const FString& Param1, const FString& Param2)
{
	return FString();
}

FName UPyTestTypeHint::CheckNameTypeHints(const FName& Param1, const FName& Param2)
{
	return FName();
}

FText UPyTestTypeHint::CheckTextTypeHints(const FText& Param1, const FText& Param2)
{
	return FText::GetEmpty();
}

TFieldPath<FProperty> UPyTestTypeHint::CheckFieldPathTypeHints(const TFieldPath<FProperty> Param1)
{
	return TFieldPath<FProperty>();
}

FPyTestStruct UPyTestTypeHint::CheckStructTypeHints(const FPyTestStruct& Param1, const FPyTestStruct& Param2)
{
	return FPyTestStruct();
}

UPyTestObject* UPyTestTypeHint::CheckObjectTypeHints(const UPyTestObject* Param1, const UPyTestObject* Param3)
{
	return nullptr;
}

TArray<FText> UPyTestTypeHint::CheckArrayTypeHints(const TArray<FString>& Param1, const TArray<FName>& Param2, const TArray<FText>& Param3, const TArray<UObject*>& Param4)
{
	return TArray<FText>();
}

TSet<FName> UPyTestTypeHint::CheckSetTypeHints(const TSet<FString>& Param1, const TSet<FName>& Param2, const TSet<UObject*>& Param4)
{
	return TSet<FName>();
}

TMap<FString, UObject*> UPyTestTypeHint::CheckMapTypeHints(const TMap<int, FString>& Param1, const TMap<int, FName>& Param2, const TMap<int, FText>& Param3, const TMap<int, UObject*>& Param4)
{
	return TMap<FString, UObject*>();
}

FPyTestDelegate& UPyTestTypeHint::CheckDelegateTypeHints(const FPyTestDelegate& Param1)
{
	return DelegateProp;
}

bool UPyTestTypeHint::CheckStaticFunction(bool Param1, int32 Param2, double Param3, const FString& Param4)
{
	return true;
}

int UPyTestTypeHint::CheckTupleReturnType(UPARAM(ref) FString& InOutString)
{
	InOutString = TEXT("Foo");
	return 0;
}


