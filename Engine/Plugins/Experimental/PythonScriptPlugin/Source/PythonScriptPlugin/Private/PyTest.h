// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PyTestInterface.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/ObjectPtr.h"
#include "PyTest.generated.h"

/**
 * Delegate to allow testing of the various script delegate features that are exposed to Python wrapped types.
 */
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam(int32, FPyTestDelegate, int32, InValue);

/**
 * Multicast delegate to allow testing of the various script delegate features that are exposed to Python wrapped types.
 */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FPyTestMulticastDelegate, FString, InStr);

/**
 * Delegate for slate pre/post tick event.
 */
DECLARE_DYNAMIC_DELEGATE_OneParam(FPyTestSlateTickDelegate, float, InDeltaTime);

/**
 * Enum to allow testing of the various UEnum features that are exposed to Python wrapped types.
 */
UENUM(BlueprintType)
enum class EPyTestEnum : uint8
{
	One UMETA(DisplayName = "Says One but my value is Zero"),
	Two,
};

/**
 * Struct to allow testing of the various UStruct features that are exposed to Python wrapped types.
 */
USTRUCT(BlueprintType)
struct FPyTestStruct
{
	GENERATED_BODY()

public:
	FPyTestStruct();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	bool Bool;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	int32 Int;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	float Float;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	EPyTestEnum Enum;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FString String;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FText Text;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TFieldPath<FProperty> FieldPath;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TFieldPath<FStructProperty> StructFieldPath;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<FString> StringArray;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TSet<FString> StringSet;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TMap<FString, int32> StringIntMap;

	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="LegacyInt is deprecated. Please use Int instead."))
	int32 LegacyInt_DEPRECATED;

	UPROPERTY(EditInstanceOnly, Category = "Python|Internal")
	bool BoolInstanceOnly;

	UPROPERTY(EditDefaultsOnly, Category = "Python|Internal")
	bool BoolDefaultsOnly;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Python|Internal")
	mutable bool BoolMutable = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TOptional<int32> OptionalInt;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TOptional<FString> OptionalString;
};

/**
 * Struct to allow testing of inheritance on Python wrapped types.
 */
USTRUCT(BlueprintType)
struct FPyTestChildStruct : public FPyTestStruct
{
	GENERATED_BODY()
};

/**
 * Function library containing methods that should be hoisted onto the test struct in Python.
 */
UCLASS()
class UPyTestStructLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptMethod="IsBoolSet;IsBoolSetOld"))
	static bool IsBoolSet(const FPyTestStruct& InStruct);

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage="LegacyIsBoolSet is deprecated. Please use IsBoolSet instead."))
	static bool LegacyIsBoolSet(const FPyTestStruct& InStruct);

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptConstant="ConstantValue", ScriptConstantHost="/Script/PythonScriptPlugin.PyTestStruct"))
	static int32 GetConstantValue();

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodSelfReturn, ScriptOperator="+;+="))
	static FPyTestStruct AddInt(const FPyTestStruct& InStruct, const int32 InValue);

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodSelfReturn, ScriptOperator="+;+="))
	static FPyTestStruct AddFloat(const FPyTestStruct& InStruct, const float InValue);

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodSelfReturn, ScriptOperator="+;+="))
	static FPyTestStruct AddStr(const FPyTestStruct& InStruct, const FString& InValue);

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodMutable))
	static void SetBoolMutable(const FPyTestStruct& InStruct);

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodMutable))
	static void ClearBoolMutable(const FPyTestStruct& InStruct);

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodMutable))
	static void SetBoolMutableViaRef(UPARAM(ref) FPyTestStruct& InStruct);

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(ScriptMethod, ScriptMethodMutable))
	static void ClearBoolMutableViaRef(UPARAM(ref) FPyTestStruct& InStruct);
};

/**
 * Struct to allow testing of class sparse data on a Python exposed type.
 */
USTRUCT(BlueprintType)
struct FPyTestClassSparseData
{
	GENERATED_BODY()

public:
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Python|Internal")
	int32 IntFromSparseData = 0;
};

/**
 * Object to allow testing of the various UObject features that are exposed to Python wrapped types.
 */
UCLASS(Blueprintable, SparseClassDataTypes = PyTestClassSparseData)
class UPyTestObject : public UObject, public IPyTestChildInterface, public IPyTestOtherInterface
{
	GENERATED_BODY()

public:
	UPyTestObject();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	bool Bool;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	int32 Int;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	float Float;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	EPyTestEnum Enum;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FString String;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FName Name;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FText Text;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TFieldPath<FProperty> FieldPath = TFieldPath<FProperty>(FPyTestStruct::StaticStruct()->FindPropertyByName(TEXT("StringArray")));

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TFieldPath<FStructProperty> StructFieldPath = TFieldPath<FStructProperty>(CastField<FStructProperty>(UPyTestObject::GetClass()->FindPropertyByName(TEXT("Struct"))));

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<FString> StringArray;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TSet<FString> StringSet;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TMap<FString, int32> StringIntMap;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FPyTestDelegate Delegate;

	UPROPERTY(EditAnywhere, BlueprintAssignable, Category = "Python|Internal")
	FPyTestMulticastDelegate MulticastDelegate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FPyTestStruct Struct;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<FPyTestStruct> StructArray;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FPyTestChildStruct ChildStruct;

	UPROPERTY(EditInstanceOnly, Category = "Python|Internal")
	bool BoolInstanceOnly;

	UPROPERTY(EditDefaultsOnly, Category = "Python|Internal")
	bool BoolDefaultsOnly;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TOptional<int32> OptionalInt;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TOptional<FString> OptionalString;

	UFUNCTION(BlueprintImplementableEvent, Category = "Python|Internal")
	int32 FuncBlueprintImplementable(const int32 InValue) const;

	UFUNCTION(BlueprintImplementableEvent, Category = "Python|Internal")
	bool FuncBlueprintImplementablePackedGetter(int32& OutValue) const;

	UFUNCTION(BlueprintNativeEvent, Category = "Python|Internal")
	int32 FuncBlueprintNative(const int32 InValue) const;

	UFUNCTION(BlueprintNativeEvent, Category = "Python|Internal")
	void FuncBlueprintNativeRef(UPARAM(ref) FPyTestStruct& InOutStruct) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	int32 CallFuncBlueprintImplementable(const int32 InValue) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	bool CallFuncBlueprintImplementablePackedGetter(int32& OutValue) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	int32 CallFuncBlueprintNative(const int32 InValue) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	void CallFuncBlueprintNativeRef(UPARAM(ref) FPyTestStruct& InOutStruct) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	void FuncTakingPyTestStruct(const FPyTestStruct& InStruct) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	void FuncTakingPyTestChildStruct(const FPyTestChildStruct& InStruct) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal", meta=(DeprecatedFunction, DeprecationMessage="LegacyFuncTakingPyTestStruct is deprecated. Please use FuncTakingPyTestStruct instead."))
	void LegacyFuncTakingPyTestStruct(const FPyTestStruct& InStruct) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	void FuncTakingPyTestStructDefault(const FPyTestStruct& InStruct = FPyTestStruct());

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	int32 FuncTakingPyTestDelegate(const FPyTestDelegate& InDelegate, const int32 InValue) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	void FuncTakingFieldPath(const TFieldPath<FProperty>& InFieldPath); // UHT couldn't parse any default value for the FieldPath.

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	int32 DelegatePropertyCallback(const int32 InValue) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	void MulticastDelegatePropertyCallback(FString InStr) const;

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	static TArray<int32> ReturnArray();

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	static TSet<int32> ReturnSet();

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	static TMap<int32, bool> ReturnMap();

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	static TFieldPath<FProperty> ReturnFieldPath();

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	static void EmitScriptError();

	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	static void EmitScriptWarning();

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptConstant="ConstantValue"))
	static int32 GetConstantValue();

	virtual int32 FuncInterface(const int32 InValue) const override;
	virtual int32 FuncInterfaceChild(const int32 InValue) const override;
	virtual int32 FuncInterfaceOther(const int32 InValue) const override;
};

/**
 * Object to allow testing of inheritance on Python wrapped types.
 */
UCLASS(Blueprintable)
class UPyTestChildObject : public UPyTestObject
{
	GENERATED_BODY()
};

/**
 * Object to test deprecation of Python wrapped types.
 */
UCLASS(Blueprintable, deprecated, meta=(DeprecationMessage="LegacyPyTestObject is deprecated. Please use PyTestObject instead."))
class UDEPRECATED_LegacyPyTestObject : public UPyTestObject
{
	GENERATED_BODY()
};

/**
 * Function library containing methods that should be hoisted onto the test object in Python.
 */
UCLASS()
class UPyTestObjectLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptMethod="IsBoolSet"))
	static bool IsBoolSet(const UPyTestObject* InObj);

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptConstant="OtherConstantValue", ScriptConstantHost="/Script/PythonScriptPlugin.PyTestObject"))
	static int32 GetOtherConstantValue();
};

/**
 * This class along with UPyTestVectorDelegate verify that 2 UObjects with the same delegate name/type, do not collide.
 */
UCLASS(Blueprintable)
class UPyTestStructDelegate : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnNameCollisionDelegate, const FPyTestStruct&, PyStruct);

	/** Called when a new item is selected in the combobox. */
	UPROPERTY(BlueprintAssignable, Category = "Python|Internal")
	FOnNameCollisionDelegate OnNameCollisionTestDelegate; // Same prop name and type name as UPyTestVectorDelegate::FOnNameCollisionTestDelegate, but different params.
};

/**
 * This class along with UPyTestStructDelegate verify that 2 UObjects with the same delegate name/type, do not collide.
 */
UCLASS(Blueprintable)
class UPyTestVectorDelegate : public UObject
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnNameCollisionDelegate, const FVector2D&, Vec);

	/** Called when a new item is selected in the combobox. */
	UPROPERTY(BlueprintAssignable, Category = "Python|Internal")
	FOnNameCollisionDelegate OnNameCollisionTestDelegate; // Same prop name and type name as UPyTestStructDelegate::FOnNameCollisionTestDelegate, but different params.
};


/* Used to verify if the generated Python stub is correctly type-hinted (if type hint is enabled). The stub is generated
 * in the project intermediate folder when the Python developer mode is enabled (Editor preferences). The type hints can
 * be checked in the stub itself or PythonScriptPlugin/Content/Python/test_type_hints.py can be loaded in a Python IDE that
 * supports type checking and look at the code to verify that there is not problems with the types.
 */
UCLASS(Blueprintable)
class UPyTestTypeHint : public UObject
{
	GENERATED_BODY()

public:
	//
	// Check type hinted init methods.
	//

	UPyTestTypeHint();
	UPyTestTypeHint(bool bParam1, int32 Param2, float Param3, const FString& Param4 = "Hi", const FText& Param5 = FText());

	//
	// Check type hinted constants
	//

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptConstant="StrConst"))
	static FString GetStringConst();

	UFUNCTION(BlueprintPure, Category = "Python|Internal", meta=(ScriptConstant="IntConst"))
	static int32 GetIntConst();

	//
	// Check type hinted properties (setter/getter)
	//

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	bool BoolProp = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	int32 IntProp = 32;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	float FloatProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	EPyTestEnum EnumProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FString StringProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FName NameProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FText TextProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TFieldPath<FProperty> FieldPathProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FPyTestStruct StructProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TObjectPtr<UPyTestObject> ObjectProp = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<FString> StrArrayProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<FName> NameArrayProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<FText> TextArrayProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TArray<TObjectPtr<UObject>> ObjectArrayProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TSet<FString> SetProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	TMap<int32, FString> MapProp;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FPyTestDelegate DelegateProp;

	UPROPERTY(EditAnywhere, BlueprintAssignable, Category = "Python|Internal")
	FPyTestMulticastDelegate MulticastDelegateProp;

	//
	// Check type hinted methods.
	//

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	bool CheckBoolTypeHints(bool bParam1, bool bParam2 = true, bool bParam3 = false);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	int32 CheckIntegerTypeHints(uint8 Param1, int32 Param2 = 4, int64 Param3 = 5);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	double CheckFloatTypeHints(float Param1, double Param2, float Param3 = -3.3f, double Param4 = 4.4);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	EPyTestEnum CheckEnumTypeHints(EPyTestEnum Param1, EPyTestEnum Param2 = EPyTestEnum::One);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	FString CheckStringTypeHints(const FString& Param1, const FString& Param2 = TEXT("Hi"));

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	FName CheckNameTypeHints(const FName& Param1, const FName& Param2 = FName(TEXT("Hi")));

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	FText CheckTextTypeHints(const FText& Param1, const FText& Param2 = INVTEXT("Hi"));

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	TFieldPath<FProperty> CheckFieldPathTypeHints(const TFieldPath<FProperty> Param1);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	FPyTestStruct CheckStructTypeHints(const FPyTestStruct& Param1, const FPyTestStruct& Param2 = FPyTestStruct());

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	UPyTestObject* CheckObjectTypeHints(const UPyTestObject* Param1, const UPyTestObject* Param4 = nullptr);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	TArray<FText> CheckArrayTypeHints(const TArray<FString>& Param1, const TArray<FName>& Param2, const TArray<FText>& Param3, const TArray<UObject*>& Param4);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	TSet<FName> CheckSetTypeHints(const TSet<FString>& Param1, const TSet<FName>& Param2, const TSet<UObject*>& Param3);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	TMap<FString, UObject*> CheckMapTypeHints(const TMap<int, FString>& Param1, const TMap<int, FName>& Param2, const TMap<int, FText>& Param3, const TMap<int, UObject*>& Param4);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	FPyTestDelegate& CheckDelegateTypeHints(const FPyTestDelegate& Param1);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	static bool CheckStaticFunction(bool Param1, int32 Param2, double Param3, const FString& Param4);

	UFUNCTION(BlueprintPure, Category = "Python|Internal")
	static int CheckTupleReturnType(UPARAM(ref) FString& InOutString);

	//
	// Members to facilitate testing particular Python API.
	//

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Python|Internal")
	FPyTestSlateTickDelegate SlateTickDelegate;
};

