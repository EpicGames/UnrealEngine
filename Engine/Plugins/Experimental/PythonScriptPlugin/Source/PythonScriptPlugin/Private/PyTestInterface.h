// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "PyTestInterface.generated.h"

/**
 * Interface to allow testing of the various UInterface features that are exposed to Python wrapped types.
 */
UINTERFACE(BlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UPyTestInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};
class IPyTestInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	virtual int32 FuncInterface(const int32 InValue) const = 0;
};

/**
 * Interface to allow testing of inheritance on Python wrapped types.
 */
UINTERFACE(BlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UPyTestChildInterface : public UPyTestInterface
{
	GENERATED_UINTERFACE_BODY()
};
class IPyTestChildInterface : public IPyTestInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	virtual int32 FuncInterfaceChild(const int32 InValue) const = 0;
};

/**
 * Interface to allow testing of multiple-inheritance on Python wrapped types.
 */
UINTERFACE(BlueprintType, meta=(CannotImplementInterfaceInBlueprint))
class UPyTestOtherInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};
class IPyTestOtherInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Python|Internal")
	virtual int32 FuncInterfaceOther(const int32 InValue) const = 0;
};
