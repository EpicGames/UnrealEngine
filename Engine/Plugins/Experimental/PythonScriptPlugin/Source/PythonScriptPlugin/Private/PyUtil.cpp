// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyUtil.h"

#include "IPythonScriptPlugin.h"
#include "PyCore.h"
#include "PyConversion.h"
#include "PyGIL.h"

#include "PyWrapperObject.h"
#include "PyWrapperStruct.h"
#include "PyWrapperEnum.h"
#include "PyWrapperDelegate.h"
#include "PyWrapperName.h"
#include "PyWrapperText.h"
#include "PyWrapperArray.h"
#include "PyWrapperFixedArray.h"
#include "PyWrapperSet.h"
#include "PyWrapperMap.h"
#include "PyWrapperFieldPath.h"
#include "PyWrapperTypeRegistry.h"

#include "HAL/FileManager.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Subsystems/EngineSubsystem.h"
#include "Templates/Casts.h"
#include "UObject/EnumProperty.h"
#include "UObject/Package.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectThreadContext.h"

#if WITH_EDITOR
#include "EditorSubsystem.h"
#endif // WITH_EDITOR

DEFINE_LOG_CATEGORY(LogPython);

#define LOCTEXT_NAMESPACE "Python"

#if WITH_PYTHON

namespace UE::Python::Private
{

bool GIsRunningUserScript = false;

}

namespace PyUtil
{

const FName DefaultPythonPropertyName = "TransientPythonProperty";

FPyApiBuffer TCHARToPyApiBuffer(const TCHAR* InStr)
{
	auto PyCharToPyBuffer = [](const FPyApiChar* InPyChar)
	{
		int32 PyCharLen = 0;
		while (InPyChar[PyCharLen++] != 0) {} // Count includes the null terminator

		FPyApiBuffer PyBuffer;
		PyBuffer.Append(InPyChar, PyCharLen);
		return PyBuffer;
	};

	return PyCharToPyBuffer(TCHARToPyApiChar(InStr));
}

FString PyObjectToUEString(PyObject* InPyObj)
{
	if (PyUnicode_Check(InPyObj))
	{
		return PyStringToUEString(InPyObj);
	}

	if (FPyObjectPtr PyStrObj = FPyObjectPtr::StealReference(PyObject_Str(InPyObj)))
	{
		return PyStringToUEString(PyStrObj);
	}

	return FString();
}

FString PyStringToUEString(PyObject* InPyStr)
{
	FString Str;
	PyConversion::Nativize(InPyStr, Str, PyConversion::ESetErrorState::No);
	return Str;
}

FString PyObjectToUEStringRepr(PyObject* InPyObj)
{
	FPyObjectPtr PyReprObj = FPyObjectPtr::StealReference(PyObject_Repr(InPyObj));
	if (PyReprObj)
	{
		return PyStringToUEString(PyReprObj);
	}
	return PyObjectToUEString(InPyObj);
}

FEvalStack& FEvalStack::Get()
{
	static FEvalStack Instance;
	return Instance;
}

void FEvalStack::PushContext(FEvalContext&& Context)
{
	Stack.Push(MoveTemp(Context));
}

void FEvalStack::PopContext()
{
	Stack.Pop();
}

const FEvalStack::FEvalContext* FEvalStack::GetCurrentContext() const
{
	return Stack.Num() > 0
		? &Stack.Top()
		: nullptr;
}

FPropValueOnScope::FPropValueOnScope(FConstPropOnScope&& InProp)
	: Prop(MoveTemp(InProp))
{
	check(Prop);

	Value = FMemory::Malloc(Prop->GetSize(), Prop->GetMinAlignment());
	Prop->InitializeValue(Value);
}

FPropValueOnScope::~FPropValueOnScope()
{
	if (Value)
	{
		Prop->DestroyValue(Value);
		FMemory::Free(Value);
	}
}

bool FPropValueOnScope::SetValue(PyObject* InPyObj, const TCHAR* InErrorCtxt)
{
	check(IsValid());

	if (PyConversion::NativizeProperty(InPyObj, Prop, Value))
	{
		return true;
	}

	PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert '%s' to '%s' (%s)"), *PyUtil::GetFriendlyTypename(InPyObj), *Prop->GetName(), *Prop->GetClass()->GetName()));
	return false;
}

bool FPropValueOnScope::IsValid() const
{
	return Prop && Value;
}

const FProperty* FPropValueOnScope::GetProp() const
{
	return Prop;
}

void* FPropValueOnScope::GetValue(const int32 InArrayIndex) const
{
	check(InArrayIndex >= 0 && InArrayIndex < Prop->ArrayDim);
	return ((uint8*)Value) + (Prop->GetElementSize() * InArrayIndex);
}

FFixedArrayElementOnScope::FFixedArrayElementOnScope(const FProperty* InProp)
	: FPropValueOnScope(FConstPropOnScope::OwnedReference(PyUtil::CreateProperty(InProp))) // We have to create a new temporary property with an ArrayDim of 1
{
}

FArrayElementOnScope::FArrayElementOnScope(const FArrayProperty* InProp)
	: FPropValueOnScope(FConstPropOnScope::ExternalReference(InProp->Inner))
{
}

FSetElementOnScope::FSetElementOnScope(const FSetProperty* InProp)
	: FPropValueOnScope(FConstPropOnScope::ExternalReference(InProp->ElementProp))
{
}

FMapKeyOnScope::FMapKeyOnScope(const FMapProperty* InProp)
	: FPropValueOnScope(FConstPropOnScope::ExternalReference(InProp->KeyProp))
{
}

FMapValueOnScope::FMapValueOnScope(const FMapProperty* InProp)
	: FPropValueOnScope(FConstPropOnScope::ExternalReference(InProp->ValueProp))
{
}

FPropertyDef::FPropertyDef(const FProperty* InProperty)
	: PropertyClass(InProperty->GetClass())
	, PropertySubType(nullptr)
	, KeyDef()
	, ValueDef()
{
	if (const FObjectPropertyBase* ObjectProp = CastField<FObjectPropertyBase>(InProperty))
	{
		PropertySubType = ObjectProp->PropertyClass;
	}

	if (const FClassProperty* ClassProp = CastField<FClassProperty>(InProperty))
	{
		PropertySubType = ClassProp->MetaClass;
	}

	if (const FSoftClassProperty* ClassProp = CastField<FSoftClassProperty>(InProperty))
	{
		PropertySubType = ClassProp->MetaClass;
	}

	if (const FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
	{
		PropertySubType = StructProp->Struct;
	}

	if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(InProperty))
	{
		PropertySubType = EnumProp->GetEnum();
	}

	if (const FDelegateProperty* DelegateProp = CastField<FDelegateProperty>(InProperty))
	{
		PropertySubType = DelegateProp->SignatureFunction;
	}

	if (const FMulticastDelegateProperty* DelegateProp = CastField<FMulticastDelegateProperty>(InProperty))
	{
		PropertySubType = DelegateProp->SignatureFunction;
	}

	if (const FByteProperty* ByteProp = CastField<FByteProperty>(InProperty))
	{
		PropertySubType = ByteProp->Enum;
	}

	if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProperty))
	{
		ValueDef = MakeShared<FPropertyDef>(ArrayProp->Inner);
	}

	if (const FSetProperty* SetProp = CastField<FSetProperty>(InProperty))
	{
		ValueDef = MakeShared<FPropertyDef>(SetProp->ElementProp);
	}

	if (const FMapProperty* MapProp = CastField<FMapProperty>(InProperty))
	{
		KeyDef = MakeShared<FPropertyDef>(MapProp->KeyProp);
		ValueDef = MakeShared<FPropertyDef>(MapProp->ValueProp);
	}

	if (const FOptionalProperty* OptionalProp = CastField<FOptionalProperty>(InProperty))
	{
		ValueDef = MakeShared<FPropertyDef>(OptionalProp->GetValueProperty());
	}
}

bool CalculatePropertyDef(PyTypeObject* InPyType, FPropertyDef& OutPropertyDef)
{
	// It is a common error for a user to pass the container type directly
	// rather than an instance of it that defines the sub-types
	// eg) To pass "unreal.Map" rather than "unreal.Map(int, str)"
	// This tests for that case and emits a suitable error
	{
		if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperArrayType) == 1)
		{
			SetPythonError(PyExc_TypeError, InPyType, TEXT("Cannot create a property definition from 'Array' directly! It must be an instance specifying the element type, eg) 'Array(int)'."));
			return false;
		}

		if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperSetType) == 1)
		{
			SetPythonError(PyExc_TypeError, InPyType, TEXT("Cannot create a property definition from 'Set' directly! It must be an instance specifying the element type, eg) 'Set(int)'."));
			return false;
		}

		if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperMapType) == 1)
		{
			SetPythonError(PyExc_TypeError, InPyType, TEXT("Cannot create a property definition from 'Map' directly! It must be an instance specifying the key and value types, eg) 'Map(int, str)'."));
			return false;
		}
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperObjectType) == 1)
	{
		OutPropertyDef.PropertyClass = FObjectProperty::StaticClass();
		OutPropertyDef.PropertySubType = (UObject*)FPyWrapperObjectMetaData::GetClass(InPyType);
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperStructType) == 1)
	{
		OutPropertyDef.PropertyClass = FStructProperty::StaticClass();
		OutPropertyDef.PropertySubType = (UObject*)FPyWrapperStructMetaData::GetStruct(InPyType);
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperEnumType) == 1)
	{
		UEnum* EnumType = FPyWrapperEnumMetaData::GetEnum(InPyType);
		if (EnumType && EnumType->GetCppForm() == UEnum::ECppForm::EnumClass)
		{
			OutPropertyDef.PropertyClass = FEnumProperty::StaticClass();
		}
		else
		{
			OutPropertyDef.PropertyClass = FByteProperty::StaticClass();
		}
		OutPropertyDef.PropertySubType = (UObject*)EnumType;
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperDelegateType) == 1)
	{
		OutPropertyDef.PropertyClass = FDelegateProperty::StaticClass();
		OutPropertyDef.PropertySubType = ConstCast(FPyWrapperDelegateMetaData::GetDelegateSignature(InPyType).Func);
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperMulticastDelegateType) == 1)
	{
		OutPropertyDef.PropertyClass = FMulticastDelegateProperty::StaticClass();
		OutPropertyDef.PropertySubType = ConstCast(FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InPyType).Func);
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperNameType) == 1)
	{
		OutPropertyDef.PropertyClass = FNameProperty::StaticClass();
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperTextType) == 1)
	{
		OutPropertyDef.PropertyClass = FTextProperty::StaticClass();
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyWrapperFieldPathType) == 1)
	{
		OutPropertyDef.PropertyClass = FFieldPathProperty::StaticClass();
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyUnicode_Type) == 1)
	{
		OutPropertyDef.PropertyClass = FStrProperty::StaticClass();
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyBool_Type) == 1)
	{
		OutPropertyDef.PropertyClass = FBoolProperty::StaticClass();
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyLong_Type) == 1)
	{
		OutPropertyDef.PropertyClass = FIntProperty::StaticClass();
		return true;
	}

	if (PyObject_IsSubclass((PyObject*)InPyType, (PyObject*)&PyFloat_Type) == 1)
	{
		OutPropertyDef.PropertyClass = FFloatProperty::StaticClass();
		return true;
	}

	return false;
}

bool CalculatePropertyDef(PyObject* InPyObj, FPropertyDef& OutPropertyDef)
{
	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperArrayType) == 1)
	{
		FPyWrapperArray* PyArray = (FPyWrapperArray*)InPyObj;
		OutPropertyDef.PropertyClass = PyArray->ArrayProp->GetClass();
		OutPropertyDef.ValueDef = MakeShared<FPropertyDef>(PyArray->ArrayProp->Inner);
		return true;
	}

	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperSetType) == 1)
	{
		FPyWrapperSet* PySet = (FPyWrapperSet*)InPyObj;
		OutPropertyDef.PropertyClass = PySet->SetProp->GetClass();
		OutPropertyDef.ValueDef = MakeShared<FPropertyDef>(PySet->SetProp->ElementProp);
		return true;
	}

	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperMapType) == 1)
	{
		FPyWrapperMap* PyMap = (FPyWrapperMap*)InPyObj;
		OutPropertyDef.PropertyClass = PyMap->MapProp->GetClass();
		OutPropertyDef.KeyDef = MakeShared<FPropertyDef>(PyMap->MapProp->KeyProp);
		OutPropertyDef.ValueDef = MakeShared<FPropertyDef>(PyMap->MapProp->ValueProp);
		return true;
	}

	if (PyObject* PyValueObj = nullptr;
		IsOptionalPropertyDef(InPyObj, &PyValueObj))
	{
		FPropertyDef ValuePropertyDef;
		if (CalculatePropertyDef(PyValueObj, ValuePropertyDef))
		{
			OutPropertyDef.PropertyClass = FOptionalProperty::StaticClass();
			OutPropertyDef.ValueDef = MakeShared<FPropertyDef>(MoveTemp(ValuePropertyDef));
			return true;
		}
	}

	return CalculatePropertyDef(PyType_Check(InPyObj) ? (PyTypeObject*)InPyObj : Py_TYPE(InPyObj), OutPropertyDef);
}

bool IsOptionalPropertyDef(PyObject* InPyObj, PyObject** OutPyValueObj)
{
	if (OutPyValueObj)
	{
		*OutPyValueObj = nullptr;
	}

	// A tuple of two values, where one is None, is considered an optional property
	if (PyTuple_Check(InPyObj) && PyTuple_GET_SIZE(InPyObj) == 2)
	{
		PyObject* PyValueObj = nullptr;
		int32 NoneCount = 0;

		if (PyObject* PotentialPyValueObj = PyTuple_GET_ITEM(InPyObj, 0);
			PotentialPyValueObj != Py_None)
		{
			PyValueObj = PotentialPyValueObj;
		}
		else
		{
			++NoneCount;
		}

		if (PyObject* PotentialPyValueObj = PyTuple_GET_ITEM(InPyObj, 1);
			PotentialPyValueObj != Py_None)
		{
			PyValueObj = PotentialPyValueObj;
		}
		else
		{
			++NoneCount;
		}

		if (PyValueObj && NoneCount == 1)
		{
			if (OutPyValueObj)
			{
				*OutPyValueObj = PyValueObj;
			}
			return true;
		}
	}

	return false;
}

FProperty* CreateProperty(const FPropertyDef& InPropertyDef, const int32 InArrayDim, FFieldVariant InOuter, const FName InName)
{
	check(InArrayDim > 0);
	FProperty* Prop = CastFieldChecked<FProperty>(InPropertyDef.PropertyClass->Construct(InOuter, InName));
	if (Prop)
	{
		Prop->ArrayDim = InArrayDim;

		if (FObjectPropertyBase* ObjectProp = CastField<FObjectPropertyBase>(Prop))
		{
			UClass* ClassType = CastChecked<UClass>(InPropertyDef.PropertySubType);
			ObjectProp->SetPropertyClass(ClassType);
		}

		if (FClassProperty* ClassProp = CastField<FClassProperty>(Prop))
		{
			UClass* ClassType = CastChecked<UClass>(InPropertyDef.PropertySubType);
			ClassProp->SetPropertyClass(UClass::StaticClass());
			ClassProp->SetMetaClass(ClassType);
		}

		if (FSoftClassProperty* ClassProp = CastField<FSoftClassProperty>(Prop))
		{
			UClass* ClassType = CastChecked<UClass>(InPropertyDef.PropertySubType);
			ClassProp->SetPropertyClass(UClass::StaticClass());
			ClassProp->SetMetaClass(ClassType);
		}

		if (FStructProperty* StructProp = CastField<FStructProperty>(Prop))
		{
			UScriptStruct* StructType = CastChecked<UScriptStruct>(InPropertyDef.PropertySubType);
			StructProp->Struct = StructType;
		}

		if (FEnumProperty* EnumProp = CastField<FEnumProperty>(Prop))
		{
			UEnum* EnumType = CastChecked<UEnum>(InPropertyDef.PropertySubType);
			EnumProp->SetEnum(EnumType);
			EnumProp->AddCppProperty(new FByteProperty(EnumProp, TEXT("UnderlyingType")));
		}

		if (FDelegateProperty* DelegateProp = CastField<FDelegateProperty>(Prop))
		{
			UFunction* DelegateSignature = CastChecked<UFunction>(InPropertyDef.PropertySubType);
			DelegateProp->SignatureFunction = DelegateSignature;
		}

		if (FMulticastDelegateProperty* DelegateProp = CastField<FMulticastDelegateProperty>(Prop))
		{
			UFunction* DelegateSignature = CastChecked<UFunction>(InPropertyDef.PropertySubType);
			DelegateProp->SignatureFunction = DelegateSignature;
		}

		if (FByteProperty * ByteProp = CastField<FByteProperty>(Prop))
		{
			UEnum* EnumType = Cast<UEnum>(InPropertyDef.PropertySubType); // Not CastChecked as this may be an actual number rather than an enum
			ByteProp->Enum = EnumType;
		}

		if (FBoolProperty* BoolProp = CastField<FBoolProperty>(Prop))
		{
			BoolProp->SetBoolSize(sizeof(bool), true);
		}

		if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
		{
			ArrayProp->Inner = CreateProperty(*InPropertyDef.ValueDef, 1, ArrayProp);
		}

		if (FSetProperty* SetProp = CastField<FSetProperty>(Prop))
		{
			SetProp->ElementProp = CreateProperty(*InPropertyDef.ValueDef, 1, SetProp);
		}

		if (FMapProperty* MapProp = CastField<FMapProperty>(Prop))
		{
			MapProp->KeyProp = CreateProperty(*InPropertyDef.KeyDef, 1, MapProp);
			MapProp->ValueProp = CreateProperty(*InPropertyDef.ValueDef, 1, MapProp);
		}

		if (FOptionalProperty* OptionalProp = CastField<FOptionalProperty>(Prop))
		{
			OptionalProp->SetValueProperty(CreateProperty(*InPropertyDef.ValueDef, 1, OptionalProp));
		}

		// Need to manually call Link to fix-up some data (such as the C++ property flags) that are only set during Link
		{
			FArchive Ar;
			Prop->LinkWithoutChangingOffset(Ar);
		}
	}

	return Prop;
}

FProperty* CreateProperty(PyTypeObject* InPyType, const int32 InArrayDim, FFieldVariant InOuter, const FName InName)
{
	FPropertyDef PropertyDef;
	return CalculatePropertyDef(InPyType, PropertyDef) ? CreateProperty(PropertyDef, InArrayDim, InOuter, InName) : nullptr;
}

FProperty* CreateProperty(PyObject* InPyObj, const int32 InArrayDim, FFieldVariant InOuter, const FName InName)
{
	FPropertyDef PropertyDef;
	return CalculatePropertyDef(InPyObj, PropertyDef) ? CreateProperty(PropertyDef, InArrayDim, InOuter, InName) : nullptr;
}

bool IsInputParameter(const FProperty* InParam)
{
	const bool bIsParam = InParam->HasAnyPropertyFlags(CPF_Parm);
	const bool bIsReturnParam = InParam->HasAnyPropertyFlags(CPF_ReturnParm);
	const bool bIsReferenceParam = InParam->HasAnyPropertyFlags(CPF_ReferenceParm);
	const bool bIsOutParam = InParam->HasAnyPropertyFlags(CPF_OutParm) && !InParam->HasAnyPropertyFlags(CPF_ConstParm);
	return bIsParam && !bIsReturnParam && (!bIsOutParam || bIsReferenceParam);
}

bool IsOutputParameter(const FProperty* InParam)
{
	const bool bIsParam = InParam->HasAnyPropertyFlags(CPF_Parm);
	const bool bIsReturnParam = InParam->HasAnyPropertyFlags(CPF_ReturnParm);
	const bool bIsOutParam = InParam->HasAnyPropertyFlags(CPF_OutParm) && !InParam->HasAnyPropertyFlags(CPF_ConstParm);
	return bIsParam && !bIsReturnParam && bIsOutParam;
}

void ImportDefaultValue(const FProperty* InProp, void* InPropValue, const FString& InDefaultValue)
{
	PropertyAccessUtil::ImportDefaultPropertyValue(InProp, InPropValue, InDefaultValue);
}

bool InvokeFunctionCall(UObject* InObj, const UFunction* InFunc, void* InBaseParamsAddr, const TCHAR* InErrorCtxt)
{
	// FEditorScriptExecutionGuard isn't thread-safe and most function calls might also not be thread-safe.
	// Python editor code that creates threads and interact with the Unreal API will always hit this case 
	// Instead of crashing, raise a Python Exception to alert the user
	if (!IsInGameThread())
	{
		FPyScopedGIL GIL;
		SetPythonError(PyExc_RuntimeError, InErrorCtxt, TEXT("Attempted to access Unreal API from outside the main game thread."));
		return false;
	}

	bool bThrewException = false;
	FScopedScriptExceptionHandler ExceptionHandler([InErrorCtxt, &bThrewException](ELogVerbosity::Type Verbosity, const TCHAR* ExceptionMessage, const TCHAR* StackMessage)
	{
		if (Verbosity == ELogVerbosity::Error)
		{
			FPyScopedGIL GIL;
			SetPythonError(PyExc_RuntimeError, InErrorCtxt, ExceptionMessage);
			bThrewException = true;
		}
		else if (Verbosity == ELogVerbosity::Warning)
		{
			FPyScopedGIL GIL;
			if (SetPythonWarning(PyExc_RuntimeWarning, InErrorCtxt, ExceptionMessage) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				bThrewException = true;
			}
		}
		else
		{
#if !NO_LOGGING
			FMsg::Logf_Internal(__FILE__, __LINE__, LogPython.GetCategoryName(), Verbosity, TEXT("%s"), ExceptionMessage);
#endif
		}
	});

	FEditorScriptExecutionGuard ScriptGuard;
	Py_BEGIN_ALLOW_THREADS
	InObj->ProcessEvent((UFunction*)InFunc, InBaseParamsAddr);
	Py_END_ALLOW_THREADS

	return !bThrewException;
}

bool InspectFunctionArgs(PyObject* InFunc, TArray<FString>& OutArgNames, TArray<FPyObjectPtr>* OutArgDefaults)
{
	if (!PyFunction_Check(InFunc) && !PyMethod_Check(InFunc))
	{
		return false;
	}

	FPyObjectPtr PyInspectModule = FPyObjectPtr::StealReference(PyImport_ImportModule("inspect"));
	if (PyInspectModule)
	{
		PyObject* PyInspectDict = PyModule_GetDict(PyInspectModule);
		PyObject* PyGetArgSpecFunc = PyDict_GetItemString(PyInspectDict, "getfullargspec");
		if (PyGetArgSpecFunc)
		{
			FPyObjectPtr PyGetArgSpecResult = FPyObjectPtr::StealReference(PyObject_CallFunctionObjArgs(PyGetArgSpecFunc, InFunc, nullptr));
			if (PyGetArgSpecResult)
			{
				PyObject* PyFuncArgNames = PyTuple_GetItem(PyGetArgSpecResult, 0);
				const int32 NumArgNames = (PyFuncArgNames && PyFuncArgNames != Py_None) ? PySequence_Size(PyFuncArgNames) : 0;

				PyObject* PyFuncArgDefaults = PyTuple_GetItem(PyGetArgSpecResult, 3);
				const int32 NumArgDefaults = (PyFuncArgDefaults && PyFuncArgDefaults != Py_None) ? PySequence_Size(PyFuncArgDefaults) : 0;

				OutArgNames.Reset(NumArgNames);
				if (OutArgDefaults)
				{
					OutArgDefaults->Reset(NumArgNames);
				}

				// Get the names
				for (int32 ArgNameIndex = 0; ArgNameIndex < NumArgNames; ++ArgNameIndex)
				{
					PyObject* PyArgName = PySequence_GetItem(PyFuncArgNames, ArgNameIndex);
					OutArgNames.Emplace(PyObjectToUEString(PyArgName));
				}

				// Get the defaults (padding the start of the array with empty strings)
				if (OutArgDefaults)
				{
					OutArgDefaults->AddDefaulted(NumArgNames - NumArgDefaults);
					for (int32 ArgDefaultIndex = 0; ArgDefaultIndex < NumArgDefaults; ++ArgDefaultIndex)
					{
						PyObject* PyArgDefault = PySequence_GetItem(PyFuncArgDefaults, ArgDefaultIndex);
						OutArgDefaults->Emplace(FPyObjectPtr::NewReference(PyArgDefault));
					}
				}

				check(!OutArgDefaults || OutArgNames.Num() == OutArgDefaults->Num());
				return true;
			}
		}
	}

	return false;
}

int ValidateContainerTypeParam(PyObject* InPyObj, FPropertyDef& OutPropDef, const char* InPythonArgName, const TCHAR* InErrorCtxt)
{
	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperArrayType) == 1 ||
		PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperSetType) == 1 ||
		PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperMapType) == 1
		)
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("'%s' (%s) cannot be a container element type (directly nested containers are not supported - consider using an intermediary struct instead)"), UTF8_TO_TCHAR(InPythonArgName), *GetFriendlyTypename(InPyObj)));
		return -1;
	}

	if (PyType_Check(InPyObj) != 1)
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("'%s' (%s) must be a type"), UTF8_TO_TCHAR(InPythonArgName), *GetFriendlyTypename(InPyObj)));
		return -1;
	}

	if (!CalculatePropertyDef((PyTypeObject*)InPyObj, OutPropDef))
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert '%s' (%s) to a 'Property' class"), UTF8_TO_TCHAR(InPythonArgName), *GetFriendlyTypename(InPyObj)));
		return -1;
	}

	if (OutPropDef.KeyDef.IsValid() || OutPropDef.ValueDef.IsValid())
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("'%s' (%s) cannot be a container element type"), UTF8_TO_TCHAR(InPythonArgName), *GetFriendlyTypename(InPyObj)));
		return -1;
	}

	if (OutPropDef.PropertyClass->HasAnyClassFlags(CLASS_Abstract))
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("'%s' (%s) converted to '%s' which is an abstract 'Property' class"), UTF8_TO_TCHAR(InPythonArgName), *GetFriendlyTypename(InPyObj), *OutPropDef.PropertyClass->GetName()));
		return -1;
	}

	return 0;
}

int ValidateContainerLenParam(PyObject* InPyObj, int32 &OutLen, const char* InPythonArgName, const TCHAR* InErrorCtxt)
{
	if (!PyConversion::Nativize(InPyObj, OutLen))
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert '%s' (%s) to 'int32'"), UTF8_TO_TCHAR(InPythonArgName), *GetFriendlyTypename(InPyObj)));
		return -1;
	}

	if (OutLen < 0)
	{
		SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("'%s' must be positive"), UTF8_TO_TCHAR(InPythonArgName)));
		return -1;
	}

	return 0;
}

int ValidateContainerIndexParam(const Py_ssize_t InIndex, const Py_ssize_t InLen, const FProperty* InProp, const TCHAR* InErrorCtxt)
{
	if (InIndex < 0 || InIndex >= InLen)
	{
		SetPythonError(PyExc_IndexError, InErrorCtxt, *FString::Printf(TEXT("Index %zd is out-of-bounds (len: %zd) for property '%s' (%s)"), InIndex, InLen, *InProp->GetName(), *InProp->GetClass()->GetName()));
		return -1;
	}

	return 0;
}

Py_ssize_t ResolveContainerIndexParam(const Py_ssize_t InIndex, const Py_ssize_t InLen)
{
	return InIndex < 0 ? InIndex + InLen : InIndex;
}

UObject* NewObject(UClass* InObjClass, UObject* InObjectOuter, const FName InObjectName, UClass* InBaseClass, const TCHAR* InErrorCtxt)
{
	if (InObjClass)
	{
		if (InObjClass->IsChildOf<UBlueprintFunctionLibrary>())
		{
			// Starting with UE 5.5, generates a warning/deprecation message.
			SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FString::Printf(TEXT("Creating an instance of a BlueprintFunctionLibrary has been deprecated since UE 5.5 and will be removed in the future. Call its classmethods directly on the class, eg, 'unreal.%s.foo()'."), *PyGenUtil::GetClassPythonName(InObjClass)));

			// For UE 5.7 or later, generate an hard error.
			//SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Cannot create an instance of a BlueprintFunctionLibrary. Call its classmethods directly on the class, eg, 'unreal.%s.foo()'."), *PyGenUtil::GetClassPythonName(InObjClass)));
			//return nullptr;
		}
		else if (InObjClass->IsChildOf<UEngineSubsystem>())
		{
			// Starting with UE 5.2, generates a warning/deprecation message.
			SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FString::Printf(TEXT("Creating an instance of an Engine subsystem has been deprecated since UE 5.2 and will be removed in the future. Use 'unreal.get_engine_subsystem(unreal.%s)' to get the subsystem instance."), *PyGenUtil::GetClassPythonName(InObjClass)));

			// For UE 5.3 or later, generate an hard error.
			//SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Cannot create an instance of an Engine subsystem. Use 'unreal.get_engine_subsystem(unreal.%s)' to get the subsystem instance."), *PyGenUtil::GetClassPythonName(InObjClass)));
			//return nullptr;
		}
#if WITH_EDITOR
		else if (InObjClass->IsChildOf<UEditorSubsystem>())
		{
			// Starting with UE 5.2, generates a warning/deprecation message.
			SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FString::Printf(TEXT("Creating an instance of an Editor subsystem has been deprecated since UE 5.2 and will be removed in the future. Use 'unreal.get_editor_subsystem(unreal.%s)' to get the subsystem instance."), *PyGenUtil::GetClassPythonName(InObjClass)));

			// For UE 5.3 or later, generate an hard error.
			//SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Cannot create an instance of an Editor subsystem. Use 'unreal.get_editor_subsystem(unreal.%s)' to get the subsystem instance."), *PyGenUtil::GetClassPythonName(InObjClass)));
			//return nullptr;
		}
#endif

		if (InObjClass == UPackage::StaticClass())
		{
			if (InObjectName.IsNone())
			{
				SetPythonError(PyExc_Exception, InErrorCtxt, TEXT("Name cannot be 'None' when creating a 'Package'"));
				return nullptr;
			}
		}
		else if (!InObjectOuter)
		{
			SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Outer cannot be null when creating a '%s'"), *InObjClass->GetName()));
			return nullptr;
		}

		if (InObjectOuter && !InObjectOuter->IsA(InObjClass->ClassWithin))
		{
			SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Outer '%s' was of type '%s' but must be of type '%s'"), *InObjectOuter->GetPathName(), *InObjectOuter->GetClass()->GetName(), *InObjClass->ClassWithin->GetName()));
			return nullptr;
		}

		if (InBaseClass && !InObjClass->IsChildOf(InBaseClass))
		{
			SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Class was of type '%s' but must be of type '%s'"), *InObjClass->GetName(), *InBaseClass->GetName()));
			return nullptr;
		}

		if (InObjClass->HasAnyClassFlags(CLASS_Abstract))
		{
			SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Class '%s' is abstract"), *InObjClass->GetName()));
			return nullptr;
		}

		UObject* ObjectInstance = ::NewObject<UObject>(InObjectOuter, InObjClass, InObjectName, RF_Transactional);
		if (!ObjectInstance)
		{
			SetPythonError(PyExc_Exception, InErrorCtxt, TEXT("NewObject returned a null instance"));
			return nullptr;
		}

		return ObjectInstance;
	}
	else
	{
		SetPythonError(PyExc_Exception, InErrorCtxt, TEXT("Class is null"));
		return nullptr;
	}
}

UObject* GetOwnerObject(PyObject* InPyObj)
{
	FPyWrapperOwnerContext OwnerContext = FPyWrapperOwnerContext(InPyObj);
	while (OwnerContext.HasOwner())
	{
		PyObject* PyObj = OwnerContext.GetOwnerObject();

		if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperObjectType) == 1)
		{
			// Found an object, this is the end of the chain
			return ((FPyWrapperObject*)PyObj)->ObjectInstance;
		}

		if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperStructType) == 1)
		{
			// Found a struct, recurse up the chain
			OwnerContext = ((FPyWrapperStruct*)PyObj)->OwnerContext;
			continue;
		}

		// Unknown object type - just bail
		break;
	}

	return nullptr;
}

PyObject* GetPropertyValue(const UStruct* InStruct, const void* InStructData, const FProperty* InProp, const char *InAttributeName, PyObject* InOwnerPyObject, const TCHAR* InErrorCtxt)
{
	if (InStruct && InProp && ensureAlways(InStructData))
	{
		const EPropertyAccessResultFlags AccessResult = PropertyAccessUtil::CanGetPropertyValue(InProp);
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
		{
			if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::AccessProtected))
		{
			SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' is protected and cannot be read"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
			return nullptr;
		}

			SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' cannot be read"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
			return nullptr;
		}

		PyObject* PyPropObj = nullptr;
		if (!PyConversion::PythonizeProperty_InContainer(InProp, InStructData, 0, PyPropObj, EPyConversionMethod::Reference, InOwnerPyObject))
		{
			SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert property '%s' (%s) for attribute '%s' on '%s'"), *InProp->GetName(), *InProp->GetClass()->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
			return nullptr;
		}
		return PyPropObj;
	}

	Py_RETURN_NONE;
}

int SetPropertyValue(const UStruct* InStruct, void* InStructData, PyObject* InValue, const FProperty* InProp, const char *InAttributeName, const FPropertyAccessChangeNotify* InChangeNotify, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const TCHAR* InErrorCtxt, const TConstArrayView<void*>& InArchetypeInstStructData)
{
	if (!InValue)
	{
		SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Cannot delete attribute '%s' from '%s'"), UTF8_TO_TCHAR(InAttributeName), (InStruct ? *InStruct->GetName() : TEXT(""))));
		return -1;
	}

	if (InStruct && InProp && ensureAlways(InStructData))
	{
		const EPropertyAccessResultFlags AccessResult = PropertyAccessUtil::CanSetPropertyValue(InProp, InReadOnlyFlags, InOwnerIsTemplate);
		if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::PermissionDenied))
		{
			if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::AccessProtected))
			{
				SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' is protected and cannot be set"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
				return -1;
			}

			if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::CannotEditTemplate))
			{
				SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' cannot be edited on templates"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
				return -1;
			}

			if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::CannotEditInstance))
			{
				SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' cannot be edited on instances"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
				return -1;
			}

			if (EnumHasAnyFlags(AccessResult, EPropertyAccessResultFlags::ReadOnly))
			{
				SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' is read-only and cannot be set"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
				return -1;
			}

			SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Property '%s' for attribute '%s' on '%s' cannot be set"), *InProp->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
			return -1;
		}

		if (!PyConversion::NativizeProperty_InContainer(InValue, InProp, InStructData, 0, InArchetypeInstStructData, InChangeNotify))
		{
			SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert type '%s' to property '%s' (%s) for attribute '%s' on '%s'"), *GetFriendlyTypename(InValue), *InProp->GetName(), *InProp->GetClass()->GetName(), UTF8_TO_TCHAR(InAttributeName), *InStruct->GetName()));
			return -1;
		}
	}

	return 0;
}

bool HasLength(PyObject* InObj)
{
	return HasLength(Py_TYPE(InObj)) && PyObject_Length(InObj) != -1;
}

bool HasLength(PyTypeObject* InType)
{
	return InType->tp_dict && PyDict_GetItemString(InType->tp_dict, "__len__");
}

bool IsMappingType(PyObject* InObj)
{
	return HasLength(InObj) && IsMappingType(Py_TYPE(InObj));
}

bool IsMappingType(PyTypeObject* InType)
{
	// We use the existing of a "keys" function here as:
	//   1) PyMapping_Check isn't accurate as sequence types use some mapping functions to enable slicing.
	//   2) PySequence_Check excludes sets as they don't provide random element access.
	// This will detect 'dict' and 'TMap' (FPyWrapperMap) as they both implement a "keys" function, which no sequence type does.
	return InType->tp_dict && PyDict_GetItemString(InType->tp_dict, "keys");
}

void FOnDiskModules::AddModules(const TCHAR* InPath)
{
	IFileManager& FileManager = IFileManager::Get();
	FileManager.IterateDirectory(InPath, [this, &FileManager](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		FString FullPath = FPaths::ConvertRelativePathToFull(FilenameOrDirectory);

		bool bPassedWildcard = true;
		if (!ModuleNameWildcard.IsEmpty())
		{
			// TODO: Would be nice to use FPathsView and FStringView here, but FStringView doesn't implement MatchesWildcard (though the implementation could easily be ported)
			const FString CleanName = FPaths::GetCleanFilename(FullPath);
			bPassedWildcard = CleanName.MatchesWildcard(ModuleNameWildcard, ESearchCase::CaseSensitive);
		}

		if (bPassedWildcard)
		{
			if (bIsDirectory)
			{
				FullPath /= TEXT("__init__.py");
				if (FileManager.FileExists(*FullPath))
				{
					CachedModules.Add(MoveTemp(FullPath));
				}
			}
			else if (FPathViews::GetExtension(FullPath) == TEXT("py"))
			{
				CachedModules.Add(MoveTemp(FullPath));
			}
		}

		return true;
	});
}

void FOnDiskModules::RemoveModules(const TCHAR* InPath)
{
	for (auto It = CachedModules.CreateIterator(); It; ++It)
	{
		if (It->StartsWith(InPath))
		{
			It.RemoveCurrent();
		}
	}
}

bool FOnDiskModules::HasModule(const TCHAR* InModuleName, FString* OutResolvedFile) const
{
	const FString ModuleSingleFile = FString::Printf(TEXT("/%s.py"), InModuleName);
	const FString ModuleFolderName = FString::Printf(TEXT("/%s/__init__.py"), InModuleName);

	for (const FString& CachedModule : CachedModules)
	{
		if (CachedModule.EndsWith(ModuleSingleFile, ESearchCase::CaseSensitive) || CachedModule.EndsWith(ModuleFolderName, ESearchCase::CaseSensitive))
		{
			if (OutResolvedFile)
			{
				*OutResolvedFile = CachedModule;
			}

			return true;
		}
	}

	return false;
}

FOnDiskModules& GetOnDiskUnrealModulesCache()
{
	static FOnDiskModules OnDiskUnrealModules(TEXT("unreal_*"));
	return OnDiskUnrealModules;
}

bool IsModuleAvailableForImport(const TCHAR* InModuleName, const FOnDiskModules* InOnDiskModules, FString* OutResolvedFile)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PyUtil::IsModuleAvailableForImport)

	// Check the sys.modules table first since it avoids hitting the filesystem
	if (PyObject* PyModulesDict = PySys_GetObject(PyCStrCast("modules")))
	{
		if (PyObject* PyModuleValue = PyDict_GetItemString(PyModulesDict, TCHAR_TO_UTF8(InModuleName)))
		{
			if (OutResolvedFile)
			{
				PyObject* PyModuleDict = PyModule_GetDict(PyModuleValue);
				if (PyObject* PyModuleFile = PyDict_GetItemString(PyModuleDict, "__file__"))
				{
					*OutResolvedFile = PyObjectToUEString(PyModuleFile);
				}
			}

			return true;
		}
	}

	// Use the on-disk modules cache, if available, to avoid hitting the filesystem
	if (InOnDiskModules)
	{
		return InOnDiskModules->HasModule(InModuleName, OutResolvedFile);
	}

	// Check the sys.path list looking for bla.py or bla/__init__.py
	if (PyObject* PyPathList = PySys_GetObject(PyCStrCast("path")))
	{
		const FString ModuleSingleFile = FString::Printf(TEXT("%s.py"), InModuleName);
		const FString ModuleFolderName = FString::Printf(TEXT("%s/__init__.py"), InModuleName);

		const Py_ssize_t PathListSize = PyList_Size(PyPathList);
		for (Py_ssize_t PathListIndex = 0; PathListIndex < PathListSize; ++PathListIndex)
		{
			PyObject* PyPathItem = PyList_GetItem(PyPathList, PathListIndex);
			if (PyPathItem)
			{
				const FString CurPath = PyObjectToUEString(PyPathItem);

				if (FPaths::FileExists(CurPath / ModuleSingleFile))
				{
					if (OutResolvedFile)
					{
						*OutResolvedFile = CurPath / ModuleSingleFile;
					}

					return true;
				}

				if (FPaths::FileExists(CurPath / ModuleFolderName))
				{
					if (OutResolvedFile)
					{
						*OutResolvedFile = CurPath / ModuleFolderName;
					}

					return true;
				}
			}
		}
	}

	return false;
}

bool IsModuleImported(const TCHAR* InModuleName, PyObject** OutPyModule)
{
	if (PyObject* PyModulesDict = PySys_GetObject(PyCStrCast("modules")))
	{
		if (PyObject* PyModuleValue = PyDict_GetItemString(PyModulesDict, TCHAR_TO_UTF8(InModuleName)))
		{
			if (OutPyModule)
			{
				*OutPyModule = PyModuleValue;
			}

			return true;
		}
	}

	return false;
}

FString GetInterpreterExecutablePath(bool* OutIsEnginePython)
{
	// Build the full Python directory (UE_PYTHON_DIR may be relative to UE engine directory for portability)
	FString PythonPath = UTF8_TO_TCHAR(UE_PYTHON_DIR);
	
	if (OutIsEnginePython)
	{
		*OutIsEnginePython = PythonPath.Contains(TEXT("{ENGINE_DIR}"), ESearchCase::CaseSensitive);
	}
	PythonPath.ReplaceInline(TEXT("{ENGINE_DIR}"), *FPaths::EngineDir(), ESearchCase::CaseSensitive);

	FPaths::NormalizeDirectoryName(PythonPath);
	FPaths::RemoveDuplicateSlashes(PythonPath);

#if PLATFORM_WINDOWS
	PythonPath /= TEXT("python.exe");
#elif PLATFORM_MAC || PLATFORM_LINUX
	PythonPath /= TEXT("bin/python3");
#else
	static_assert(false, "Python not supported on this platform!");
#endif

	PythonPath = FPaths::ConvertRelativePathToFull(PythonPath);

	return PythonPath;
}

FString GetArchString()
{
#if PLATFORM_CPU_ARM_FAMILY
	return TEXT("Arm64");
#else
	return TEXT("X64");
#endif //PLATFORM_CPU_ARM_FAMILY
}

const TArray<FString>& GetSitePackageSubdirs()
{
	const static TArray<FString> CheckSitePackageDirs = {
		FPaths::Combine(TEXT("Lib"), FPlatformMisc::GetUBTPlatform(), PyUtil::GetArchString(), TEXT("site-packages")),
		FPaths::Combine(TEXT("Lib"), FPlatformMisc::GetUBTPlatform(), TEXT("site-packages")),
		FPaths::Combine(TEXT("Lib"), TEXT("site-packages")),
	};
	return CheckSitePackageDirs;
}
	

void AddSitePackagesPath(const FString& InPath)
{
	if (!IFileManager::Get().DirectoryExists(*InPath))
	{
		return;
	}
	
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 13
	// We can safely call site.addsitedir on Python 3.13 as it is now decoded first with utf-8 https://docs.python.org/3/library/site.html
	if (FPyObjectPtr PySiteModule = FPyObjectPtr::StealReference(PyImport_ImportModule("site")))
	{
		PyObject* PySiteDict = PyModule_GetDict(PySiteModule);
		if (PyObject* PyAddSiteDirFunc = PyDict_GetItemString(PySiteDict, "addsitedir"))
		{
			FPyObjectPtr PyPath;
			if (PyConversion::Pythonize(InPath, PyPath.Get(), PyConversion::ESetErrorState::No))
			{
				FPyObjectPtr PyAddSiteDirResult = FPyObjectPtr::StealReference(PyObject_CallFunctionObjArgs(PyAddSiteDirFunc, PyPath.Get(), nullptr));
				
				if (!PyAddSiteDirResult)
				{
					PyUtil::LogPythonError(); // Ensure the error is logged and cleared
				}
			}
		}
	}
#else
	// Before Python 3.13, we cannot call site.addsitedir directly as it uses a TextIOWrapper with a default "locale" encoding to read .pth files 
	// (which is inconsistent depending on the platform) while we need the .pth to be consistently read as utf-8.
	// As a workaround, we temporarily redefine TextIOWrapper to force a utf-8 encoding
	FString PyCommandStr = FString::Printf(TEXT(
	"import io\n"
	"import site\n"
	
	"old_TextIOWrapper = io.TextIOWrapper\n" // Save the existing io.TextIOWrapper implementation
	"def TextIOWrapper_utf8(buffer, encoding = None):\n"
	"    return old_TextIOWrapper(buffer, encoding='utf-8')\n" // Create a new io.TextIOWrapper forcing the encoding to be utf-8
	
	"io.TextIOWrapper = TextIOWrapper_utf8\n" // Redefine io.TextIOWrapper
	
	"try:\n"
	"    site.addsitedir(r\"%s\")\n" // sets the sitedir
	"finally:\n"
	"    io.TextIOWrapper = old_TextIOWrapper\n" // Reset io.TextIOWrapper to the original implementation
	), *InPath);
	
	ensure(IPythonScriptPlugin::Get()->ExecPythonCommand(*PyCommandStr));
#endif
}

void AddSystemPath(const FString& InPath)
{
	if (!IFileManager::Get().DirectoryExists(*InPath))
	{
		return;
	}

	if (PyObject* PyPathList = PySys_GetObject(PyCStrCast("path")))
	{
		FPyObjectPtr PyPath;
		if (PyConversion::Pythonize(InPath, PyPath.Get(), PyConversion::ESetErrorState::No))
		{
			if (PySequence_Contains(PyPathList, PyPath) != 1)
			{
				PyList_Append(PyPathList, PyPath);
			}
		}

	}
}

void RemoveSystemPath(const FString& InPath)
{
	if (PyObject* PyPathList = PySys_GetObject(PyCStrCast("path")))
	{
		FPyObjectPtr PyPath;
		if (PyConversion::Pythonize(InPath, PyPath.Get(), PyConversion::ESetErrorState::No))
		{
			if (PySequence_Contains(PyPathList, PyPath) == 1)
			{
				PySequence_DelItem(PyPathList, PySequence_Index(PyPathList, PyPath));
			}
		}
	}
}

TArray<FString> GetSystemPaths()
{
	TArray<FString> Paths;

	if (PyObject* PyPathList = PySys_GetObject(PyCStrCast("path")))
	{
		const Py_ssize_t PyPathLen = PyList_Size(PyPathList);
		for (Py_ssize_t PyPathIndex = 0; PyPathIndex < PyPathLen; ++PyPathIndex)
		{
			PyObject* PyPathItem = PyList_GetItem(PyPathList, PyPathIndex);
			Paths.Add(PyObjectToUEString(PyPathItem));
		}
	}

	return Paths;
}

FString GetDocString(PyObject* InPyObj)
{
	FString DocString;
	if (FPyObjectPtr DocStringObj = FPyObjectPtr::StealReference(PyObject_GetAttrString(InPyObj, "__doc__")))
	{
		DocString = PyStringToUEString(DocStringObj);
	}
	return DocString;
}

FString GetFriendlyStructValue(const UScriptStruct* InStruct, const void* InStructValue, const uint32 InPortFlags)
{
	FString FriendlyStructValue;

	if (PyTypeObject* PyStructType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(InStruct))
	{
		FriendlyStructValue += TEXT('{');

		FPyWrapperStructMetaData* StructMetaData = FPyWrapperStructMetaData::GetMetaData(PyStructType);

		bool bFirst = true;
		for (const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam : StructMetaData->InitParams)
		{
			if (!bFirst)
			{
				FriendlyStructValue += TEXT(", ");
			}
			bFirst = false;

			FriendlyStructValue += UTF8_TO_TCHAR(InitParam.ParamName.GetData());
			FriendlyStructValue += TEXT(": ");

			FriendlyStructValue += GetFriendlyPropertyValue(InitParam.ParamProp, InitParam.ParamProp->ContainerPtrToValuePtr<void>(InStructValue), InPortFlags | PPF_Delimited);
		}

		FriendlyStructValue += TEXT('}');
	}
	else
	{
		check(InStruct);
		InStruct->ExportText(FriendlyStructValue, InStructValue, InStructValue, nullptr, InPortFlags, nullptr);
	}

	return FriendlyStructValue;
}

FString GetFriendlyPropertyValue(const FProperty* InProp, const void* InPropValue, const uint32 InPortFlags)
{
	if (auto* CastProp = CastField<FStructProperty>(InProp))
	{
		return GetFriendlyStructValue(CastProp->Struct, InPropValue, InPortFlags);
	}
	
	FString FriendlyPropertyValue;
	InProp->ExportTextItem_Direct(FriendlyPropertyValue, InPropValue, InPropValue, nullptr, InPortFlags, nullptr);
	return FriendlyPropertyValue;
}

FString GetFriendlyTypename(PyTypeObject* InPyType)
{
	return UTF8_TO_TCHAR(InPyType->tp_name);
}

FString GetFriendlyTypename(PyObject* InPyObj)
{
	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperArrayType) == 1)
	{
		FPyWrapperArray* PyArray = (FPyWrapperArray*)InPyObj;
		const FString PropTypeName = PyArray->ArrayProp->Inner ? PyArray->ArrayProp->Inner->GetClass()->GetName() : FString();
		return FString::Printf(TEXT("%s (%s)"), UTF8_TO_TCHAR(Py_TYPE(InPyObj)->tp_name), *PropTypeName);
	}

	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperFixedArrayType) == 1)
	{
		FPyWrapperFixedArray* PyFixedArray = (FPyWrapperFixedArray*)InPyObj;
		const FString PropTypeName = PyFixedArray->ArrayProp ? PyFixedArray->ArrayProp->GetClass()->GetName() : FString();
		return FString::Printf(TEXT("%s (%s)"), UTF8_TO_TCHAR(Py_TYPE(InPyObj)->tp_name), *PropTypeName);
	}

	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperSetType) == 1)
	{
		FPyWrapperSet* PySet = (FPyWrapperSet*)InPyObj;
		const FString PropTypeName = PySet->SetProp ? PySet->SetProp->ElementProp->GetClass()->GetName() : FString();
		return FString::Printf(TEXT("%s (%s)"), UTF8_TO_TCHAR(Py_TYPE(InPyObj)->tp_name), *PropTypeName);
	}

	if (PyObject_IsInstance(InPyObj, (PyObject*)&PyWrapperMapType) == 1)
	{
		FPyWrapperMap* PyMap = (FPyWrapperMap*)InPyObj;
		const FString PropKeyName = PyMap->MapProp ? PyMap->MapProp->KeyProp->GetClass()->GetName() : FString();
		const FString PropTypeName = PyMap->MapProp ? PyMap->MapProp->ValueProp->GetClass()->GetName() : FString();
		return FString::Printf(TEXT("%s (%s, %s)"), UTF8_TO_TCHAR(Py_TYPE(InPyObj)->tp_name), *PropKeyName, *PropTypeName);
	}

	return GetFriendlyTypename(PyType_Check(InPyObj) ? (PyTypeObject*)InPyObj : Py_TYPE(InPyObj));
}

FString GetCleanTypename(PyTypeObject* InPyType)
{
	FString Typename = UTF8_TO_TCHAR(InPyType->tp_name);

	int32 LastDotIndex = INDEX_NONE;
	if (Typename.FindLastChar(TEXT('.'), LastDotIndex))
	{
		Typename.RemoveAt(0, LastDotIndex + 1, EAllowShrinking::No);
	}

	return Typename;
}

FString GetCleanTypename(PyObject* InPyObj)
{
	return GetCleanTypename(PyType_Check(InPyObj) ? (PyTypeObject*)InPyObj : Py_TYPE(InPyObj));
}

void GetGeneratedTypeOuterAndName(PyTypeObject* InPyType, UObject*& OutOuter, FString& OutName)
{
	OutOuter = GetPythonTypeContainer();
	OutName = GetCleanTypename(InPyType);

	FString TypeFilename;

	// Favor "inspect.getfile" if possible, as this will return the correct information for files executed via an import
	if (FPyObjectPtr PyInspectModule = FPyObjectPtr::StealReference(PyImport_ImportModule("inspect")))
	{
		PyObject* PyInspectDict = PyModule_GetDict(PyInspectModule);
		if (PyObject* PyGetFileFunc = PyDict_GetItemString(PyInspectDict, "getfile"))
		{
			if (FPyObjectPtr PyGetFileResult = FPyObjectPtr::StealReference(PyObject_CallFunctionObjArgs(PyGetFileFunc, InPyType, nullptr)))
			{
				TypeFilename = PyObjectToUEString(PyGetFileResult);
			}
			else
			{
				// Clear any exception information if getfile failed
				PyErr_Clear();
			}
		}
	}

	// If "inspect.getfile" failed then try and access the current "__file__", as that will work for files being directly executed (ie, not an import)
	if (TypeFilename.IsEmpty())
	{
		// Note: This doesn't use PyEval_GetGlobals() as that will return the result for the current "frame", which may be from an intermediate file (eg, unreal_core.py due to the unreal.uthing() decorator)
		if (const FEvalStack::FEvalContext* CurrentContext = FEvalStack::Get().GetCurrentContext())
		{
			if (PyObject* PyGlobalsFile = PyDict_GetItemString(CurrentContext->GlobalDict, "__file__"))
			{
				TypeFilename = PyObjectToUEString(PyGlobalsFile);
			}
		}
	}

	// Normalize the found path for consistency (as the absolute path may vary)
	if (!TypeFilename.IsEmpty())
	{
		FString TypePackageName;

		FPaths::NormalizeFilename(TypeFilename);
		if (!FPackageName::TryConvertFilenameToLongPackageName(TypeFilename, TypePackageName))
		{
			if (FPaths::IsUnderDirectory(TypeFilename, FPaths::EngineDir()))
			{
				FPaths::MakePathRelativeTo(TypeFilename, *FPaths::EngineDir());
				TypePackageName = FPaths::Combine(TEXTVIEW("/Engine"), TypeFilename);
			}
			else if (FPaths::IsUnderDirectory(TypeFilename, FPaths::ProjectDir()))
			{
				FPaths::MakePathRelativeTo(TypeFilename, *FPaths::ProjectDir());
				TypePackageName = FPaths::Combine(TEXTVIEW("/Game"), TypeFilename);
			}
		}
			
		if (TypePackageName.IsEmpty())
		{
			// This filename is something we can't resolve into a stable package path
			// Just hash it into the type name to try and keep things unique
			TypeFilename.ToLowerInline(); // To produce a case-insensitive hash
			OutName += TStringBuilder<12>().Appendf(TEXT("_0x%08X"), FCrc::StrCrc32(*TypeFilename));
		}
		else
		{
			// Remove any remaining extension and add the "_PY" suffix
			TypePackageName = FPaths::ChangeExtension(TypePackageName, TEXT(""));
			TypePackageName += TEXTVIEW("_PY");

			// This filename resolved into a stable package path, so put the generated type in that package
			UPackage* TypePackage = FindObject<UPackage>(nullptr, *TypePackageName);
			if (!TypePackage)
			{
				TypePackage = NewObject<UPackage>(nullptr, *TypePackageName, RF_Public | RF_Transient);
				TypePackage->SetPackageFlags(PKG_ContainsScript);
			}
			OutOuter = TypePackage;
		}
	}
}

FString GetGeneratedTypeDisplayName(PyTypeObject* InPyType)
{
	FString CleanName = GetCleanTypename(InPyType);
	return FName::NameToDisplayString(CleanName, /*bIsBool*/false);
}

FString GetErrorContext(PyTypeObject* InPyType)
{
	return UTF8_TO_TCHAR(InPyType->tp_name);
}

FString GetErrorContext(PyObject* InPyObj)
{
	return GetErrorContext(PyType_Check(InPyObj) ? (PyTypeObject*)InPyObj : Py_TYPE(InPyObj));
}

void SetPythonError(PyObject* InException, PyTypeObject* InErrorContext, const TCHAR* InErrorMsg)
{
	return SetPythonError(InException, *GetErrorContext(InErrorContext), InErrorMsg);
}

void SetPythonError(PyObject* InException, PyObject* InErrorContext, const TCHAR* InErrorMsg)
{
	return SetPythonError(InException, *GetErrorContext(InErrorContext), InErrorMsg);
}

void SetPythonError(PyObject* InException, const TCHAR* InErrorContext, const TCHAR* InErrorMsg)
{
	// Extract any inner exception so we can combine it with the current exception
	FString InnerException;
	{
		FPyObjectPtr PyExceptionType;
		FPyObjectPtr PyExceptionValue;
		FPyObjectPtr PyExceptionTraceback;
		PyErr_Fetch(&PyExceptionType.Get(), &PyExceptionValue.Get(), &PyExceptionTraceback.Get());
		PyErr_NormalizeException(&PyExceptionType.Get(), &PyExceptionValue.Get(), &PyExceptionTraceback.Get());

		if (PyExceptionValue)
		{
			if (PyExceptionType)
			{
				FPyObjectPtr PyExceptionTypeName = FPyObjectPtr::StealReference(PyObject_GetAttrString(PyExceptionType, "__name__"));
				InnerException = FString::Printf(TEXT("%s: %s"), PyExceptionTypeName ? *PyObjectToUEString(PyExceptionTypeName) : *PyObjectToUEString(PyExceptionType), *PyObjectToUEString(PyExceptionValue));
			}
			else
			{
				InnerException = PyObjectToUEString(PyExceptionValue);
			}
		}
	}

	FString FinalException = FString::Printf(TEXT("%s: %s"), InErrorContext, InErrorMsg);
	if (!InnerException.IsEmpty())
	{
		TArray<FString> InnerExceptionLines;
		InnerException.ParseIntoArrayLines(InnerExceptionLines);

		for (const FString& InnerExceptionLine : InnerExceptionLines)
		{
			FinalException += TEXT("\n  ");
			FinalException += InnerExceptionLine;
		}
	}

	PyErr_SetString(InException, TCHAR_TO_UTF8(*FinalException));
}

int SetPythonWarning(PyObject* InException, PyTypeObject* InErrorContext, const TCHAR* InErrorMsg)
{
	return SetPythonWarning(InException, *GetErrorContext(InErrorContext), InErrorMsg);
}

int SetPythonWarning(PyObject* InException, PyObject* InErrorContext, const TCHAR* InErrorMsg)
{
	return SetPythonWarning(InException, *GetErrorContext(InErrorContext), InErrorMsg);
}

int SetPythonWarning(PyObject* InException, const TCHAR* InErrorContext, const TCHAR* InErrorMsg)
{
	const FString FinalException = FString::Printf(TEXT("%s: %s"), InErrorContext, InErrorMsg);
	return PyErr_WarnEx(InException, TCHAR_TO_UTF8(*FinalException), 1);
}

bool EnableDeveloperWarnings()
{
	FPyObjectPtr PyWarningsModule = FPyObjectPtr::StealReference(PyImport_ImportModule("warnings"));
	if (PyWarningsModule)
	{
		PyObject* PyWarningsDict = PyModule_GetDict(PyWarningsModule);

		PyObject* PySimpleFilterFunc = PyDict_GetItemString(PyWarningsDict, "simplefilter");
		if (PySimpleFilterFunc)
		{
			FPyObjectPtr PySimpleFilterResult = FPyObjectPtr::StealReference(PyObject_CallFunction(PySimpleFilterFunc, PyCStrCast("s"), "default"));
			if (PySimpleFilterResult)
			{
				return true;
			}
		}
	}

	return false;
}

bool FetchPythonError(FString& OutError)
{
	OutError.Reset();

	// This doesn't just call PyErr_Print as it also needs to work before stderr redirection has been set-up in Python
	FPyObjectPtr PyExceptionType;
	FPyObjectPtr PyExceptionValue;
	FPyObjectPtr PyExceptionTraceback;
	PyErr_Fetch(&PyExceptionType.Get(), &PyExceptionValue.Get(), &PyExceptionTraceback.Get());
	PyErr_NormalizeException(&PyExceptionType.Get(), &PyExceptionValue.Get(), &PyExceptionTraceback.Get());

	if (!PyExceptionType)
	{
		// No exception is pending, so nothing more to do!
		return false;
	}

	if (PyExceptionType == PyExc_SystemExit && PyExceptionValue)
	{
		auto IsZeroExitCode = [](PyObject* PyCodeObj)
		{
			if (!PyCodeObj || PyCodeObj == Py_None)
			{
				// None implies a zero error code
				return true;
			}

			int32 ExitCode = 0;
			if (PyConversion::Nativize(PyCodeObj, ExitCode, PyConversion::ESetErrorState::No))
			{
				return ExitCode == 0;
			}

			return false;
		};

		PySystemExitObject* PySysExit = (PySystemExitObject*)PyExceptionValue.Get();
		if (IsZeroExitCode(PySysExit->code))
		{
			// Trap and discard SystemExit with an exit code of zero, as it is designed to make the interpreter process exit (which doesn't make sense for an embedded interpreter)
			// If someone wants to actually exit the editor itself, then there is another Unreal API function to let them do that
			PyErr_Clear();
			return false;
		}
	}

	if (PyExceptionTraceback)
	{
		FPyObjectPtr PyTracebackModule = FPyObjectPtr::StealReference(PyImport_ImportModule("traceback"));
		if (PyTracebackModule)
		{
			PyObject* PyTracebackDict = PyModule_GetDict(PyTracebackModule);
			PyObject* PyFormatExceptionFunc = PyDict_GetItemString(PyTracebackDict, "format_exception");
			if (PyFormatExceptionFunc)
			{
				FPyObjectPtr PyFormatExceptionResult = FPyObjectPtr::StealReference(PyObject_CallFunctionObjArgs(PyFormatExceptionFunc, PyExceptionType.Get(), PyExceptionValue.Get(), PyExceptionTraceback.Get(), nullptr));
				if (PyFormatExceptionResult)
				{
					if (PyList_Check(PyFormatExceptionResult))
					{
						const Py_ssize_t FormatExceptionResultSize = PyList_Size(PyFormatExceptionResult);
						for (Py_ssize_t FormatExceptionResultIndex = 0; FormatExceptionResultIndex < FormatExceptionResultSize; ++FormatExceptionResultIndex)
						{
							PyObject* PyFormatExceptionResultItem = PyList_GetItem(PyFormatExceptionResult, FormatExceptionResultIndex);
							if (PyFormatExceptionResultItem)
							{
								if (FormatExceptionResultIndex > 0)
								{
									OutError += '\n';
								}
								OutError += PyObjectToUEString(PyFormatExceptionResultItem);
							}
						}
					}
					else
					{
						OutError += PyObjectToUEString(PyFormatExceptionResult);
					}
				}
			}
		}
	}

	if (OutError.IsEmpty() && PyExceptionValue)
	{
		if (PyExceptionType && PyType_Check(PyExceptionType))
		{
			FPyObjectPtr PyExceptionTypeName = FPyObjectPtr::StealReference(PyObject_GetAttrString(PyExceptionType, "__name__"));
			OutError = FString::Printf(TEXT("%s: %s"), PyExceptionTypeName ? *PyObjectToUEString(PyExceptionTypeName) : *PyObjectToUEString(PyExceptionType), *PyObjectToUEString(PyExceptionValue));

			// Syntax errors require special handling to get the extended error information out of them (their str() only returns the error message and not the context info)
			if (PyObject_IsInstance(PyExceptionValue, PyExc_SyntaxError))
			{
				FPyObjectPtr PySyntaxErrorText = FPyObjectPtr::StealReference(PyObject_GetAttrString(PyExceptionValue, "text"));
				FPyObjectPtr PySyntaxErrorOffset = FPyObjectPtr::StealReference(PyObject_GetAttrString(PyExceptionValue, "offset"));
				if (PySyntaxErrorText && PySyntaxErrorOffset)
				{
					OutError += TEXT("\n  ");
					OutError += PyObjectToUEString(PySyntaxErrorText);

					int32 OffsetInt = 0;
					PyConversion::Nativize(PySyntaxErrorOffset, OffsetInt, PyConversion::ESetErrorState::No);
					if (OffsetInt > 0)
					{
						OutError += TEXT("\n  ");
						for (int32 OffsetIndex = 1; OffsetIndex < OffsetInt; ++OffsetIndex)
						{
							OutError += TEXT(" ");
						}
						OutError += TEXT("^");
					}
				}
			}
		}
		else
		{
			OutError = PyObjectToUEString(PyExceptionValue);
		}
	}

	if (OutError.IsEmpty())
	{
		OutError = PyObjectToUEString(PyExceptionType);
	}

	if (OutError.IsEmpty())
	{
		OutError = TEXT("<unknown exception>");
	}

	PyErr_Clear();

	// Raise the excepthook if it was changed.
	{
		PyObject* PyDefaultExceptHook = PySys_GetObject(PyCStrCast("__excepthook__"));
		PyObject* PyCurrentExceptHook = PySys_GetObject(PyCStrCast("excepthook"));
		if (PyCurrentExceptHook && PyDefaultExceptHook && PyCurrentExceptHook != PyDefaultExceptHook)
		{
			FPyObjectPtr PyExceptHookResult = FPyObjectPtr::StealReference(
				PyObject_CallFunctionObjArgs(
					PyCurrentExceptHook,
					PyExceptionType.Get(),
					PyExceptionValue ? PyExceptionValue.Get() : Py_None,
					PyExceptionTraceback ? PyExceptionTraceback.Get() : Py_None,
					nullptr));
		}
	}

	check(!OutError.IsEmpty());
	return true;
}

bool LogPythonError(FString* OutError, const bool bInteractive)
{
	FString LocalErrorStr;
	FString& ErrorStr = OutError ? *OutError : LocalErrorStr;
	const bool bFetchedError = FetchPythonError(ErrorStr);

	if (bFetchedError)
	{
		// Log the error
		{
			TArray<FString> ErrorLines;
			ErrorStr.ParseIntoArrayLines(ErrorLines);

			for (const FString& ErrorLine : ErrorLines)
			{
				UE_LOGF(LogPython, Error, "%ls", *ErrorLine);
			}
		}

		// Also display the error if this was an interactive request
		if (bInteractive)
		{
			FMessageDialog::Open(EAppMsgType::Ok, FText::AsCultureInvariant(ErrorStr), LOCTEXT("PythonErrorTitle", "Python Error"));
		}
	}

	return bFetchedError;
}

bool ReThrowPythonError(FString* OutError)
{
	FString LocalErrorStr;
	FString& ErrorStr = OutError ? *OutError : LocalErrorStr;
	const bool bFetchedError = FetchPythonError(ErrorStr);

	if (bFetchedError)
	{
		FFrame::KismetExecutionMessage(*ErrorStr, ELogVerbosity::Error);
	}

	return bFetchedError;
}

void CollectGarbage(int32 Generations)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PyUtil::CollectGarbage);
	if (Generations == 2)
	{
		// The C API hardcodes Generations == 2 
		PyGC_Collect();
	}
	else
	{
		// The Python API allows Generations to be overridden
		if (FPyObjectPtr PyGCModule = FPyObjectPtr::StealReference(PyImport_ImportModule("gc")))
		{
			PyObject* PyGCDict = PyModule_GetDict(PyGCModule);

			if (PyObject* PyCollectFunc = PyDict_GetItemString(PyGCDict, "collect"))
			{
				FPyObjectPtr PyCollectResult = FPyObjectPtr::StealReference(PyObject_CallFunction(PyCollectFunc, PyCStrCast("i"), Generations));
				// PyCollectResult is unused, but the FPyObjectPtr handles its ref-count
			}
		}
	}
}

bool CanTickPython()
{
	// Do not tick into Python while Python is already running on the stack
	// Doing so could cause re-entrancy issues and confuse exception handling
	if (UE::Python::Private::GIsRunningUserScript)
	{
		return false;
	}

	// Do not tick into Python when it may not be safe to call back into C++
	if (UE::IsSavingPackage() || IsGarbageCollecting() || FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return false;
	}

	return true;
}

}

#endif	// WITH_PYTHON

#undef LOCTEXT_NAMESPACE
