// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"
#include "CoreMinimal.h"
#include "Containers/ArrayView.h"
#include "Logging/LogMacros.h"
#include "Templates/Requires.h"
#include "UObject/Field.h"
#include "UObject/FieldIterator.h"

#include <type_traits>

DECLARE_LOG_CATEGORY_EXTERN(LogPython, Log, All);

#if WITH_PYTHON

struct FPropertyAccessChangeNotify;

/** Cast a function pointer to PyCFunction (via a void* to avoid a compiler warning) */
#define PyCFunctionCast(FUNCPTR) (PyCFunction)(void*)(FUNCPTR)

/** Cast a string literal to a char* (via a void* to avoid a compiler warning) */
#define PyCStrCast(STR) (char*)(void*)(STR)

namespace UE::Python::Private
{

/**
 * Set to true whenever we're running user-defined Python script.
 * This is primarily set for FPythonScriptPlugin::EvalString, but should 
 * also be set whenever calling back into user-defined Python from other
 * places (eg, PythonGeneratedClass functions, delegates, etc).
 */
extern bool GIsRunningUserScript;

}

namespace PyUtil
{
	/** Types used by various APIs that change between versions */
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 2
	typedef PyObject FPySliceType;
	typedef Py_hash_t FPyHashType;
	typedef PyObject FPyCodeObjectType;
#else	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 2
	typedef long FPyHashType;
	typedef PySliceObject FPySliceType;
	typedef PyCodeObject FPyCodeObjectType;
#endif	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 2

	/** Character type that this version of Python uses in its API */
	typedef wchar_t FPyApiChar;
	typedef TArray<FPyApiChar> FPyApiBuffer;

	/** Convert a TCHAR to a transient buffer that can be passed to a Python API that doesn't hold the result */
	#define TCHARToPyApiChar(InStr) TCHAR_TO_WCHAR(InStr)

	extern const FName DefaultPythonPropertyName;

	/** Convert a TCHAR to a persistent buffer that can be passed to a Python API that does hold the result (you have to keep the buffer around as long as Python needs it) */
	FPyApiBuffer TCHARToPyApiBuffer(const TCHAR* InStr);

	/** Given a Python object, convert it to a string and extract the string value into an FString */
	FString PyObjectToUEString(PyObject* InPyObj);

	/** Given a Python string/unicode object, extract the string value into an FString */
	FString PyStringToUEString(PyObject* InPyStr);

	/** Given a Python object, convert it to its string representation (repr) and extract the string value into an FString */
	FString PyObjectToUEStringRepr(PyObject* InPyObj);

	/** Given two values, perform a rich-comparison and return the result */
	template <typename T, typename U>
	PyObject* PyRichCmp(const T& InLHS, const U& InRHS, const int InOp)
	{
		const bool bResult = 
			(InOp == Py_EQ && InLHS == InRHS) ||
			(InOp == Py_NE && InLHS != InRHS) ||
			(InOp == Py_LT && InLHS < InRHS)  ||
			(InOp == Py_GT && InLHS > InRHS)  ||
			(InOp == Py_LE && InLHS <= InRHS) ||
			(InOp == Py_GE && InLHS > InRHS);
		return PyBool_FromLong(bResult);
	}

	/** Stack of context information given when calling PyEval_EvalCode */
	class FEvalStack
	{
	public:
		struct FEvalContext
		{
			const TCHAR* Context = nullptr;
			PyObject* GlobalDict = nullptr;
			PyObject* LocalDict = nullptr;
		};

		static FEvalStack& Get();

		void PushContext(FEvalContext&& Context);
		void PopContext();
		const FEvalContext* GetCurrentContext() const;

	private:
		TArray<FEvalContext> Stack;
	};

	/** Helper to manage a property pointer that may be potentially owned by a wrapper or stack instance */
	template <typename TPropType>
	class TPropOnScope
	{
		static_assert(TIsDerivedFrom<TPropType, FProperty>::IsDerived, "TPropType must be a FProperty-based type!");

	public:
		static TPropOnScope OwnedReference(TPropType* InProp)
		{
			return TPropOnScope(InProp, true);
		}

		static TPropOnScope ExternalReference(TPropType* InProp)
		{
			return TPropOnScope(InProp, false);
		}

		TPropOnScope() = default;

		~TPropOnScope()
		{
			Reset();
		}

		TPropOnScope(const TPropOnScope&) = delete;
		TPropOnScope& operator=(const TPropOnScope&) = delete;

		template <
			typename TOtherPropType
			UE_REQUIRES(std::is_convertible_v<TOtherPropType*, TPropType*>)
		>
		TPropOnScope(TPropOnScope<TOtherPropType>&& Other)
		{
			bOwnsProp = Other.OwnsProp();
			Prop = Other.Release();

			Other.Reset();
		}

		template <
			typename TOtherPropType
			UE_REQUIRES(std::is_convertible_v<TOtherPropType*, TPropType*>)
		>
		TPropOnScope& operator=(TPropOnScope<TOtherPropType>&& Other)
		{
			if (this != (void*)&Other)
			{
				Reset();

				bOwnsProp = Other.OwnsProp();
				Prop = Other.Release();

				Other.Reset();
			}
			return *this;
		}

		explicit operator bool() const
		{
			return IsValid();
		}

		bool IsValid() const
		{
			return Prop != nullptr;
		}

		operator TPropType*() const
		{
			return Prop;
		}

		TPropType& operator*() const
		{
			check(Prop);
			return *Prop;
		}

		TPropType* operator->() const
		{
			check(Prop);
			return Prop;
		}

		TPropType* Get() const
		{
			return Prop;
		}

		bool OwnsProp() const
		{
			return bOwnsProp;
		}

		TPropType* Release()
		{
			TPropType* LocalProp = Prop;
			Prop = nullptr;
			bOwnsProp = false;
			return LocalProp;
		}

		void Reset()
		{
			if (bOwnsProp)
			{
				delete Prop;
			}
			Prop = nullptr;
			bOwnsProp = false;
		}

		void AddReferencedObjects(FReferenceCollector& Collector)
		{
			if (Prop)
			{
				((std::remove_const_t<TPropType>*)Prop)->AddReferencedObjects(Collector);
			}
		}

	private:
		TPropOnScope(TPropType* InProp, const bool InOwnsProp)
			: Prop(InProp)
			, bOwnsProp(InOwnsProp)
		{
		}

		TPropType* Prop = nullptr;
		bool bOwnsProp = false;
	};

	typedef TPropOnScope<FProperty> FPropOnScope;
	typedef TPropOnScope<const FProperty> FConstPropOnScope;
	typedef TPropOnScope<FArrayProperty> FArrayPropOnScope;
	typedef TPropOnScope<const FArrayProperty> FConstArrayPropOnScope;
	typedef TPropOnScope<FSetProperty> FSetPropOnScope;
	typedef TPropOnScope<const FSetProperty> FConstSetPropOnScope;
	typedef TPropOnScope<FMapProperty> FMapPropOnScope;
	typedef TPropOnScope<const FMapProperty> FConstMapPropOnScope;

	/** Helper used to hold the value for a property value on the stack */
	class FPropValueOnScope
	{
	public:
		explicit FPropValueOnScope(FConstPropOnScope&& InProp);
		~FPropValueOnScope();

		bool SetValue(PyObject* InPyObj, const TCHAR* InErrorCtxt);

		bool IsValid() const;

		const FProperty* GetProp() const;

		void* GetValue(const int32 InArrayIndex = 0) const;

	private:
		FConstPropOnScope Prop;
		void* Value = nullptr;
	};

	/** Helper used to hold the value for a single fixed array element on the stack */
	class FFixedArrayElementOnScope : public FPropValueOnScope
	{
	public:
		explicit FFixedArrayElementOnScope(const FProperty* InProp);
	};

	/** Helper used to hold the value for a single array element on the stack */
	class FArrayElementOnScope : public FPropValueOnScope
	{
	public:
		explicit FArrayElementOnScope(const FArrayProperty* InProp);
	};

	/** Helper used to hold the value for a single set element on the stack */
	class FSetElementOnScope : public FPropValueOnScope
	{
	public:
		explicit FSetElementOnScope(const FSetProperty* InProp);
	};

	/** Helper used to hold the value for a single map key on the stack */
	class FMapKeyOnScope : public FPropValueOnScope
	{
	public:
		explicit FMapKeyOnScope(const FMapProperty* InProp);
	};

	/** Helper used to hold the value for a single map value on the stack */
	class FMapValueOnScope : public FPropValueOnScope
	{
	public:
		explicit FMapValueOnScope(const FMapProperty* InProp);
	};

	/** Helper used to hold the stack data for a UFunction call */
	struct FFunctionStackOnScope
	{
	public:
		FFunctionStackOnScope(const UFunction* InFunc, void* InStackMemory)
			: Func(InFunc)
			, StackMemory(InStackMemory)
		{
			for (TFieldIterator<FProperty> PropIt(Func); PropIt; ++PropIt)
			{
				PropIt->InitializeValue_InContainer(StackMemory);
			}
		}

		~FFunctionStackOnScope()
		{
			for (TFieldIterator<FProperty> PropIt(Func); PropIt; ++PropIt)
			{
				PropIt->DestroyValue_InContainer(StackMemory);
			}
		}

		const UFunction* GetFunction() const
		{
			return Func;
		}

		void* GetMemory() const
		{
			return StackMemory;
		}

	private:
		const UFunction* Func = nullptr;
		void* StackMemory = nullptr;
	};
	#define PY_UFUNCTION_STACK(NAME, FUNCTION) \
		PyUtil::FFunctionStackOnScope NAME(FUNCTION, FMemory_Alloca_Aligned(FMath::Max(1, (FUNCTION)->GetStructureSize()), (FUNCTION)->GetMinAlignment()))

	/** Struct containing information needed to construct a property instance */
	struct FPropertyDef
	{
		FPropertyDef()
			: PropertyClass(nullptr)
			, PropertySubType(nullptr)
			, KeyDef()
			, ValueDef()
		{
		}

		explicit FPropertyDef(FFieldClass* InPropertyClass, UObject* InPropertySubType = nullptr, TSharedPtr<FPropertyDef> InKeyDef = nullptr, TSharedPtr<FPropertyDef> InValueDef = nullptr)
			: PropertyClass(InPropertyClass)
			, PropertySubType(InPropertySubType)
			, KeyDef(InKeyDef)
			, ValueDef(InValueDef)
		{
		}

		FPropertyDef(const FProperty* InProperty);

		FORCEINLINE bool operator==(const FPropertyDef& Other) const
		{
			return PropertyClass == Other.PropertyClass
				&& PropertySubType == Other.PropertySubType
				&& (KeyDef.IsValid() == Other.KeyDef.IsValid() || *KeyDef == *Other.KeyDef)
				&& (ValueDef.IsValid() == Other.ValueDef.IsValid() || *ValueDef == *Other.ValueDef);
		}

		FORCEINLINE bool operator!=(const FPropertyDef& Other) const
		{
			return !(*this == Other);
		}

		/** Class of the property to create */
		FFieldClass* PropertyClass;

		/** Sub-type of the property (the class for object properties, the struct for struct properties, the enum for enum properties, the function for delegate properties) */
		UObject* PropertySubType;

		/** Key definition of this property (for map properties) */
		TSharedPtr<FPropertyDef> KeyDef;

		/** Value definition of this property (for array, set, map, and optional properties) */
		TSharedPtr<FPropertyDef> ValueDef;
	};

	/** Given a Python type, work out what kind of property we would need to create to hold this data */
	bool CalculatePropertyDef(PyTypeObject* InPyType, FPropertyDef& OutPropertyDef);

	/** Given a Python instance, work out what kind of property we would need to create to hold this data */
	bool CalculatePropertyDef(PyObject* InPyObj, FPropertyDef& OutPropertyDef);

	/** Given a Python instance, work out whether this will produce an Optional property def, and if so (and requested), get the value that the Optional type will hold */
	bool IsOptionalPropertyDef(PyObject* InPyObj, PyObject** OutPyValueObj = nullptr);

	/** Given a property definition, create a property instance */
	FProperty* CreateProperty(const FPropertyDef& InPropertyDef, const int32 InArrayDim = 1, FFieldVariant InOuter = nullptr, const FName InName = DefaultPythonPropertyName);

	/** Given a Python type, create a compatible property instance */
	FProperty* CreateProperty(PyTypeObject* InPyType, const int32 InArrayDim = 1, FFieldVariant InOuter = nullptr, const FName InName = DefaultPythonPropertyName);

	/** Given a Python instance, create a compatible property instance */
	FProperty* CreateProperty(PyObject* InPyObj, const int32 InArrayDim = 1, FFieldVariant InOuter = nullptr, const FName InName = DefaultPythonPropertyName);

	/** Check to see if the given property is an input parameter for a function */
	bool IsInputParameter(const FProperty* InParam);

	/** Check to see if the given property is an output parameter for a function */
	bool IsOutputParameter(const FProperty* InParam);

	/** Import a UHT default value on the given property */
	void ImportDefaultValue(const FProperty* InProp, void* InPropValue, const FString& InDefaultValue);

	/** Invoke a function call. Returns false if a Python exception was raised */
	bool InvokeFunctionCall(UObject* InObj, const UFunction* InFunc, void* InBaseParamsAddr, const TCHAR* InErrorCtxt);

	/** Given a Python function, get the names of the arguments along with their default values */
	bool InspectFunctionArgs(PyObject* InFunc, TArray<FString>& OutArgNames, TArray<FPyObjectPtr>* OutArgDefaults = nullptr);

	/** Validate that the given Python object is valid for a 'type' parameter used by containers */
	int ValidateContainerTypeParam(PyObject* InPyObj, FPropertyDef& OutPropDef, const char* InPythonArgName, const TCHAR* InErrorCtxt);

	/** Validate that the given Python object is valid for the 'len' parameter used by containers */
	int ValidateContainerLenParam(PyObject* InPyObj, int32 &OutLen, const char* InPythonArgName, const TCHAR* InErrorCtxt);

	/** Validate that the given index is valid for the container length */
	int ValidateContainerIndexParam(const Py_ssize_t InIndex, const Py_ssize_t InLen, const FProperty* InProp, const TCHAR* InErrorCtxt);

	/** Resolve a container index (taking into account negative indices) */
	Py_ssize_t ResolveContainerIndexParam(const Py_ssize_t InIndex, const Py_ssize_t InLen);

	/**
	 * A NewObject wrapper for Python which catches some internal check conditions and raises them as Python exceptions instead.
	 *
	 * @param InObjClass The class type to create an instance of.
	 * @param InOuter The outer object that should host the new instance.
	 * @param InName The name that should be given to the new instance (will generate a unique name if None).
	 * @param InBaseClass The base class type required for InObjClass (ignored if null).
	 */
	UObject* NewObject(UClass* InObjClass, UObject* InOuter, const FName InName, UClass* InBaseClass, const TCHAR* InErrorCtxt);

	/**
	 * Given a Python object, try and get the owner Unreal object for the instance.
	 * For wrapped objects this is the wrapped instance, for wrapped structs it will attempt to walk through the owner chain to find a wrapped object.
	 * @return The owner Unreal object, or null.
	 */
	UObject* GetOwnerObject(PyObject* InPyObj);

	/** Get the current value of the given property from the given struct */
	PyObject* GetPropertyValue(const UStruct* InStruct, const void* InStructData, const FProperty* InProp, const char *InAttributeName, PyObject* InOwnerPyObject, const TCHAR* InErrorCtxt);

	/** Set the current value of the given property from the given struct */
	int SetPropertyValue(const UStruct* InStruct, void* InStructData, PyObject* InValue, const FProperty* InProp, const char *InAttributeName, const FPropertyAccessChangeNotify* InChangeNotify, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const TCHAR* InErrorCtxt, const TConstArrayView<void*>& InArchetypeInstStructData);

	/**
	 * Check to see if the given object implements a length function.
	 * @return true if it does, false otherwise.
	 */
	bool HasLength(PyObject* InObj);

	/**
	 * Check to see if the given type implements a length function.
	 * @return true if it does, false otherwise.
	 */
	bool HasLength(PyTypeObject* InType);

	/**
	 * Check to see if the given object looks like a mapping type (implements a "keys" function).
	 * @return true if it does, false otherwise.
	 */
	bool IsMappingType(PyObject* InObj);

	/**
	 * Check to see if the given type looks like a mapping type (implements a "keys" function).
	 * @return true if it does, false otherwise.
	 */
	bool IsMappingType(PyTypeObject* InType);

	/**
	 * Cache of on-disk Python modules to optimize repeated IsModuleAvailableForImport queries.
	 * @note This cache does not automatically watch or update itself from file-system changes after the initial scan!
	 */
	class FOnDiskModules
	{
	public:
		FOnDiskModules() = default;

		explicit FOnDiskModules(FString InModuleNameWildcard)
			: ModuleNameWildcard(MoveTemp(InModuleNameWildcard))
		{
		}

		/**
		 * Add any module files found under the given path.
		 * @note Function is non-recursive, and will find both "{Module}.py" and "{Module}/__init__.py" module layouts.
		 */
		void AddModules(const TCHAR* InPath);

		/**
		 * Remove any module files found under the given path.
		 */
		void RemoveModules(const TCHAR* InPath);

		/**
		 * Check to see whether the given module is known to the cache, optionally returning the actual module file location if it is.
		 * @note Will look for both "{Module}.py" and "{Module}/__init__.py" module layouts.
		 */
		bool HasModule(const TCHAR* InModuleName, FString* OutResolvedFile = nullptr) const;

	private:
		FString ModuleNameWildcard;

		TSet<FString> CachedModules;
	};

	/**
	 * Get the cache of on-disk Unreal Python modules ("unreal_*" wildcard).
	 */
	FOnDiskModules& GetOnDiskUnrealModulesCache();

	/**
	 * Test to see whether the given module is available for import (is in the sys.modules table (which includes built-in modules), or is in any known sys.path path).
	 * @note This function can't handle dot separated names.
	 */
	bool IsModuleAvailableForImport(const TCHAR* InModuleName, const FOnDiskModules* InOnDiskModules = nullptr, FString* OutResolvedFile = nullptr);

	/**
	 * Test to see whether the given module is available for import (is in the sys.modules table).
	 * @note This function can't handle dot separated names.
	 */
	bool IsModuleImported(const TCHAR* InModuleName, PyObject** OutPyModule = nullptr);

	/**
	 * Get the path to the Python interpreter executable of the Python SDK this plugin was compiled against.
	 */
	FString GetInterpreterExecutablePath(bool* OutIsEnginePython = nullptr);

	/**
	 * Get the arch-specific path component for potential lib site-packages differences (currently X64/Arm) 
	 */
	FString GetArchString();

	/**
	 * Get static list of site-package subdirs to check
	 */
	const TArray<FString>& GetSitePackageSubdirs();

	/**
	 * Register the given path as a known site-packages path (@see site.addsitedir in Python).
	 */
	void AddSitePackagesPath(const FString& InPath);

	/**
	 * Ensure that the given path is on the sys.path list.
	 */
	void AddSystemPath(const FString& InPath);

	/**
	 * Remove the given path from the sys.path list.
	 */
	void RemoveSystemPath(const FString& InPath);

	/**
	 * Get the current sys.path list.
	 */
	TArray<FString> GetSystemPaths();

	/**
	 * Get the doc string of the given object (if any).
	 */
	FString GetDocString(PyObject* InPyObj);

	/**
	 * Get the friendly value of the given struct that can be used when stringifying struct values for Python.
	 */
	FString GetFriendlyStructValue(const UScriptStruct* InStruct, const void* InStructValue, const uint32 InPortFlags);

	/**
	 * Get the friendly value of the given property that can be used when stringifying property values for Python.
	 */
	FString GetFriendlyPropertyValue(const FProperty* InProp, const void* InPropValue, const uint32 InPropPortFlags);

	/**
	 * Get the friendly typename of the given object that can be used in error reporting.
	 */
	FString GetFriendlyTypename(PyTypeObject* InPyType);

	/**
	 * Get the friendly typename of the given object that can be used in error reporting.
	 * @note Passing a PyTypeObject returns the name of that object, rather than 'type'.
	 */
	FString GetFriendlyTypename(PyObject* InPyObj);

	template <typename T>
	FString GetFriendlyTypename(T* InPyObj)
	{
		return GetFriendlyTypename((PyObject*)InPyObj);
	}

	/**
	 * Get the clean type name for the given Python type (strip any module information).
	 */
	FString GetCleanTypename(PyTypeObject* InPyType);

	/**
	 * Get the clean type name for the given Python object (strip any module information).
	 * @note Passing a PyTypeObject returns the name of that object, rather than 'type'.
	 */
	FString GetCleanTypename(PyObject* InPyObj);

	template <typename T>
	FString GetCleanTypename(T* InPyObj)
	{
		return GetCleanTypename((PyObject*)InPyObj);
	}

	/**
	 * Get the generated type outer and name to use for the given Python type.
	 * @note This should only be called when immediately processing the generation request 
	 *       from Python, as it relies on getting information from the Python exec stack.
	 */
	void GetGeneratedTypeOuterAndName(PyTypeObject* InPyType, UObject*& OutOuter, FString& OutName);

	/**
	 * Get the generated display name to use for the given Python type.
	 */
	FString GetGeneratedTypeDisplayName(PyTypeObject* InPyType);

	/**
	 * Get the error context string of the given object.
	 */
	FString GetErrorContext(PyTypeObject* InPyType);
	FString GetErrorContext(PyObject* InPyObj);

	template <typename T>
	FString GetErrorContext(T* InPyObj)
	{
		return GetErrorContext((PyObject*)InPyObj);
	}

	/** Set the pending Python error, nesting any other pending error within this one */
	void SetPythonError(PyObject* InException, PyTypeObject* InErrorContext, const TCHAR* InErrorMsg);
	void SetPythonError(PyObject* InException, PyObject* InErrorContext, const TCHAR* InErrorMsg);
	void SetPythonError(PyObject* InException, const TCHAR* InErrorContext, const TCHAR* InErrorMsg);

	template <typename T>
	void SetPythonError(PyObject* InException, T* InErrorContext, const TCHAR* InErrorMsg)
	{
		return SetPythonError(InException, (PyObject*)InErrorContext, InErrorMsg);
	}

	/** Set a Python warning (see PyErr_WarnEx for the return value meaning) */
	int SetPythonWarning(PyObject* InException, PyTypeObject* InErrorContext, const TCHAR* InErrorMsg);
	int SetPythonWarning(PyObject* InException, PyObject* InErrorContext, const TCHAR* InErrorMsg);
	int SetPythonWarning(PyObject* InException, const TCHAR* InErrorContext, const TCHAR* InErrorMsg);

	template <typename T>
	int SetPythonWarning(PyObject* InException, T* InErrorContext, const TCHAR* InErrorMsg)
	{
		return SetPythonWarning(InException, (PyObject*)InErrorContext, InErrorMsg);
	}

	/** Enable developer warnings (eg, deprecation warnings) */
	bool EnableDeveloperWarnings();

	/**
	 * Log and clear any pending Python error, optionally retrieving the error text that was logged.
	 * @return True if Python was in an error state, false otherwise.
	 */
	bool LogPythonError(FString* OutError = nullptr, const bool bInteractive = false);

	/**
	 * Re-throw and clear any pending Python error via FFrame::KismetExecutionMessage, optionally retrieving the error text that was thrown.
	 * @return True if Python was in an error state, false otherwise.
	 */
	bool ReThrowPythonError(FString* OutError = nullptr);

	/**
	 * Run the Python garbage collector to clean-up objects that have internal reference cycles that can be broken.
	 * @see https://github.com/python/cpython/blob/main/InternalDocs/garbage_collector.md#optimization-generations
	 */
	void CollectGarbage(int32 Generations = 2);

	/** Return true if it's currently considered safe to invoke Python from tick handlers */
	bool CanTickPython();
}

#endif	// WITH_PYTHON
