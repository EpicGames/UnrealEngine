// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperArray.h"
#include "PyWrapperTypeRegistry.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"

#if WITH_PYTHON

/** Python type for FPyWrapperArrayIterator */
extern PyTypeObject PyWrapperArrayIteratorType;

/** Iterator used with arrays */
struct FPyWrapperArrayIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Instance being iterated over */
	FPyWrapperArray* IterInstance;

	/** Current iteration index */
	int32 IterIndex;

	/** New this iterator instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperArrayIterator* New(PyTypeObject* InType)
	{
		FPyWrapperArrayIterator* Self = (FPyWrapperArrayIterator*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->IterInstance = nullptr;
			Self->IterIndex = 0;
		}
		return Self;
	}

	/** Free this iterator instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperArrayIterator* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this iterator instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperArrayIterator* InSelf, FPyWrapperArray* InInstance)
	{
		Deinit(InSelf);

		Py_INCREF(InInstance);
		InSelf->IterInstance = InInstance;
		InSelf->IterIndex = 0;

		return 0;
	}

	/** Deinitialize this iterator instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperArrayIterator* InSelf)
	{
		Py_XDECREF(InSelf->IterInstance);
		InSelf->IterInstance = nullptr;
		InSelf->IterIndex = 0;
	}

	/** Called to validate the internal state of this iterator instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperArrayIterator* InSelf)
	{
		if (!InSelf->IterInstance)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - IterInstance is null!"));
			return false;
		}

		return true;
	}

	/** Get the iterator */
	static FPyWrapperArrayIterator* GetIter(FPyWrapperArrayIterator* InSelf)
	{
		Py_INCREF(InSelf);
		return InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(FPyWrapperArrayIterator* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		if (InSelf->IterIndex < FPyWrapperArray::Len(InSelf->IterInstance))
		{
			return FPyWrapperArray::GetItem(InSelf->IterInstance, InSelf->IterIndex++);
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}
};

void InitializePyWrapperArray(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperArrayType) == 0)
	{
		static FPyWrapperArrayMetaData MetaData;
		FPyWrapperArrayMetaData::SetMetaData(&PyWrapperArrayType, &MetaData);
		ModuleInfo.AddType(&PyWrapperArrayType);
	}

	PyType_Ready(&PyWrapperArrayIteratorType);
}

FPyWrapperArray* FPyWrapperArray::New(PyTypeObject* InType)
{
	FPyWrapperArray* Self = (FPyWrapperArray*)FPyWrapperBase::New(InType);
	if (Self)
	{
		new(&Self->OwnerContext) FPyWrapperOwnerContext();
		new(&Self->ArrayProp) PyUtil::FConstArrayPropOnScope();
		Self->ArrayInstance = nullptr;
	}
	return Self;
}

void FPyWrapperArray::Free(FPyWrapperArray* InSelf)
{
	Deinit(InSelf);

	InSelf->OwnerContext.~FPyWrapperOwnerContext();
	InSelf->ArrayProp.~TPropOnScope();
	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperArray::Init(FPyWrapperArray* InSelf, const PyUtil::FPropertyDef& InElementDef)
{
	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	PyUtil::FPropOnScope ArrayElementProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InElementDef, 1));
	if (!ArrayElementProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Array element property was null during init"));
		return -1;
	}

	PyUtil::FArrayPropOnScope ArrayProp = PyUtil::FArrayPropOnScope::OwnedReference(new FArrayProperty(FFieldVariant(), PyUtil::DefaultPythonPropertyName));
	ArrayProp->Inner = ArrayElementProp.Release();

	// Need to manually call Link to fix-up some data (such as the C++ property flags) that are only set during Link
	{
		FArchive Ar;
		ArrayProp->LinkWithoutChangingOffset(Ar);
	}

	void* ArrayValue = FMemory::Malloc(ArrayProp->GetSize(), ArrayProp->GetMinAlignment());
	ArrayProp->InitializeValue(ArrayValue);

	InSelf->ArrayProp = MoveTemp(ArrayProp);
	InSelf->ArrayInstance = ArrayValue;

	FPyWrapperArrayFactory::Get().MapInstance(InSelf->ArrayInstance, InSelf);
	return 0;
}

int FPyWrapperArray::Init(FPyWrapperArray* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FArrayProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod)
{
	InOwnerContext.AssertValidConversionMethod(InConversionMethod);

	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	check(InProp && InValue);

	PyUtil::FConstArrayPropOnScope PropToUse;
	void* ArrayInstanceToUse = nullptr;
	switch (InConversionMethod)
	{
	case EPyConversionMethod::Copy:
	case EPyConversionMethod::Steal:
		{
			PyUtil::FPropOnScope ArrayElementProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InProp->Inner));
			if (!ArrayElementProp)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to create element property from '%s' (%s)"), *InProp->Inner->GetName(), *InProp->Inner->GetClass()->GetName()));
				return -1;
			}

			PyUtil::FArrayPropOnScope ArrayProp = PyUtil::FArrayPropOnScope::OwnedReference(new FArrayProperty(FFieldVariant(), PyUtil::DefaultPythonPropertyName));
			ArrayProp->Inner = ArrayElementProp.Release();

			// Need to manually call Link to fix-up some data (such as the C++ property flags) that are only set during Link
			{
				FArchive Ar;
				ArrayProp->LinkWithoutChangingOffset(Ar);
			}

			PropToUse = MoveTemp(ArrayProp);

			ArrayInstanceToUse = FMemory::Malloc(PropToUse->GetSize(), PropToUse->GetMinAlignment());
			PropToUse->InitializeValue(ArrayInstanceToUse);
			if (InConversionMethod == EPyConversionMethod::Steal)
			{
				FScriptArrayHelper SelfScriptArrayHelper(PropToUse, ArrayInstanceToUse);
				SelfScriptArrayHelper.MoveAssign(InValue);
			}
			else
			{
				PropToUse->CopyCompleteValue(ArrayInstanceToUse, InValue);
			}
		}
		break;

	case EPyConversionMethod::Reference:
		{
			PropToUse = PyUtil::FConstArrayPropOnScope::ExternalReference(InProp);
			ArrayInstanceToUse = InValue;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EPyConversionMethod"));
		break;
	}

	check(PropToUse && ArrayInstanceToUse);

	InSelf->OwnerContext = InOwnerContext;
	InSelf->ArrayProp = MoveTemp(PropToUse);
	InSelf->ArrayInstance = ArrayInstanceToUse;

	FPyWrapperArrayFactory::Get().MapInstance(InSelf->ArrayInstance, InSelf);
	return 0;
}

void FPyWrapperArray::Deinit(FPyWrapperArray* InSelf)
{
	if (InSelf->ArrayInstance)
	{
		FPyWrapperArrayFactory::Get().UnmapInstance(InSelf->ArrayInstance, Py_TYPE(InSelf));
	}

	if (InSelf->OwnerContext.HasOwner())
	{
		InSelf->OwnerContext.Reset();
	}
	else if (InSelf->ArrayInstance)
	{
		if (InSelf->ArrayProp)
		{
			InSelf->ArrayProp->DestroyValue(InSelf->ArrayInstance);
		}
		FMemory::Free(InSelf->ArrayInstance);
	}
	InSelf->ArrayProp.Reset();
	InSelf->ArrayInstance = nullptr;
}

bool FPyWrapperArray::ValidateInternalState(FPyWrapperArray* InSelf)
{
	if (!InSelf->ArrayProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ArrayProp is null!"));
		return false;
	}

	if (!InSelf->ArrayInstance)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ArrayInstance is null!"));
		return false;
	}

	return true;
}

FPyWrapperArray* FPyWrapperArray::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperArrayType) == 1)
	{
		Py_INCREF(InPyObject);
		return (FPyWrapperArray*)InPyObject;
	}

	return nullptr;
}

FPyWrapperArray* FPyWrapperArray::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InElementDef, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperArrayType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperArrayType) == 1))
	{
		FPyWrapperArray* Self = (FPyWrapperArray*)InPyObject;

		if (!ValidateInternalState(Self))
		{
			return nullptr;
		}

		const PyUtil::FPropertyDef SelfElementPropDef = Self->ArrayProp->Inner;
		if (SelfElementPropDef == InElementDef)
		{
			SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

			Py_INCREF(Self);
			return Self;
		}

		FPyWrapperArrayPtr NewArray = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::New(InType));
		if (FPyWrapperArray::Init(NewArray, InElementDef) != 0)
		{
			return nullptr;
		}

		// Attempt to convert the entries in the array to the native format of the new array
		{
			FScriptArrayHelper SelfScriptArrayHelper(Self->ArrayProp, Self->ArrayInstance);
			FScriptArrayHelper NewScriptArrayHelper(NewArray->ArrayProp, NewArray->ArrayInstance);

			const int32 ElementCount = SelfScriptArrayHelper.Num();
			NewScriptArrayHelper.Resize(ElementCount);

			FString ExportedEntry;
			for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
			{
				ExportedEntry.Reset();
				if (!Self->ArrayProp->Inner->ExportText_Direct(ExportedEntry, SelfScriptArrayHelper.GetRawPtr(ElementIndex), SelfScriptArrayHelper.GetRawPtr(ElementIndex), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to export text for element property '%s' (%s) at index %d"), *Self->ArrayProp->Inner->GetName(), *Self->ArrayProp->Inner->GetClass()->GetName(), ElementIndex));
					return nullptr;
				}

				if (!NewArray->ArrayProp->Inner->ImportText_Direct(*ExportedEntry, NewScriptArrayHelper.GetRawPtr(ElementIndex), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to import text '%s' element for property '%s' (%s) at index %d"), *ExportedEntry, *NewArray->ArrayProp->Inner->GetName(), *NewArray->ArrayProp->Inner->GetClass()->GetName(), ElementIndex));
					return nullptr;
				}
			}
		}

		SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
		return NewArray.Release();
	}

	// Attempt conversion from any iterable sequence type that has a defined length
	if (!PyUtil::IsMappingType(InPyObject))
	{
		const Py_ssize_t SequenceLen = PyObject_Length(InPyObject);
		if (SequenceLen != -1)
		{
			FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(InPyObject));
			if (PyObjIter)
			{
				FPyWrapperArrayPtr NewArray = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::New(InType));
				if (FPyWrapperArray::Init(NewArray, InElementDef) != 0)
				{
					return nullptr;
				}

				FScriptArrayHelper NewScriptArrayHelper(NewArray->ArrayProp, NewArray->ArrayInstance);
				NewScriptArrayHelper.Resize(SequenceLen);

				for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
				{
					FPyObjectPtr SequenceItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
					if (!SequenceItem)
					{
						return nullptr;
					}

					if (!PyConversion::NativizeProperty(SequenceItem, NewArray->ArrayProp->Inner, NewScriptArrayHelper.GetRawPtr((int32)SequenceIndex)))
					{
						return nullptr;
					}
				}

				SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
				return NewArray.Release();
			}
		}
	}

	return nullptr;
}

Py_ssize_t FPyWrapperArray::Len(FPyWrapperArray* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	return SelfScriptArrayHelper.Num();
}

PyObject* FPyWrapperArray::GetItem(FPyWrapperArray* InSelf, Py_ssize_t InIndex)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();
	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, ElementCount);

	if (PyUtil::ValidateContainerIndexParam(ResolvedIndex, ElementCount, InSelf->ArrayProp, *PyUtil::GetErrorContext(InSelf)) != 0)
	{
		return nullptr;
	}

	PyObject* PyItemObj = nullptr;
	if (!PyConversion::PythonizeProperty(InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(ResolvedIndex), PyItemObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert element property '%s' (%s) at index %zd"), *InSelf->ArrayProp->Inner->GetName(), *InSelf->ArrayProp->Inner->GetClass()->GetName(), ResolvedIndex));
		return nullptr;
	}
	return PyItemObj;
}

int FPyWrapperArray::SetItem(FPyWrapperArray* InSelf, Py_ssize_t InIndex, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	if (InValue == nullptr)
	{
		return RemoveAt(InSelf, InIndex);
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();
	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, ElementCount);

	const int ValidateIndexResult = PyUtil::ValidateContainerIndexParam(ResolvedIndex, ElementCount, InSelf->ArrayProp, *PyUtil::GetErrorContext(InSelf));
	if (ValidateIndexResult != 0)
	{
		return ValidateIndexResult;
	}

	if (!PyConversion::NativizeProperty(InValue, InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(ResolvedIndex)))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert element property '%s' (%s) at index %zd"), *InSelf->ArrayProp->Inner->GetName(), *InSelf->ArrayProp->Inner->GetClass()->GetName(), ResolvedIndex));
		return -1;
	}

	return 0;
}

int FPyWrapperArray::RemoveAt(FPyWrapperArray* InSelf, Py_ssize_t InIndex)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();
	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, ElementCount);

	const int ValidateIndexResult = PyUtil::ValidateContainerIndexParam(ResolvedIndex, ElementCount, InSelf->ArrayProp, *PyUtil::GetErrorContext(InSelf));
	if (ValidateIndexResult != 0)
	{
		return ValidateIndexResult;
	}

	SelfScriptArrayHelper.RemoveValues(ResolvedIndex);

	return 0;
}

int FPyWrapperArray::Contains(FPyWrapperArray* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FArrayElementOnScope ContainerEntryValue(InSelf->ArrayProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();

	for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
	{
		if (ContainerEntryValue.GetProp()->Identical(SelfScriptArrayHelper.GetRawPtr(ElementIndex), ContainerEntryValue.GetValue()))
		{
			return 1;
		}
	}

	return 0;
}

FPyWrapperArray* FPyWrapperArray::Concat(FPyWrapperArray* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperArrayPtr NewArray = FPyWrapperArrayPtr::StealReference(FPyWrapperArrayFactory::Get().CreateInstance(InSelf->ArrayInstance, InSelf->ArrayProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy));
	if (ConcatInplace(NewArray, InOther) != 0)
	{
		return nullptr;
	}

	return NewArray.Release();
}

int FPyWrapperArray::ConcatInplace(FPyWrapperArray* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->ArrayProp->Inner;
	FPyWrapperArrayPtr Other = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::CastPyObject(InOther, &PyWrapperArrayType, SelfElementDef));
	if (!Other)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot concatenate types '%s' and '%s' together"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(InOther)));
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	FScriptArrayHelper OtherScriptArrayHelper(Other->ArrayProp, Other->ArrayInstance);

	const int32 SelfElementCount = SelfScriptArrayHelper.Num();
	const int32 OtherElementCount = OtherScriptArrayHelper.Num();

	SelfScriptArrayHelper.AddValues(OtherElementCount);
	for (int32 ElementIndex = 0; ElementIndex < OtherElementCount; ++ElementIndex)
	{
		InSelf->ArrayProp->Inner->CopyCompleteValue(SelfScriptArrayHelper.GetRawPtr(SelfElementCount + ElementIndex), OtherScriptArrayHelper.GetRawPtr(ElementIndex));
	}

	return 0;
}

FPyWrapperArray* FPyWrapperArray::Repeat(FPyWrapperArray* InSelf, Py_ssize_t InMultiplier)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperArrayPtr NewArray = FPyWrapperArrayPtr::StealReference(FPyWrapperArrayFactory::Get().CreateInstance(InSelf->ArrayInstance, InSelf->ArrayProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy));
	if (RepeatInplace(NewArray, InMultiplier) != 0)
	{
		return nullptr;
	}

	return NewArray.Release();
}

int FPyWrapperArray::RepeatInplace(FPyWrapperArray* InSelf, Py_ssize_t InMultiplier)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	InMultiplier = FMath::Max(InMultiplier, (Py_ssize_t)0);

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 SelfElementCount = SelfScriptArrayHelper.Num();

	SelfScriptArrayHelper.Resize(SelfElementCount * InMultiplier);

	int32 NewElementIndex = SelfElementCount;
	for (int32 MultipleIndex = 1; MultipleIndex < (int32)InMultiplier; ++MultipleIndex)
	{
		for (int32 ElementIndex = 0; ElementIndex < SelfElementCount; ++ElementIndex, ++NewElementIndex)
		{
			InSelf->ArrayProp->Inner->CopyCompleteValue(SelfScriptArrayHelper.GetRawPtr(NewElementIndex), SelfScriptArrayHelper.GetRawPtr(ElementIndex));
		}
	}

	return 0;
}

int FPyWrapperArray::Append(FPyWrapperArray* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 NewValueIndex = SelfScriptArrayHelper.AddValue();

	if (!PyConversion::NativizeProperty(InValue, InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(NewValueIndex)))
	{
		SelfScriptArrayHelper.RemoveValues(NewValueIndex);
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert '%s' to an element of '%s' for insertion"), *PyUtil::GetFriendlyTypename(InValue), *PyUtil::GetFriendlyTypename(InSelf)));
		return -1;
	}

	return 0;
}

Py_ssize_t FPyWrapperArray::Count(FPyWrapperArray* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FArrayElementOnScope ContainerEntryValue(InSelf->ArrayProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();

	int32 ValueCount = 0;
	for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
	{
		if (ContainerEntryValue.GetProp()->Identical(SelfScriptArrayHelper.GetRawPtr(ElementIndex), ContainerEntryValue.GetValue()))
		{
			++ValueCount;
		}
	}

	return (Py_ssize_t)ValueCount;
}

Py_ssize_t FPyWrapperArray::Index(FPyWrapperArray* InSelf, PyObject* InValue, Py_ssize_t InStartIndex, Py_ssize_t InStopIndex)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FArrayElementOnScope ContainerEntryValue(InSelf->ArrayProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();
	const Py_ssize_t ResolvedStartIndex = PyUtil::ResolveContainerIndexParam(InStartIndex, ElementCount);
	const Py_ssize_t ResolvedStopIndex = PyUtil::ResolveContainerIndexParam(InStopIndex, ElementCount);

	const int32 StartIndex = FMath::Min((int32)ResolvedStartIndex, ElementCount);
	const int32 StopIndex = FMath::Max((int32)ResolvedStopIndex, ElementCount);

	int32 ReturnIndex = INDEX_NONE;
	for (int32 ElementIndex = StartIndex; ElementIndex < StopIndex; ++ElementIndex)
	{
		if (ContainerEntryValue.GetProp()->Identical(SelfScriptArrayHelper.GetRawPtr(ElementIndex), ContainerEntryValue.GetValue()))
		{
			ReturnIndex = ElementIndex;
			break;
		}
	}

	if (ReturnIndex == INDEX_NONE)
	{
		PyUtil::SetPythonError(PyExc_ValueError, InSelf, TEXT("The given value was not found in the array"));
		return -1;
	}

	return ReturnIndex;
}

int FPyWrapperArray::Insert(FPyWrapperArray* InSelf, Py_ssize_t InIndex, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();
	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, ElementCount);

	const int32 InsertIndex = FMath::Min((int32)ResolvedIndex, ElementCount);
	SelfScriptArrayHelper.InsertValues(InsertIndex);

	if (!PyConversion::NativizeProperty(InValue, InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(InsertIndex)))
	{
		SelfScriptArrayHelper.RemoveValues(InsertIndex);
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert '%s' to an element of '%s' for insertion"), *PyUtil::GetFriendlyTypename(InValue), *PyUtil::GetFriendlyTypename(InSelf)));
		return -1;
	}

	return 0;
}

PyObject* FPyWrapperArray::Pop(FPyWrapperArray* InSelf, Py_ssize_t InIndex)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();
	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, ElementCount);

	if (PyUtil::ValidateContainerIndexParam(ResolvedIndex, ElementCount, InSelf->ArrayProp, *PyUtil::GetErrorContext(InSelf)) != 0)
	{
		return nullptr;
	}

	PyObject* PyReturnValue = nullptr;
	if (!PyConversion::PythonizeProperty(InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(ResolvedIndex), PyReturnValue))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert element property '%s' (%s) at index %zd"), *InSelf->ArrayProp->Inner->GetName(), *InSelf->ArrayProp->Inner->GetClass()->GetName(), ResolvedIndex));
		return nullptr;
	}

	SelfScriptArrayHelper.RemoveValues(ResolvedIndex);

	return PyReturnValue;
}

int FPyWrapperArray::Remove(FPyWrapperArray* InSelf, PyObject* InValue)
{
	const int32 ValueIndex = Index(InSelf, InValue);
	if (ValueIndex == INDEX_NONE)
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	SelfScriptArrayHelper.RemoveValues(ValueIndex);

	return 0;
}

int FPyWrapperArray::Clear(FPyWrapperArray* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	SelfScriptArrayHelper.EmptyValues();

	return 0;
}

int FPyWrapperArray::Reverse(FPyWrapperArray* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();

	// Taken from Algo::Reverse
	for (int32 i = 0, i2 = ElementCount - 1; i < ElementCount / 2 /*rounding down*/; ++i, --i2)
	{
		SelfScriptArrayHelper.SwapValues(i, i2);
	}

	return 0;
}

int FPyWrapperArray::Sort(FPyWrapperArray* InSelf, PyObject* InKey, bool InReverse)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	const int32 ElementCount = SelfScriptArrayHelper.Num();

	// This isn't ideal, but we have no sorting algorithms that take untyped data, and it's the simplest way to deal with the key (and cmp) arguments that need processing in Python.
	FPyObjectPtr PyList = FPyObjectPtr::StealReference(PyList_New(ElementCount));
	if (!PyList)
	{
		return -1;
	}

	for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
	{
		PyObject* PyItemObj = nullptr;
		if (!PyConversion::PythonizeProperty(InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(ElementIndex), PyItemObj))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert element property '%s' (%s) at index %d"), *InSelf->ArrayProp->Inner->GetName(), *InSelf->ArrayProp->Inner->GetClass()->GetName(), ElementIndex));
			return -1;
		}
		PyList_SetItem(PyList, ElementIndex, PyItemObj); // PyList_SetItem steals this reference
	}

	// We need to call 'sort' directly since PyList_Sort doesn't expose the version that takes arguments
	FPyObjectPtr PyListSortFunc = FPyObjectPtr::StealReference(PyObject_GetAttrString(PyList, "sort"));
	FPyObjectPtr PyListSortArgs = FPyObjectPtr::StealReference(PyTuple_New(0));
	FPyObjectPtr PyListSortKwds = FPyObjectPtr::StealReference(PyDict_New());

	PyDict_SetItemString(PyListSortKwds, "key", InKey ? InKey : Py_None);
	PyDict_SetItemString(PyListSortKwds, "reverse", InReverse ? Py_True : Py_False);

	FPyObjectPtr PyListSortResult = FPyObjectPtr::StealReference(PyObject_Call(PyListSortFunc, PyListSortArgs, PyListSortKwds));
	if (!PyListSortResult)
	{
		return -1;
	}

	for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
	{
		PyObject* PyItemObj = PyList_GetItem(PyList, ElementIndex);
		PyConversion::NativizeProperty(PyItemObj, InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(ElementIndex));
	}

	return 0;
}

int FPyWrapperArray::Resize(FPyWrapperArray* InSelf, Py_ssize_t InLen)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
	SelfScriptArrayHelper.Resize(FMath::Max(0, (int32)InLen));
	return 0;
}

PyTypeObject InitializePyWrapperArrayType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperArray::New(InType);
		}

		static void Dealloc(FPyWrapperArray* InSelf)
		{
			FPyWrapperArray::Free(InSelf);
		}

		static int Init(FPyWrapperArray* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;

			static const char *ArgsKwdList[] = { "type", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:call", (char**)ArgsKwdList, &PyTypeObj))
			{
				return -1;
			}

			PyUtil::FPropertyDef ArrayElementDef;
			const int ValidateTypeResult = PyUtil::ValidateContainerTypeParam(PyTypeObj, ArrayElementDef, "type", *PyUtil::GetErrorContext(Py_TYPE(InSelf)));
			if (ValidateTypeResult != 0)
			{
				return ValidateTypeResult;
			}

			return FPyWrapperArray::Init(InSelf, ArrayElementDef);
		}

		static PyObject* Str(FPyWrapperArray* InSelf)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
			const int32 ElementCount = SelfScriptArrayHelper.Num();

			FString ExportedArray;
			for (int32 ElementIndex = 0; ElementIndex < ElementCount; ++ElementIndex)
			{
				if (ElementIndex > 0)
				{
					ExportedArray += TEXT(", ");
				}
				ExportedArray += PyUtil::GetFriendlyPropertyValue(InSelf->ArrayProp->Inner, SelfScriptArrayHelper.GetRawPtr(ElementIndex), PPF_Delimited | PPF_IncludeTransient);
			}
			return PyUnicode_FromFormat("[%s]", TCHAR_TO_UTF8(*ExportedArray));
		}

		static PyObject* RichCmp(FPyWrapperArray* InSelf, PyObject* InOther, int InOp)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyUtil::FPropertyDef SelfElementDef = InSelf->ArrayProp->Inner;
			FPyWrapperArrayPtr Other = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::CastPyObject(InOther, &PyWrapperArrayType, SelfElementDef));
			if (!Other)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (InOp != Py_EQ && InOp != Py_NE)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Only == and != comparison is supported"));
				return nullptr;
			}

			const bool bIsIdentical = InSelf->ArrayProp->Identical(InSelf->ArrayInstance, Other->ArrayInstance, PPF_None);
			return PyBool_FromLong(InOp == Py_EQ ? bIsIdentical : !bIsIdentical);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperArray* InSelf)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return -1;
			}

			if (InSelf->ArrayProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
			{
				const uint32 ArrayHash = InSelf->ArrayProp->GetValueTypeHash(InSelf->ArrayInstance);
				return (PyUtil::FPyHashType)(ArrayHash != -1 ? ArrayHash : 0);
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Type cannot be hashed"));
			return -1;
		}

		static PyObject* GetIter(FPyWrapperArray* InSelf)
		{
			typedef TPyPtr<FPyWrapperArrayIterator> FPyWrapperArrayIteratorPtr;

			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FPyWrapperArrayIteratorPtr NewIter = FPyWrapperArrayIteratorPtr::StealReference(FPyWrapperArrayIterator::New(&PyWrapperArrayIteratorType));
			if (FPyWrapperArrayIterator::Init(NewIter, InSelf) != 0)
			{
				return nullptr;
			}

			return (PyObject*)NewIter.Release();
		}
	};

	struct FSequenceFuncs
	{
		static Py_ssize_t Len(FPyWrapperArray* InSelf)
		{
			return FPyWrapperArray::Len(InSelf);
		}

		static PyObject* GetItem(FPyWrapperArray* InSelf, Py_ssize_t InIndex)
		{
			return FPyWrapperArray::GetItem(InSelf, InIndex);
		}

		static int SetItem(FPyWrapperArray* InSelf, Py_ssize_t InIndex, PyObject* InValue)
		{
			return FPyWrapperArray::SetItem(InSelf, InIndex, InValue);
		}

		static int Contains(FPyWrapperArray* InSelf, PyObject* InValue)
		{
			return FPyWrapperArray::Contains(InSelf, InValue);
		}

		static PyObject* Concat(FPyWrapperArray* InSelf, PyObject* InOther)
		{
			return (PyObject*)FPyWrapperArray::Concat(InSelf, InOther);
		}

		static PyObject* ConcatInplace(FPyWrapperArray* InSelf, PyObject* InOther)
		{
			if (FPyWrapperArray::ConcatInplace(InSelf, InOther) != 0)
			{
				return nullptr;
			}
			Py_RETURN_NONE;
		}

		static PyObject* Repeat(FPyWrapperArray* InSelf, Py_ssize_t InMultiplier)
		{
			return (PyObject*)FPyWrapperArray::Repeat(InSelf, InMultiplier);
		}

		static PyObject* RepeatInplace(FPyWrapperArray* InSelf, Py_ssize_t InMultiplier)
		{
			if (FPyWrapperArray::RepeatInplace(InSelf, InMultiplier) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* GetSlice(FPyWrapperArray* InSelf, Py_ssize_t InSliceStart, Py_ssize_t InSliceStop)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
			const int32 SelfElementCount = SelfScriptArrayHelper.Num();
			const Py_ssize_t ResolvedSliceStart = PyUtil::ResolveContainerIndexParam(InSliceStart, SelfElementCount);
			const Py_ssize_t ResolvedSliceStop = PyUtil::ResolveContainerIndexParam(InSliceStop, SelfElementCount);

			const Py_ssize_t SliceLen = FMath::Max(ResolvedSliceStop - ResolvedSliceStart, (Py_ssize_t)0);

			const PyUtil::FPropertyDef SelfElementDef = InSelf->ArrayProp->Inner;
			FPyWrapperArrayPtr NewArray = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::New(Py_TYPE(InSelf)));
			if (FPyWrapperArray::Init(NewArray, SelfElementDef) != 0)
			{
				return nullptr;
			}

			FScriptArrayHelper NewScriptArrayHelper(NewArray->ArrayProp, NewArray->ArrayInstance);
			NewScriptArrayHelper.Resize(SliceLen);

			for (Py_ssize_t ElementIndex = ResolvedSliceStart; ElementIndex < ResolvedSliceStop; ++ElementIndex)
			{
				InSelf->ArrayProp->Inner->CopyCompleteValue(NewScriptArrayHelper.GetRawPtr((int32)(ElementIndex - ResolvedSliceStart)), SelfScriptArrayHelper.GetRawPtr((int32)ElementIndex));
			}
			return (PyObject*)NewArray.Release();
		}

		static int SetSlice(FPyWrapperArray* InSelf, Py_ssize_t InSliceStart, Py_ssize_t InSliceStop, PyObject* InValue)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return -1;
			}

			FScriptArrayHelper SelfScriptArrayHelper(InSelf->ArrayProp, InSelf->ArrayInstance);
			const int32 SelfElementCount = SelfScriptArrayHelper.Num();
			const Py_ssize_t ResolvedSliceStart = PyUtil::ResolveContainerIndexParam(InSliceStart, SelfElementCount);
			const Py_ssize_t ResolvedSliceStop = PyUtil::ResolveContainerIndexParam(InSliceStop, SelfElementCount);

			const Py_ssize_t SliceLen = FMath::Max(ResolvedSliceStop - ResolvedSliceStart, (Py_ssize_t)0);

			// Value will be null when performing a slice delete
			FPyWrapperArrayPtr Value;
			if (InValue)
			{
				const PyUtil::FPropertyDef SelfElementDef = InSelf->ArrayProp->Inner;
				Value = FPyWrapperArrayPtr::StealReference(FPyWrapperArray::CastPyObject(InValue, &PyWrapperArrayType, SelfElementDef));
				if (!Value)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot assign type '%s' to type '%s' during a slice"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(InValue)));
					return -1;
				}
			}

			SelfScriptArrayHelper.RemoveValues((int32)ResolvedSliceStart, (int32)SliceLen);

			if (Value)
			{
				FScriptArrayHelper ValueScriptArrayHelper(Value->ArrayProp, Value->ArrayInstance);
				const int32 ValueElementCount = ValueScriptArrayHelper.Num();

				SelfScriptArrayHelper.InsertValues((int32)ResolvedSliceStart, ValueElementCount);
				for (int32 ElementIndex = 0; ElementIndex < ValueElementCount; ++ElementIndex)
				{
					InSelf->ArrayProp->Inner->CopyCompleteValue(SelfScriptArrayHelper.GetRawPtr(ResolvedSliceStart + ElementIndex), ValueScriptArrayHelper.GetRawPtr(ElementIndex));
				}
			}

			return 0;
		}
	};

	struct FMappingFuncs
	{
		static PyObject* GetItem(FPyWrapperArray* InSelf, PyObject* InIndexer)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			// Array types support numeric and slice based indexing
			int32 Index = 0;
			if (PyConversion::Nativize(InIndexer, Index, PyConversion::ESetErrorState::No))
			{
				return FPyWrapperArray::GetItem(InSelf, (Py_ssize_t)Index);
			}

			if (PySlice_Check(InIndexer))
			{
				Py_ssize_t SliceStart = 0;
				Py_ssize_t SliceStop = 0;
				Py_ssize_t SliceStep = 0;
				Py_ssize_t SliceLen = 0;
				if (PySlice_GetIndicesEx((PyUtil::FPySliceType*)InIndexer, FPyWrapperArray::Len(InSelf), &SliceStart, &SliceStop, &SliceStep, &SliceLen) != 0)
				{
					return nullptr;
				}
				if (SliceStep == 1)
				{
					return FSequenceFuncs::GetSlice(InSelf, SliceStart, SliceStop);
				}
				else
				{
					PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Slice step must be 1"));
					return nullptr;
				}
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Indexer '%s' was invalid for '%s' (%s)"), *PyUtil::GetFriendlyTypename(InIndexer), *InSelf->ArrayProp->GetName(), *InSelf->ArrayProp->GetClass()->GetName()));
			return nullptr;
		}

		static int SetItem(FPyWrapperArray* InSelf, PyObject* InIndexer, PyObject* InValue)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return -1;
			}

			// Array types support numeric and slice based indexing
			int32 Index = 0;
			if (PyConversion::Nativize(InIndexer, Index, PyConversion::ESetErrorState::No))
			{
				return FPyWrapperArray::SetItem(InSelf, (Py_ssize_t)Index, InValue);
			}

			if (PySlice_Check(InIndexer))
			{
				Py_ssize_t SliceStart = 0;
				Py_ssize_t SliceStop = 0;
				Py_ssize_t SliceStep = 0;
				Py_ssize_t SliceLen = 0;
				if (PySlice_GetIndicesEx((PyUtil::FPySliceType*)InIndexer, FPyWrapperArray::Len(InSelf), &SliceStart, &SliceStop, &SliceStep, &SliceLen) != 0)
				{
					return -1;
				}
				if (SliceStep == 1)
				{
					return FSequenceFuncs::SetSlice(InSelf, SliceStart, SliceStop, InValue);
				}
				else
				{
					PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Slice step must be 1"));
					return -1;
				}
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Indexer '%s' was invalid for '%s' (%s)"), *PyUtil::GetFriendlyTypename(InIndexer), *InSelf->ArrayProp->GetName(), *InSelf->ArrayProp->GetClass()->GetName()));
			return -1;
		}
	};

	struct FMethods
	{
		// This is for type hinting, to make the class appears as a generic (https://peps.python.org/pep-0560/) and support hinting like 'a: unreal.Array[int] = unreal.Array(int)'
		static PyObject* GetClassItem(PyObject* Type, PyObject* Item)
		{
			Py_INCREF(Type);
			return Type;
		}

		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;
			PyObject* PyObj = nullptr;

			static const char *ArgsKwdList[] = { "type", "obj", nullptr };
			if (PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:call", (char**)ArgsKwdList, &PyTypeObj, &PyObj))
			{
				PyUtil::FPropertyDef ArrayElementDef;
				if (PyUtil::ValidateContainerTypeParam(PyTypeObj, ArrayElementDef, "type", *PyUtil::GetErrorContext(InType)) != 0)
				{
					return nullptr;
				}

				PyObject* PyCastResult = (PyObject*)FPyWrapperArray::CastPyObject(PyObj, InType, ArrayElementDef);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperArray* InSelf)
		{
			if (!FPyWrapperArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperArrayFactory::Get().CreateInstance(InSelf->ArrayInstance, InSelf->ArrayProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}

		static PyObject* Append(FPyWrapperArray* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:append", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperArray::Append(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Count(FPyWrapperArray* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:count", &PyObj))
			{
				return nullptr;
			}

			const int32 ValueCount = FPyWrapperArray::Count(InSelf, PyObj);
			if (ValueCount == -1)
			{
				return nullptr;
			}

			return PyConversion::Pythonize(ValueCount);
		}

		static PyObject* Extend(FPyWrapperArray* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:extend", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperArray::ConcatInplace(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Index(FPyWrapperArray* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyValueObj = nullptr;
			PyObject* PyStartObj = nullptr;
			PyObject* PyStopObj = nullptr;

			static const char *ArgsKwdList[] = { "value", "start", "stop", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|OO:index", (char**)ArgsKwdList, &PyValueObj, &PyStartObj, &PyStopObj))
			{
				return nullptr;
			}

			int32 StartIndex = 0;
			if (PyStartObj && !PyConversion::Nativize(PyStartObj, StartIndex))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'start' (%s) to 'int32'"), *PyUtil::GetFriendlyTypename(PyStartObj)));
				return nullptr;
			}

			int32 StopIndex = -1;
			if (PyStopObj && !PyConversion::Nativize(PyStopObj, StopIndex))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'stop' (%s) to 'int32'"), *PyUtil::GetFriendlyTypename(PyStopObj)));
				return nullptr;
			}

			const int32 ValueIndex = FPyWrapperArray::Index(InSelf, PyValueObj, StartIndex, StopIndex);
			if (ValueIndex == -1)
			{
				return nullptr;
			}

			return PyConversion::Pythonize(ValueIndex);
		}

		static PyObject* Insert(FPyWrapperArray* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyIndexObj = nullptr;
			PyObject* PyValueObj = nullptr;

			static const char *ArgsKwdList[] = { "index", "value", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:insert", (char**)ArgsKwdList, &PyIndexObj, &PyValueObj))
			{
				return nullptr;
			}

			int32 InsertIndex = 0;
			if (!PyConversion::Nativize(PyIndexObj, InsertIndex))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'index' (%s) to 'int32'"), *PyUtil::GetFriendlyTypename(PyIndexObj)));
				return nullptr;
			}

			if (FPyWrapperArray::Insert(InSelf, InsertIndex, PyValueObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Pop(FPyWrapperArray* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "|O:pop", &PyObj))
			{
				return nullptr;
			}

			int32 ValueIndex = INDEX_NONE;
			if (PyObj && !PyConversion::Nativize(PyObj, ValueIndex))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'index' (%s) to 'int32'"), *PyUtil::GetFriendlyTypename(PyObj)));
				return nullptr;
			}

			return FPyWrapperArray::Pop(InSelf, ValueIndex);
		}

		static PyObject* Clear(FPyWrapperArray* InSelf)
		{
			if (FPyWrapperArray::Clear(InSelf) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Remove(FPyWrapperArray* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:remove", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperArray::Remove(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Reverse(FPyWrapperArray* InSelf)
		{
			if (FPyWrapperArray::Reverse(InSelf) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Sort(FPyWrapperArray* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyKeyObj = nullptr;
			PyObject* PyReverseObj = nullptr;

			static const char *ArgsKwdList[] = { "key", "reverse", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "|OO:sort", (char**)ArgsKwdList, &PyKeyObj, &PyReverseObj))
			{
				return nullptr;
			}

			bool bReverse = false;
			if (PyReverseObj && !PyConversion::Nativize(PyReverseObj, bReverse))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'reverse' (%s) to 'bool'"), *PyUtil::GetFriendlyTypename(PyReverseObj)));
				return nullptr;
			}

			if (FPyWrapperArray::Sort(InSelf, PyKeyObj, bReverse) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Resize(FPyWrapperArray* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:resize", &PyObj))
			{
				return nullptr;
			}

			int32 Len = 0;
			if (PyUtil::ValidateContainerLenParam(PyObj, Len, "len", *PyUtil::GetErrorContext(InSelf)) != 0)
			{
				return nullptr;
			}

			if (FPyWrapperArray::Resize(InSelf, Len) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}
	};

	// NOTE: _ElemType = typing.TypeVar('_ElemType') and Type/Optional/Iterable/Callable are defines by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "__class_getitem__", PyCFunctionCast(FMethods::GetClassItem), METH_O|METH_CLASS, "__class_getitem__(cls, item: _ElemType) -- implemented for type hinting purpose only" },
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_KEYWORDS | METH_CLASS, "cast(cls, type: Type[_ElemType], obj: object) -> Array[_ElemType] -- cast the given object to this Unreal array type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> Array[_ElemType] -- copy this Unreal array" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> Array[_ElemType] -- copy this Unreal array" },
		{ "append", PyCFunctionCast(&FMethods::Append), METH_VARARGS, "append(self, value: _ElemType) -> None -- append the given value to the end of this Unreal array (equivalent to TArray::Add in C++)" },
		{ "count", PyCFunctionCast(&FMethods::Count), METH_VARARGS, "count(self, value: _ElemType) -> int -- return the number of times that value appears in this this Unreal array" },
		{ "extend", PyCFunctionCast(&FMethods::Extend), METH_VARARGS, "extend(self, values: Iterable[_ElemType]) -> None -- extend this Unreal array by appending elements from the given iterable (equivalent to TArray::Append in C++)" },
		{ "index", PyCFunctionCast(&FMethods::Index), METH_VARARGS | METH_KEYWORDS, "index(self, value: _ElemType, start: int = 0, stop: int = -1) -> int -- get the index of the first matching value in this Unreal array, or raise ValueError if missing (equivalent to TArray::IndexOfByKey in C++)" },
		{ "insert", PyCFunctionCast(&FMethods::Insert), METH_VARARGS | METH_KEYWORDS, "insert(self, index: int, value: _ElemType) -> None -- insert the given value at the given index in this Unreal array" },
		{ "pop", PyCFunctionCast(&FMethods::Pop), METH_VARARGS, "pop(self, index: int = -1) -> _ElemType -- remove and return the value at the given index in this Unreal array, or raise IndexError if the index is out-of-bounds" },
		{ "clear", PyCFunctionCast(&FMethods::Clear), METH_NOARGS, "clear(self) -> None -- remove all values from this Unreal array" },
		{ "remove", PyCFunctionCast(&FMethods::Remove), METH_VARARGS, "remove(self, value: _ElemType) -> None -- remove the first matching value in this Unreal array, or raise ValueError if missing" },
		{ "reverse", PyCFunctionCast(&FMethods::Reverse), METH_NOARGS, "reverse(self) -> None -- reverse this Unreal array in-place" },
		{ "sort", PyCFunctionCast(&FMethods::Sort), METH_VARARGS | METH_KEYWORDS, "sort(self, key: Optional[Callable[[_ElemType], object]]=None, reverse: bool=False) -> None -- stable sort this Unreal array in-place" },
		{ "resize", PyCFunctionCast(&FMethods::Resize), METH_VARARGS, "resize(self, len: int) -> None -- resize this Unreal array to hold the given number of elements" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"Array", /* tp_name */
		sizeof(FPyWrapperArray), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed array instances";

	static PySequenceMethods PySequence;
	PySequence.sq_length = (lenfunc)&FSequenceFuncs::Len;
	PySequence.sq_item = (ssizeargfunc)&FSequenceFuncs::GetItem;
	PySequence.sq_ass_item = (ssizeobjargproc)&FSequenceFuncs::SetItem;
	PySequence.sq_contains = (objobjproc)&FSequenceFuncs::Contains;
	PySequence.sq_concat = (binaryfunc)&FSequenceFuncs::Concat;
	PySequence.sq_inplace_concat = (binaryfunc)&FSequenceFuncs::ConcatInplace;
	PySequence.sq_repeat = (ssizeargfunc)&FSequenceFuncs::Repeat;
	PySequence.sq_inplace_repeat = (ssizeargfunc)&FSequenceFuncs::RepeatInplace;

	static PyMappingMethods PyMapping;
	PyMapping.mp_subscript = (binaryfunc)&FMappingFuncs::GetItem;
	PyMapping.mp_ass_subscript = (objobjargproc)&FMappingFuncs::SetItem;

	PyType.tp_as_sequence = &PySequence;
	PyType.tp_as_mapping = &PyMapping;

	return PyType;
}

PyTypeObject InitializePyWrapperArrayIteratorType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperArrayIterator::New(InType);
		}

		static void Dealloc(FPyWrapperArrayIterator* InSelf)
		{
			FPyWrapperArrayIterator::Free(InSelf);
		}

		static int Init(FPyWrapperArrayIterator* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				return -1;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperArrayType) != 1)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot initialize '%s' with an instance of '%s'"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return FPyWrapperArrayIterator::Init(InSelf, (FPyWrapperArray*)PyObj);
		}

		static FPyWrapperArrayIterator* GetIter(FPyWrapperArrayIterator* InSelf)
		{
			return FPyWrapperArrayIterator::GetIter(InSelf);
		}

		static PyObject* IterNext(FPyWrapperArrayIterator* InSelf)
		{
			return FPyWrapperArrayIterator::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"ArrayIterator", /* tp_name */
		sizeof(FPyWrapperArrayIterator), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed array iterators";

	return PyType;
}

PyTypeObject PyWrapperArrayType = InitializePyWrapperArrayType();
PyTypeObject PyWrapperArrayIteratorType = InitializePyWrapperArrayIteratorType();

void FPyWrapperArrayMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperArray* Self = static_cast<FPyWrapperArray*>(Instance);
	if (Self->ArrayProp && Self->ArrayInstance && !Self->OwnerContext.HasOwner())
	{
		Self->ArrayProp.AddReferencedObjects(Collector);
		FPyReferenceCollector::AddReferencedObjectsFromProperty(Collector, Self->ArrayProp, Self->ArrayInstance);
	}
}

#endif	// WITH_PYTHON
