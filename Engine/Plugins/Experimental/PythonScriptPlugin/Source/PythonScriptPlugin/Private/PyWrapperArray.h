// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "PyWrapperOwnerContext.h"
#include "PyUtil.h"

#if WITH_PYTHON

/** Python type for FPyWrapperArray */
extern PyTypeObject PyWrapperArrayType;

/** Initialize the PyWrapperArray types and add them to the given Python module */
void InitializePyWrapperArray(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed array instances */
struct FPyWrapperArray : public FPyWrapperBase
{
	/** The owner of the wrapped array instance (if any) */
	FPyWrapperOwnerContext OwnerContext;

	/** Property describing the array */
	PyUtil::FConstArrayPropOnScope ArrayProp;

	/** Wrapped array instance */
	void* ArrayInstance;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperArray* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperArray* InSelf);

	/** Initialize this wrapper (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperArray* InSelf, const PyUtil::FPropertyDef& InElementDef);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperArray* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FArrayProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperArray* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperArray* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperArray* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperArray* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InElementDef, FPyConversionResult* OutCastResult = nullptr);

	/** Get the length of this container (equivalent to 'len(x)' in Python) */
	static Py_ssize_t Len(FPyWrapperArray* InSelf);

	/** Get the item at index N (equivalent to 'x[N]' in Python, returns new reference) */
	static PyObject* GetItem(FPyWrapperArray* InSelf, Py_ssize_t InIndex);

	/** Set the item at index N (equivalent to 'x[N] = v' in Python) */
	static int SetItem(FPyWrapperArray* InSelf, Py_ssize_t InIndex, PyObject* InValue);

	/** Removes the item at index N */
	static int RemoveAt(FPyWrapperArray* InSelf, Py_ssize_t InIndex);

	/** Does this container have an entry with the given value? (equivalent to 'v in x' in Python) */
	static int Contains(FPyWrapperArray* InSelf, PyObject* InValue);

	/** Concatenate the other object to this one, returning a new container (equivalent to 'x + o' in Python, returns new reference) */
	static FPyWrapperArray* Concat(FPyWrapperArray* InSelf, PyObject* InOther);

	/** Concatenate the other object to this one in-place (equivalent to 'x += o' in Python) */
	static int ConcatInplace(FPyWrapperArray* InSelf, PyObject* InOther);

	/** Repeat this container by N, returning a new container (equivalent to 'x * N' in Python, returns new reference) */
	static FPyWrapperArray* Repeat(FPyWrapperArray* InSelf, Py_ssize_t InMultiplier);

	/** Repeat this container by N in-place (equivalent to 'x *= N' in Python) */
	static int RepeatInplace(FPyWrapperArray* InSelf, Py_ssize_t InMultiplier);

	/** Append the given value to the end this container (equivalent to 'x.append(v)' in Python) */
	static int Append(FPyWrapperArray* InSelf, PyObject* InValue);

	/** Count the number of times that the given value appears in this container (equivalent to 'x.count(v)' in Python) */
	static Py_ssize_t Count(FPyWrapperArray* InSelf, PyObject* InValue);

	/** Get the index of the first the given value appears in this container (equivalent to 'x.index(v)' in Python) */
	static Py_ssize_t Index(FPyWrapperArray* InSelf, PyObject* InValue, Py_ssize_t InStartIndex = 0, Py_ssize_t InStopIndex = INDEX_NONE);

	/** Insert the given value into this container at the given index (equivalent to 'x.insert(i, v)' in Python) */
	static int Insert(FPyWrapperArray* InSelf, Py_ssize_t InIndex, PyObject* InValue);

	/** Pop and return the value at the given index (or the end) of this container (equivalent to 'x.pop()' in Python) */
	static PyObject* Pop(FPyWrapperArray* InSelf, Py_ssize_t InIndex = INDEX_NONE);

	/** Remove the given value from this container (equivalent to 'x.remove(v)' in Python) */
	static int Remove(FPyWrapperArray* InSelf, PyObject* InValue);

	/** Remove all values from this container (equivalent to 'x.clear()' in Python) */
	static int Clear(FPyWrapperArray* InSelf);

	/** Reverse this container (equivalent to 'x.reverse()' in Python) */
	static int Reverse(FPyWrapperArray* InSelf);

	/** Sort this container (equivalent to 'x.sort()' in Python) */
	static int Sort(FPyWrapperArray* InSelf, PyObject* InKey, bool InReverse);

	/** Resize this container (equivalent to 'x.resize(l)' in Python) */
	static int Resize(FPyWrapperArray* InSelf, Py_ssize_t InLen);
};

/** Meta-data for all Unreal exposed array types */
struct FPyWrapperArrayMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperArrayMetaData, FGuid(0xFCD8DF06, 0x7AD6426F, 0x97482D82, 0x6E100C39))

	FPyWrapperArrayMetaData()
	{
	}

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;
};

typedef TPyPtr<FPyWrapperArray> FPyWrapperArrayPtr;

#endif	// WITH_PYTHON
