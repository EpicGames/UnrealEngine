// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperBase.h"
#include "PyReferenceCollector.h"
#include "PyGIL.h"
#include "PyUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PyWrapperBase)

#if WITH_PYTHON

/** Python type for FPyWrapperBaseMetaDataObject */
extern PyTypeObject PyWrapperBaseMetaDataType;

/** Python wrapper object for FPyWrapperBaseMetaData */
struct FPyWrapperBaseMetaDataObject
{
	/** Common Python Object */
	PyObject_HEAD

	/** Type meta-data */
	FPyWrapperBaseMetaData* MetaData;
};

void InitializePyWrapperBase(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperBaseType) == 0)
	{
		ModuleInfo.AddType(&PyWrapperBaseType);
	}

	PyType_Ready(&PyWrapperBaseMetaDataType);
}

FPyWrapperBase* FPyWrapperBase::New(PyTypeObject* InType)
{
	FPyWrapperBase* Self = (FPyWrapperBase*)InType->tp_alloc(InType, 0);
	if (Self)
	{
		FPyReferenceCollector::Get().AddWrappedInstance(Self);
	}
	return Self;
}

void FPyWrapperBase::Free(FPyWrapperBase* InSelf)
{
	FPyReferenceCollector::Get().RemoveWrappedInstance(InSelf);
	Deinit(InSelf);
	Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
}

int FPyWrapperBase::Init(FPyWrapperBase* InSelf)
{
	Deinit(InSelf);
	return 0;
}

void FPyWrapperBase::Deinit(FPyWrapperBase* InSelf)
{
}

PyTypeObject InitializePyWrapperBaseType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperBase::New(InType);
		}

		static void Dealloc(FPyWrapperBase* InSelf)
		{
			FPyWrapperBase::Free(InSelf);
		}

		static int Init(FPyWrapperBase* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			return FPyWrapperBase::Init(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"_WrapperBase", /* tp_name */
		sizeof(FPyWrapperBase), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Base type for all Unreal exposed types";

	return PyType;
}

PyTypeObject InitializePyWrapperBaseMetaDataType()
{
	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"PyWrapperBaseMetaData", /* tp_name */
		sizeof(FPyWrapperBaseMetaDataObject), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&PyType_GenericNew;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Python wrapper object for FPyWrapperBaseMetaData";

	return PyType;
}

PyTypeObject PyWrapperBaseType = InitializePyWrapperBaseType();
PyTypeObject PyWrapperBaseMetaDataType = InitializePyWrapperBaseMetaDataType();

void FPyWrapperBaseMetaData::SetMetaData(PyTypeObject* PyType, FPyWrapperBaseMetaData* MetaData)
{
	if (PyType && PyType->tp_dict)
	{
		FPyScopedGIL GIL;

		FPyWrapperBaseMetaDataObject* PyWrapperMetaData = (FPyWrapperBaseMetaDataObject*)PyDict_GetItemString(PyType->tp_dict, "_wrapper_meta_data");
		if (!PyWrapperMetaData)
		{
			PyWrapperMetaData = (FPyWrapperBaseMetaDataObject*)PyObject_CallObject((PyObject*)&PyWrapperBaseMetaDataType, nullptr);
			checkf(PyWrapperMetaData, TEXT("PyWrapperMetaData is null for PyType: %s"),
				*PyUtil::GetFriendlyTypename(PyType));

			PyDict_SetItemString(PyType->tp_dict, "_wrapper_meta_data", (PyObject*)PyWrapperMetaData);
			Py_DECREF(PyWrapperMetaData);
		}
		PyWrapperMetaData->MetaData = MetaData;
	}
}

FPyWrapperBaseMetaData* FPyWrapperBaseMetaData::GetMetaData(PyTypeObject* PyType)
{
	if (PyType && PyType->tp_dict)
	{
		FPyScopedGIL GIL;

		FPyWrapperBaseMetaDataObject* PyWrapperMetaData = (FPyWrapperBaseMetaDataObject*)PyDict_GetItemString(PyType->tp_dict, "_wrapper_meta_data");
		if (PyWrapperMetaData)
		{
			return PyWrapperMetaData->MetaData;
		}
	}
	return nullptr;
}

FPyWrapperBaseMetaData* FPyWrapperBaseMetaData::GetMetaData(FPyWrapperBase* Instance)
{
	return GetMetaData(Py_TYPE(Instance));
}

#endif	// WITH_PYTHON

UPythonResourceOwner::UPythonResourceOwner(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_PYTHON

UPythonObjectHandle* UPythonObjectHandle::Create(PyObject* PyObjPtr)
{
	UPythonObjectHandle* Handle = nullptr;
	if (PyObjPtr && PyObjPtr != Py_None)
	{
		Handle = NewObject<UPythonObjectHandle>();
		{
			FPyScopedGIL GIL;
			Handle->PyObj = FPyObjectPtr::NewReference(PyObjPtr);
		}
	}
	return Handle;
}

PyObject* UPythonObjectHandle::Resolve() const
{
	FPyScopedGIL GIL;
	PyObject* PyObjPtr = PyObj.GetPtr();
	return PyObjPtr ? PyObjPtr : Py_None;
}

void UPythonObjectHandle::BeginDestroy()
{
	ReleasePythonResources();
	Super::BeginDestroy();
}

void UPythonObjectHandle::ReleasePythonResources()
{
	// This may be called after Python has already shut down
	if (Py_IsInitialized())
	{
		FPyScopedGIL GIL;
		PyObj.Reset();
	}
	else
	{
		// Release ownership if Python has been shut down to avoid attempting to delete the objects (which are already dead)
		PyObj.Release();
	}
}

#endif // WITH_PYTHON
