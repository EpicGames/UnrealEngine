// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"
#include "PyGenUtil.h"
#include "PyConversionMethod.h"
#include "PyConversionResult.h"
#include "Misc/Guid.h"
#include "UObject/GCObject.h"
#include "UObject/Interface.h"
#include "PyWrapperBase.generated.h"

#if WITH_PYTHON

/** Python type for FPyWrapperBase */
extern PyTypeObject PyWrapperBaseType;

/** Initialize the PyWrapperBase types and add them to the given Python module */
void InitializePyWrapperBase(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Base type for all Unreal exposed instances */
struct FPyWrapperBase
{
	/** Common Python Object */
	PyObject_HEAD

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperBase* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperBase* InSelf);

	/** Initialize this wrapper instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperBase* InSelf);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperBase* InSelf);
};

#define PY_OVERRIDE_GETSET_METADATA(TYPE)																						\
	static void SetMetaData(PyTypeObject* PyType, TYPE* MetaData) { FPyWrapperBaseMetaData::SetMetaData(PyType, MetaData); }	\
	static TYPE* GetMetaData(PyTypeObject* PyType) { return (TYPE*)FPyWrapperBaseMetaData::GetMetaData(PyType); }				\
	static TYPE* GetMetaData(FPyWrapperBase* Instance) { return (TYPE*)FPyWrapperBaseMetaData::GetMetaData(Instance); }

#define PY_METADATA_METHODS(TYPE, GUID)																							\
	PY_OVERRIDE_GETSET_METADATA(TYPE)																							\
	static FGuid StaticTypeId() { return (GUID); }																				\
	virtual FGuid GetTypeId() const override { return StaticTypeId(); }

/** Base meta-data for all Unreal exposed types */
struct FPyWrapperBaseMetaData
{
	/** Set the meta-data object on the given type */
	static void SetMetaData(PyTypeObject* PyType, FPyWrapperBaseMetaData* MetaData);

	/** Get the meta-data object from the given type */
	static FPyWrapperBaseMetaData* GetMetaData(PyTypeObject* PyType);

	/** Get the meta-data object from the type of the given instance */
	static FPyWrapperBaseMetaData* GetMetaData(FPyWrapperBase* Instance);

	FPyWrapperBaseMetaData()
	{
	}

	virtual ~FPyWrapperBaseMetaData() = default;

	/** Get the ID associated with this meta-data type */
	virtual FGuid GetTypeId() const = 0;

	/** Get the reflection meta data type object associated with this wrapper type if there is one or nullptr if not. */
	virtual const UField* GetMetaType() const
	{
		return nullptr;
	}

	/** Add object references from this type meta-data to the given collector */
	virtual void AddTypeReferencedObjects(FReferenceCollector& Collector)
	{
	}

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
	{
	}
};

typedef TPyPtr<FPyWrapperBase> FPyWrapperBasePtr;

#endif	// WITH_PYTHON

UINTERFACE(MinimalApi)
class UPythonResourceOwner : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IPythonResourceOwner
{
	GENERATED_IINTERFACE_BODY()

public:
	/**
	 * Release all Python resources owned by this object.
	 * Called during Python shutdown to free resources from lingering UObject-based instances.
	 */
	virtual void ReleasePythonResources() = 0;
};

/**
 * Handle object to wrap a Python object for use as a UPROPERTY on a UCLASS or USTRUCT.
 * This allows Python generated types to store arbitrary Python objects as native properties, 
 * rather than rely on the lifetime of the Python wrapper object to keep a non-property object reference alive.
 * 
 * The following functions are available in Python script to interact with these handles:
 *   - unreal.create_python_object_handle(obj) -> handle  - Creates an unreal.PythonObjectHandle from a given PyObject.
 *   - unreal.resolve_python_object_handle(handle) -> obj - Resolves an unreal.PythonObjectHandle to its PyObject, or None.
 *   - unreal.destroy_python_object_handle(handle)        - Destroys an unreal.PythonObjectHandle (clearing its PyObject reference).
 * 
 * Manually calling destroy is optional, but can be useful when you need to *immediately* clean-up a reference to the wrapped 
 * Python object, rather than wait for UE GC to clean it up at an arbitrary point in the future.
 * 
 * This can be used to create a property on Python generated types via the standard unreal.uproperty syntax, eg):
 *   MyObjectHandleProperty = unreal.uproperty(unreal.PythonObjectHandle)
 */
UCLASS(Transient, BlueprintType)
class UPythonObjectHandle final : public UObject, public IPythonResourceOwner
{
	GENERATED_BODY()

#if WITH_PYTHON

public:
	/**
	 * Create a handle that wraps the given Python object.
	 * @note Will return null if passed a null or None Python object.
	 * @param PyObjPtr Python object to wrap, may be null or None.
	 */
	static UPythonObjectHandle* Create(PyObject* PyObjPtr);

	/**
	 * Resolve this handle to its underlying Python object, if any.
	 * @return The wrapped object, or None (but not null; borrowed reference).
	 */
	PyObject* Resolve() const;

	//~ UObject interface
	virtual void BeginDestroy() override;

	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override;

private:
	/** Wrapped Python object, if any */
	FPyObjectPtr PyObj;

#else	// WITH_PYTHON

public:
	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override {}

#endif	// WITH_PYTHON
};
