// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"

#if WITH_PYTHON

template <typename SelfType>
PyTypeObject InitializePyWrapperBasicType(const char* InTypeName, const char* InTypeDoc)
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)SelfType::New(InType);
		}

		static void Dealloc(SelfType* InSelf)
		{
			SelfType::Free(InSelf);
		}

		static int Init(SelfType* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			return SelfType::Init(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		InTypeName, /* tp_name */
		sizeof(SelfType), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = InTypeDoc;

	return PyType;
}

/** Base type for any Unreal exposed simple value instances (that just copy data into Python) */
template <typename ValueType, typename SelfType>
struct TPyWrapperBasic : public FPyWrapperBase
{
	/** The wrapped instance */
	ValueType Value;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static SelfType* New(PyTypeObject* InType)
	{
		SelfType* Self = (SelfType*)FPyWrapperBase::New(InType);
		if (Self)
		{
			new(&Self->Value) ValueType();
		}
		return Self;
	}

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(SelfType* InSelf)
	{
		Deinit(InSelf);

		InSelf->Value.~ValueType();
		FPyWrapperBase::Free(InSelf);
	}

	/** Initialize this wrapper instance (called via tp_init for Python, or directly in C++) */
	static int Init(SelfType* InSelf)
	{
		Deinit(InSelf);

		const int BaseInit = FPyWrapperBase::Init(InSelf);
		if (BaseInit != 0)
		{
			return BaseInit;
		}

		SelfType::InitValue(InSelf, ValueType());
		return 0;
	}

	/** Initialize this wrapper instance to the given value (called directly in C++) */
	static int Init(SelfType* InSelf, const ValueType InValue)
	{
		Deinit(InSelf);

		const int BaseInit = FPyWrapperBase::Init(InSelf);
		if (BaseInit != 0)
		{
			return BaseInit;
		}

		SelfType::InitValue(InSelf, InValue);
		return 0;
	}

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(SelfType* InSelf)
	{
		SelfType::DeinitValue(InSelf);
	}

	/** Initialize the value of this wrapper instance (internal: define this in a derived type to "override" InitValue behavior) */
	static void InitValue(SelfType* InSelf, const ValueType InValue)
	{
		InSelf->Value = InValue;
	}

	/** Deinitialize the value of this wrapper instance (internal: define this in a derived type to "override" DeinitValue behavior) */
	static void DeinitValue(SelfType* InSelf)
	{
		InSelf->Value = ValueType();
	}
};

#endif	// WITH_PYTHON
