// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperDelegate.h"
#include "PyWrapperObject.h"
#include "PyWrapperTypeRegistry.h"
#include "PyGIL.h"
#include "PyUtil.h"
#include "PyGenUtil.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/StructOnScope.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PyWrapperDelegate)

const FName UPythonCallableForDelegate::GeneratedFuncName = "CallPython";

DEFINE_FUNCTION(UPythonCallableForDelegate::CallPythonNative)
{
#if WITH_PYTHON
	// Note: This function *must not* return until InvokePythonCallableFromUnrealFunctionThunk has been called, as we need to step over the correct amount of data from the bytecode stack!

	const UFunction* Func = Stack.CurrentNativeFunction;

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		if (!PyGenUtil::InvokePythonCallableFromUnrealFunctionThunk(FPyObjectPtr(), P_THIS->PyCallable, Func, Context, Stack, RESULT_PARAM))
		{
			PyUtil::ReThrowPythonError();
		}
	}
#endif	// WITH_PYTHON
}

#if WITH_PYTHON
void UPythonCallableForDelegate::BeginDestroy()
{
	ReleasePythonResources();
	Super::BeginDestroy();
}

void UPythonCallableForDelegate::ReleasePythonResources()
{
	// This may be called after Python has already shut down
	if (Py_IsInitialized())
	{
		FPyScopedGIL GIL;
		PyCallable.Reset();
	}
	else
	{
		// Release ownership if Python has been shut down to avoid attempting to delete the object (which is already dead)
		PyCallable.Release();
	}
}

PyObject* UPythonCallableForDelegate::GetCallable() const
{
	return (PyObject*)PyCallable.GetPtr();
}

void UPythonCallableForDelegate::SetCallable(PyObject* InCallable)
{
	FPyScopedGIL GIL;
	PyCallable = FPyObjectPtr::NewReference(InCallable);
}
#endif	// WITH_PYTHON

#if WITH_PYTHON

void InitializePyWrapperDelegate(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperDelegateType) == 0)
	{
		static FPyWrapperDelegateMetaData MetaData;
		FPyWrapperDelegateMetaData::SetMetaData(&PyWrapperDelegateType, &MetaData);
		ModuleInfo.AddType(&PyWrapperDelegateType);
	}

	if (PyType_Ready(&PyWrapperMulticastDelegateType) == 0)
	{
		static FPyWrapperMulticastDelegateMetaData MetaData;
		FPyWrapperMulticastDelegateMetaData::SetMetaData(&PyWrapperMulticastDelegateType, &MetaData);
		ModuleInfo.AddType(&PyWrapperMulticastDelegateType);
	}
}


namespace PyDelegateUtil
{

/**
 * Tests for callable equality
 * ob_type->tp_call is the function pointer that will be called, so they need to be the same.
 * If the callable is a method, check that the self parameter is also the same.
 * 
 * Lhs & Rhs need to be callables!
 */
bool AreCallablesEqual(PyObject* Lhs, PyObject* Rhs)
{
	check(PyCallable_Check(Lhs) && PyCallable_Check(Rhs));

	bool bAreCallablesEqual = (Lhs->ob_type == Rhs->ob_type); // Must be of the same type
	bAreCallablesEqual = bAreCallablesEqual && (Lhs->ob_type->tp_call == Rhs->ob_type->tp_call); // Must be calling the same function/method

	if (bAreCallablesEqual && PyMethod_Check(Lhs))
	{
		bAreCallablesEqual = (PyMethod_Self(Lhs) == PyMethod_Self(Rhs)); // Must have the same self
	}

	return bAreCallablesEqual;
}

bool PythonArgsToDelegate_ObjectAndName(PyObject* InArgs, const PyGenUtil::FGeneratedWrappedFunction& InDelegateSignature, FScriptDelegate& OutDelegate, const TCHAR* InFuncCtxt, const TCHAR* InErrorCtxt)
{
	PyObject* PyObj = nullptr;
	PyObject* PyFuncNameObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, TCHAR_TO_UTF8(*FString::Printf(TEXT("OO:%s"), InFuncCtxt)), &PyObj, &PyFuncNameObj))
	{
		return false;
	}

	UObject* Obj = nullptr;
	if (!PyConversion::Nativize(PyObj, Obj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert argument 0 (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyObj)));
		return false;
	}

	FName FuncName;
	if (!PyConversion::Nativize(PyFuncNameObj, FuncName))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert argument 1 (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyFuncNameObj)));
		return false;
	}

	if (Obj)
	{
		check(PyObj);

		// Is the function name we've been given a Python alias? If so, we need to resolve that now
		FuncName = FPyWrapperObjectMetaData::ResolveFunctionName(Py_TYPE(PyObj), FuncName);
		const UFunction* Func = Obj->FindFunction(FuncName);

		// Valid signature?
		if (Func && !InDelegateSignature.Func->IsSignatureCompatibleWith(Func))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Function '%s' on '%s' does not match the signature required by the delegate '%s'"), *Func->GetName(), *Obj->GetName(), *InDelegateSignature.Func->GetName()));
			return false;
		}
	}

	OutDelegate.BindUFunction(Obj, FuncName);
	return true;
}

bool PythonArgsToPythonCallable(PyObject* InArgs, PyObject*& OutPyCallable, const TCHAR* InFuncCtxt, const TCHAR* InErrorCtxt)
{
	PyObject* PyObj = nullptr;
	if (!PyArg_ParseTuple(InArgs, TCHAR_TO_UTF8(*FString::Printf(TEXT("O:%s"), InFuncCtxt)), &PyObj))
	{
		return false;
	}

	if (!PyCallable_Check(PyObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Given argument is of type '%s' which isn't callable"), *PyUtil::GetFriendlyTypename(PyObj)));
		return false;
	}

	OutPyCallable = PyObj;
	return true;
}

bool PythonCallableToDelegate(PyObject* InPyCallable, const PyGenUtil::FGeneratedWrappedFunction& InDelegateSignature, const UClass* InPythonCallableForDelegateClass, FScriptDelegate& OutDelegate, const TCHAR* InErrorCtxt)
{
	if (!InPythonCallableForDelegateClass)
	{
		PyUtil::SetPythonError(PyExc_Exception, InErrorCtxt, TEXT("Delegate wrapper proxy class is null! Cannot create binding"));
		return false;
	}

	// Inspect the arguments from the Python callable
	// If this fails, don't error as it may be a C++ wrapped function we were passed (and inspect doesn't work with those)
	TArray<FString> CallableArgNames;
	if (PyUtil::InspectFunctionArgs(InPyCallable, CallableArgNames))
	{
		// If the callable is a method with a bound "self", remove the first argument
		const bool bHasSelf = PyMethod_Check(InPyCallable) && PyMethod_GET_SELF(InPyCallable);
		if (bHasSelf && CallableArgNames.Num() > 0)
		{
			CallableArgNames.RemoveAt(0, EAllowShrinking::No);
		}

		if (InDelegateSignature.InputParams.Num() != CallableArgNames.Num())
		{
			PyUtil::SetPythonError(PyExc_Exception, InErrorCtxt, *FString::Printf(TEXT("Callable has the incorrect number of arguments (expected %d, got %d)"), InDelegateSignature.InputParams.Num(), CallableArgNames.Num()));
			return false;
		}
	}

	// Note: -----------------------------------------------------------------------------------------------------------------------------------------------------------
	//  Delegates only hold a weak reference to the object. Wrapped delegates will attempt to keep the proxy object alive as long as it is referenced in Python, 
	//  but once Python is no longer referencing it, there is no guarantee that the proxy won't be GC'd unless C++ code explicitly keeps the delegate object alive.
	//  This is a known and accepted state of delegates as they currently stand. In the future we may revisit this and attempt to improve the lifetime management.
	UPythonCallableForDelegate* PythonCallableForDelegate = NewObject<UPythonCallableForDelegate>(GetTransientPackage(), (UClass*)InPythonCallableForDelegateClass);
	PythonCallableForDelegate->SetCallable(InPyCallable);
	OutDelegate.BindUFunction(PythonCallableForDelegate, UPythonCallableForDelegate::GeneratedFuncName);
	return true;
}

bool PythonArgsToDelegate_Callable(PyObject* InArgs, const PyGenUtil::FGeneratedWrappedFunction& InDelegateSignature, const UClass* InPythonCallableForDelegateClass, FScriptDelegate& OutDelegate, const TCHAR* InFuncCtxt, const TCHAR* InErrorCtxt)
{
	PyObject* PyCallable = nullptr;
	if (!PythonArgsToPythonCallable(InArgs, PyCallable, InFuncCtxt, InErrorCtxt))
	{
		return false;
	}
	return PythonCallableToDelegate(PyCallable, InDelegateSignature, InPythonCallableForDelegateClass, OutDelegate, InErrorCtxt);
}

} // namespace PyDelegateUtil


template <typename DelegateType>
struct TPyDelegateInvocation
{
};

template <>
struct TPyDelegateInvocation<FScriptDelegate>
{
	static bool CanCall(const FScriptDelegate& Delegate)
	{
		return Delegate.IsBound();
	}

	static void Call(const FScriptDelegate& Delegate, void* Params)
	{
		Delegate.ProcessDelegate<UObject>(Params);
	}
};

template <>
struct TPyDelegateInvocation<FMulticastScriptDelegate>
{
	static bool CanCall(const FMulticastScriptDelegate& Delegate)
	{
		return true;
	}

	static void Call(const FMulticastScriptDelegate& Delegate, void* Params)
	{
		Delegate.ProcessDelegate<UObject>(Params);
	}
};


template <typename WrapperType, typename MetaDataType, typename DelegateType, typename FactoryType>
struct TPyWrapperDelegateImpl
{
	static WrapperType* New(PyTypeObject* InType)
	{
		WrapperType* Self = (WrapperType*)FPyWrapperBase::New(InType);
		if (Self)
		{
			new(&Self->OwnerContext) FPyWrapperOwnerContext();
			Self->DelegateInstance = nullptr;
			new(&Self->InternalDelegateInstance) DelegateType();
		}
		return Self;
	}

	static void Free(WrapperType* InSelf)
	{
		Deinit(InSelf);

		InSelf->OwnerContext.~FPyWrapperOwnerContext();
		InSelf->InternalDelegateInstance.~DelegateType();
		FPyWrapperBase::Free(InSelf);
	}

	static int Init(WrapperType* InSelf)
	{
		Deinit(InSelf);

		const int BaseInit = FPyWrapperBase::Init(InSelf);
		if (BaseInit != 0)
		{
			return BaseInit;
		}

		InSelf->DelegateInstance = &InSelf->InternalDelegateInstance;

		FactoryType::Get().MapInstance(InSelf->DelegateInstance, InSelf);
		return 0;
	}

	static int Init(WrapperType* InSelf, const FPyWrapperOwnerContext& InOwnerContext, DelegateType* InValue, const EPyConversionMethod InConversionMethod)
	{
		InOwnerContext.AssertValidConversionMethod(InConversionMethod);

		Deinit(InSelf);

		const int BaseInit = FPyWrapperBase::Init(InSelf);
		if (BaseInit != 0)
		{
			return BaseInit;
		}

		check(InValue);

		DelegateType* DelegateInstanceToUse = &InSelf->InternalDelegateInstance;
		switch (InConversionMethod)
		{
		case EPyConversionMethod::Copy:
		case EPyConversionMethod::Steal:
			InSelf->InternalDelegateInstance = *InValue;
			break;

		case EPyConversionMethod::Reference:
			DelegateInstanceToUse = InValue;
			break;

		default:
			checkf(false, TEXT("Unknown EPyConversionMethod"));
			break;
		}

		check(DelegateInstanceToUse);

		InSelf->OwnerContext = InOwnerContext;
		InSelf->DelegateInstance = DelegateInstanceToUse;

		FactoryType::Get().MapInstance(InSelf->DelegateInstance, InSelf);
		return 0;
	}

	static void Deinit(WrapperType* InSelf)
	{
		if (InSelf->DelegateInstance)
		{
			FactoryType::Get().UnmapInstance(InSelf->DelegateInstance, Py_TYPE(InSelf));
		}

		if (InSelf->OwnerContext.HasOwner())
		{
			InSelf->OwnerContext.Reset();
		}

		InSelf->DelegateInstance = nullptr;
		InSelf->InternalDelegateInstance.Clear();
	}

	static bool ValidateInternalState(WrapperType* InSelf)
	{
		if (!InSelf->DelegateInstance)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - DelegateInstance is null!"));
			return false;
		}

		return true;
	}

	static PyObject* CallDelegate(WrapperType* InSelf, PyObject* InArgs)
	{
		typedef TPyDelegateInvocation<DelegateType> FDelegateInvocation;

		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		if (!FDelegateInvocation::CanCall(*InSelf->DelegateInstance))
		{
			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Cannot call an unbound delegate"));
			return nullptr;
		}

		const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = MetaDataType::GetDelegateSignature(InSelf);

		if (DelegateSignature.Func->ChildProperties == nullptr)
		{
			// Simple case, no parameters or return value
			FDelegateInvocation::Call(*InSelf->DelegateInstance, nullptr);
			Py_RETURN_NONE;
		}

		// Complex case, parameters or return value
		TArray<PyObject*> Params;
		if (!PyGenUtil::ParseMethodParameters(InArgs, nullptr, DelegateSignature.InputParams, "delegate", Params))
		{
			return nullptr;
		}

		PY_UFUNCTION_STACK(DelegateParams, DelegateSignature.Func);
		PyGenUtil::ApplyParamDefaults(DelegateParams.GetMemory(), DelegateSignature.InputParams);
		for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ++ParamIndex)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ParamDef = DelegateSignature.InputParams[ParamIndex];

			PyObject* PyValue = Params[ParamIndex];
			if (PyValue)
			{
				if (!PyConversion::NativizeProperty_InContainer(PyValue, ParamDef.ParamProp, DelegateParams.GetMemory(), 0))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert parameter '%s' when calling delegate"), UTF8_TO_TCHAR(ParamDef.ParamName.GetData())));
					return nullptr;
				}
			}
		}
		FDelegateInvocation::Call(*InSelf->DelegateInstance, DelegateParams.GetMemory());
		return PyGenUtil::PackReturnValues(DelegateParams.GetMemory(), DelegateSignature.OutputParams, *PyUtil::GetErrorContext(InSelf), TEXT("delegate"));
	}
};

typedef TPyWrapperDelegateImpl<FPyWrapperDelegate, FPyWrapperDelegateMetaData, FScriptDelegate, FPyWrapperDelegateFactory> FPyWrapperDelegateImpl;
typedef TPyWrapperDelegateImpl<FPyWrapperMulticastDelegate, FPyWrapperMulticastDelegateMetaData, FMulticastScriptDelegate, FPyWrapperMulticastDelegateFactory> FPyWrapperMulticastDelegateImpl;


FPyWrapperDelegate* FPyWrapperDelegate::New(PyTypeObject* InType)
{
	return FPyWrapperDelegateImpl::New(InType);
}

void FPyWrapperDelegate::Free(FPyWrapperDelegate* InSelf)
{
	FPyWrapperDelegateImpl::Free(InSelf);
}

int FPyWrapperDelegate::Init(FPyWrapperDelegate* InSelf)
{
	return FPyWrapperDelegateImpl::Init(InSelf);
}

int FPyWrapperDelegate::Init(FPyWrapperDelegate* InSelf, const FPyWrapperOwnerContext& InOwnerContext, FScriptDelegate* InValue, const EPyConversionMethod InConversionMethod)
{
	return FPyWrapperDelegateImpl::Init(InSelf, InOwnerContext, InValue, InConversionMethod);
}

void FPyWrapperDelegate::Deinit(FPyWrapperDelegate* InSelf)
{
	FPyWrapperDelegateImpl::Deinit(InSelf);
}

bool FPyWrapperDelegate::ValidateInternalState(FPyWrapperDelegate* InSelf)
{
	return FPyWrapperDelegateImpl::ValidateInternalState(InSelf);
}

FPyWrapperDelegate* FPyWrapperDelegate::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperDelegateType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperDelegate*)InPyObject;
	}

	return nullptr;
}

FPyWrapperDelegate* FPyWrapperDelegate::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperDelegateType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperDelegateType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperDelegate*)InPyObject;
	}

	// Note: -----------------------------------------------------------------------------------------------------------------------------------------------------------
	//  We currently don't allow coercion from a Python callable here as the lifetime rules of delegate proxies mean we want people to make that choice explicitly

	return nullptr;
}

PyObject* FPyWrapperDelegate::CallDelegate(FPyWrapperDelegate* InSelf, PyObject* InArgs)
{
	return FPyWrapperDelegateImpl::CallDelegate(InSelf, InArgs);
}


FPyWrapperMulticastDelegate* FPyWrapperMulticastDelegate::New(PyTypeObject* InType)
{
	return FPyWrapperMulticastDelegateImpl::New(InType);
}

void FPyWrapperMulticastDelegate::Free(FPyWrapperMulticastDelegate* InSelf)
{
	FPyWrapperMulticastDelegateImpl::Free(InSelf);
}

int FPyWrapperMulticastDelegate::Init(FPyWrapperMulticastDelegate* InSelf)
{
	return FPyWrapperMulticastDelegateImpl::Init(InSelf);
}

int FPyWrapperMulticastDelegate::Init(FPyWrapperMulticastDelegate* InSelf, const FPyWrapperOwnerContext& InOwnerContext, FMulticastScriptDelegate* InValue, const EPyConversionMethod InConversionMethod)
{
	return FPyWrapperMulticastDelegateImpl::Init(InSelf, InOwnerContext, InValue, InConversionMethod);
}

void FPyWrapperMulticastDelegate::Deinit(FPyWrapperMulticastDelegate* InSelf)
{
	FPyWrapperMulticastDelegateImpl::Deinit(InSelf);
}

bool FPyWrapperMulticastDelegate::ValidateInternalState(FPyWrapperMulticastDelegate* InSelf)
{
	return FPyWrapperMulticastDelegateImpl::ValidateInternalState(InSelf);
}

FPyWrapperMulticastDelegate* FPyWrapperMulticastDelegate::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperMulticastDelegateType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperMulticastDelegate*)InPyObject;
	}

	return nullptr;
}

FPyWrapperMulticastDelegate* FPyWrapperMulticastDelegate::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperMulticastDelegateType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperMulticastDelegateType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperMulticastDelegate*)InPyObject;
	}

	return nullptr;
}

PyObject* FPyWrapperMulticastDelegate::CallDelegate(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
{
	return FPyWrapperMulticastDelegateImpl::CallDelegate(InSelf, InArgs);
}


PyTypeObject InitializePyWrapperDelegateType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperDelegate::New(InType);
		}

		static void Dealloc(FPyWrapperDelegate* InSelf)
		{
			FPyWrapperDelegate::Free(InSelf);
		}

		static int Init(FPyWrapperDelegate* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			const int BaseInit = FPyWrapperDelegate::Init(InSelf);
			if (BaseInit != 0)
			{
				return BaseInit;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperDelegateMetaData::GetDelegateSignature(InSelf);
			const UClass* PythonCallableForDelegateClass = FPyWrapperDelegateMetaData::GetPythonCallableForDelegateClass(InSelf);

			const int32 ArgsCount = PyTuple_Size(InArgs);
			if (ArgsCount == 1)
			{
				// One argument is assumed to be a callable
				if (!PyDelegateUtil::PythonArgsToDelegate_Callable(InArgs, DelegateSignature, PythonCallableForDelegateClass, *InSelf->DelegateInstance, TEXT("call"), *PyUtil::GetErrorContext(InSelf)))
				{
					return -1;
				}
			}
			else if (ArgsCount > 0)
			{
				// Anything else is assumed to be an object and name pair
				if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, *InSelf->DelegateInstance, TEXT("call"), *PyUtil::GetErrorContext(InSelf)))
			{
				return -1;
			}
			}

			return 0;
		}

		static PyObject* Str(FPyWrapperDelegate* InSelf)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyUnicode_FromFormat("<Delegate '%s' (%p) %s>", TCHAR_TO_UTF8(*PyUtil::GetFriendlyTypename(InSelf)), InSelf->DelegateInstance, TCHAR_TO_UTF8(*InSelf->DelegateInstance->ToString<UObject>()));
		}

		static PyObject* Call(FPyWrapperDelegate* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (InKwds && PyDict_Size(InKwds) != 0)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Cannot call a delegate with keyword arguments"));
				return nullptr;
			}

			return FPyWrapperDelegate::CallDelegate(InSelf, InArgs);
		}
	};

	struct FNumberFuncs
	{
		static int Bool(FPyWrapperDelegate* InSelf)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return -1;
			}

			return InSelf->DelegateInstance->IsBound() ? 1 : 0;
		}
	};

	struct FMethods
	{
		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:cast", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperDelegate::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperDelegate* InSelf)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperDelegateMetaData::GetDelegateSignature(InSelf);
			return (PyObject*)FPyWrapperDelegateFactory::Get().CreateInstance(DelegateSignature.Func, InSelf->DelegateInstance, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}

		static PyObject* IsBound(FPyWrapperDelegate* InSelf)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			if (InSelf->DelegateInstance->IsBound())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* BindDelegate(FPyWrapperDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:bind_delegate", &PyObj))
			{
				return nullptr;
			}

			FPyWrapperDelegate* PyOtherDelegate = FPyWrapperDelegate::CastPyObject(PyObj, Py_TYPE(InSelf));
			if (!PyOtherDelegate)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert argument 0 (%s) to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InSelf)));
				return nullptr;
			}

			*InSelf->DelegateInstance = *PyOtherDelegate->DelegateInstance;

			Py_RETURN_NONE;
		}

		static PyObject* BindFunction(FPyWrapperDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperDelegateMetaData::GetDelegateSignature(InSelf);
			if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, *InSelf->DelegateInstance, TEXT("bind_function"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* BindCallable(FPyWrapperDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperDelegateMetaData::GetDelegateSignature(InSelf);
			const UClass* PythonCallableForDelegateClass = FPyWrapperDelegateMetaData::GetPythonCallableForDelegateClass(InSelf);
			if (!PyDelegateUtil::PythonArgsToDelegate_Callable(InArgs, DelegateSignature, PythonCallableForDelegateClass, *InSelf->DelegateInstance, TEXT("bind_callable"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Unbind(FPyWrapperDelegate* InSelf)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->Unbind();

			Py_RETURN_NONE;
		}

		static PyObject* Execute(FPyWrapperDelegate* InSelf, PyObject* InArgs)
		{
			return FPyWrapperDelegate::CallDelegate(InSelf, InArgs);
		}

		static PyObject* ExecuteIfBound(FPyWrapperDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			if (InSelf->DelegateInstance->IsBound())
			{
				return FPyWrapperDelegate::CallDelegate(InSelf, InArgs);
			}

			Py_RETURN_NONE;
		}
	};

	// NOTE: The delegate couldn't be strictly type-hinted. The exact return/param types of the delegate are unknown here. In Python 3.11, we should be able
	//       to use typing.Self for return type of __copy__() and copy() methods.
	//       _T = typing.TypeVar('_T') and Type/Any/Callable/Union are defines by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls: Type[_T], object: object) -> _T -- cast the given object to this Unreal delegate type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> Any -- copy this Unreal delegate" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> Any -- copy this Unreal delegate" },
		{ "is_bound", PyCFunctionCast(&FMethods::IsBound), METH_NOARGS, "is_bound(self) -> bool -- is this Unreal delegate bound to something?" },
		{ "bind_delegate", PyCFunctionCast(&FMethods::BindDelegate), METH_VARARGS, "bind_delegate(self, delegate: DelegateBase) -> None -- bind this Unreal delegate to the same object and function as another delegate" },
		{ "bind_function", PyCFunctionCast(&FMethods::BindFunction), METH_VARARGS, "bind_function(self, obj: Object, name: Union[Name, str]) -> None -- bind this Unreal delegate to a named Unreal function on the given object instance" },
		{ "bind_callable", PyCFunctionCast(&FMethods::BindCallable), METH_VARARGS, "bind_callable(self, callable: Callable[..., Any]) -> None -- bind this Unreal delegate to a Python callable" },
		{ "unbind", PyCFunctionCast(&FMethods::Unbind), METH_NOARGS, "unbind(self) -> None -- unbind this Unreal delegate" },
		{ "execute", PyCFunctionCast(&FMethods::Execute), METH_VARARGS, "execute(self, *args: Any) -> Any -- call this Unreal delegate, but error if it's unbound" },
		{ "execute_if_bound", PyCFunctionCast(&FMethods::ExecuteIfBound), METH_VARARGS, "x.execute_if_bound(self, *args: Any) -> Any -- call this Unreal delegate, but only if it's bound to something" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"DelegateBase", /* tp_name */
		sizeof(FPyWrapperDelegate), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_call = (ternaryfunc)&FFuncs::Call;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed delegate instances";

	static PyNumberMethods PyNumber;
	PyNumber.nb_bool = (inquiry)&FNumberFuncs::Bool;
	PyType.tp_as_number = &PyNumber;

	return PyType;
}

PyTypeObject InitializePyWrapperMulticastDelegateType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperMulticastDelegate::New(InType);
		}

		static void Dealloc(FPyWrapperMulticastDelegate* InSelf)
		{
			FPyWrapperMulticastDelegate::Free(InSelf);
		}

		static int Init(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			const int BaseInit = FPyWrapperMulticastDelegate::Init(InSelf);
			if (BaseInit != 0)
			{
				return BaseInit;
			}

			if (PyTuple_Size(InArgs) > 0)
			{
				const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);

				FScriptDelegate Delegate;
				if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, Delegate, TEXT("call"), *PyUtil::GetErrorContext(InSelf)))
				{
					return -1;
				}
				InSelf->DelegateInstance->Add(Delegate);
			}

			return 0;
		}

		static PyObject* Str(FPyWrapperMulticastDelegate* InSelf)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyUnicode_FromFormat("<Multicast delegate '%s' (%p) %s>", TCHAR_TO_UTF8(*PyUtil::GetFriendlyTypename(InSelf)), InSelf->DelegateInstance, TCHAR_TO_UTF8(*InSelf->DelegateInstance->ToString<UObject>()));
		}

		static PyObject* Call(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (InKwds && PyDict_Size(InKwds) != 0)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Cannot call a delegate with keyword arguments"));
				return nullptr;
			}

			return FPyWrapperMulticastDelegate::CallDelegate(InSelf, InArgs);
		}
	};

	struct FNumberFuncs
	{
		static int Bool(FPyWrapperMulticastDelegate* InSelf)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return -1;
			}

			return InSelf->DelegateInstance->IsBound() ? 1 : 0;
		}
	};

	struct FMethods
	{
		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:cast", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperMulticastDelegate::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperMulticastDelegate* InSelf)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);
			return (PyObject*)FPyWrapperMulticastDelegateFactory::Get().CreateInstance(DelegateSignature.Func, InSelf->DelegateInstance, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}

		static PyObject* IsBound(FPyWrapperMulticastDelegate* InSelf)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			if (InSelf->DelegateInstance->IsBound())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* AddFunction(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);

			FScriptDelegate Delegate;
			if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, Delegate, TEXT("add_function"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->Add(Delegate);

			Py_RETURN_NONE;
		}

		static PyObject* AddCallable(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);
			const UClass* PythonCallableForDelegateClass = FPyWrapperMulticastDelegateMetaData::GetPythonCallableForDelegateClass(InSelf);

			FScriptDelegate Delegate;
			if (!PyDelegateUtil::PythonArgsToDelegate_Callable(InArgs, DelegateSignature, PythonCallableForDelegateClass, Delegate, TEXT("add_callable"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->Add(Delegate);

			Py_RETURN_NONE;
		}

		static PyObject* AddFunctionUnique(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);

			FScriptDelegate Delegate;
			if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, Delegate, TEXT("add_function_unique"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->AddUnique(Delegate);

			Py_RETURN_NONE;
		}

		static PyObject* AddCallableUnique(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			// We need to search for an entry using the current callable rather than use the normal AddUnique function, 
			// as that only checks the object and function name and each Python callable proxy is its own instance
			PyObject* PyCallable = nullptr;
			if (!PyDelegateUtil::PythonArgsToPythonCallable(InArgs, PyCallable, TEXT("add_callable_unique"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			bool bAddDelegate = true;
			for (const UObject* DelegateObj : InSelf->DelegateInstance->GetAllObjects())
			{
				if (const UPythonCallableForDelegate* PythonCallableForDelegate = ::Cast<UPythonCallableForDelegate>(DelegateObj))
				{
					if (PyDelegateUtil::AreCallablesEqual(PythonCallableForDelegate->GetCallable(), PyCallable))
					{
						bAddDelegate = false;
						break;
					}
				}
			}

			if (bAddDelegate)
			{
				const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);
				const UClass* PythonCallableForDelegateClass = FPyWrapperMulticastDelegateMetaData::GetPythonCallableForDelegateClass(InSelf);

				FScriptDelegate Delegate;
				if (!PyDelegateUtil::PythonCallableToDelegate(PyCallable, DelegateSignature, PythonCallableForDelegateClass, Delegate, *PyUtil::GetErrorContext(InSelf)))
				{
					return nullptr;
				}

				InSelf->DelegateInstance->Add(Delegate);
			}

			Py_RETURN_NONE;
		}

		static PyObject* RemoveFunction(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);

			FScriptDelegate Delegate;
			if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, Delegate, TEXT("remove_function"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->Remove(Delegate);

			Py_RETURN_NONE;
		}

		static PyObject* RemoveCallable(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			// We need to search for an entry using the current callable rather than use the normal Remove function, 
			// as that only checks the object and function name and each Python callable proxy is its own instance
			PyObject* PyCallable = nullptr;
			if (!PyDelegateUtil::PythonArgsToPythonCallable(InArgs, PyCallable, TEXT("remove_callable"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			UPythonCallableForDelegate* PythonCallableForDelegateToRemove = nullptr;
			for (UObject* DelegateObj : InSelf->DelegateInstance->GetAllObjects())
			{
				if (UPythonCallableForDelegate* PythonCallableForDelegate = ::Cast<UPythonCallableForDelegate>(DelegateObj))
				{
					if (PyDelegateUtil::AreCallablesEqual(PythonCallableForDelegate->GetCallable(), PyCallable))
					{
						PythonCallableForDelegateToRemove = PythonCallableForDelegate;
						break;
					}
				}
			}

			if (PythonCallableForDelegateToRemove)
			{
				InSelf->DelegateInstance->RemoveAll(PythonCallableForDelegateToRemove);
			}

			Py_RETURN_NONE;
		}

		static PyObject* RemoveObject(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:remove_object", &PyObj))
			{
				return nullptr;
			}

			UObject* Obj = nullptr;
			if (!PyConversion::Nativize(PyObj, Obj))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->RemoveAll(Obj);

			Py_RETURN_NONE;
		}

		static PyObject* ContainsFunction(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedFunction& DelegateSignature = FPyWrapperMulticastDelegateMetaData::GetDelegateSignature(InSelf);

			FScriptDelegate Delegate;
			if (!PyDelegateUtil::PythonArgsToDelegate_ObjectAndName(InArgs, DelegateSignature, Delegate, TEXT("contains_function"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			if (InSelf->DelegateInstance->Contains(Delegate))
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* ContainsCallable(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			// We need to search for an entry using the current callable rather than use the normal Contains function, 
			// as that only checks the object and function name and each Python callable proxy is its own instance
			PyObject* PyCallable = nullptr;
			if (!PyDelegateUtil::PythonArgsToPythonCallable(InArgs, PyCallable, TEXT("contains_callable"), *PyUtil::GetErrorContext(InSelf)))
			{
				return nullptr;
			}

			bool bContainsCallable = false;
			for (const UObject* DelegateObj : InSelf->DelegateInstance->GetAllObjects())
			{
				if (const UPythonCallableForDelegate* PythonCallableForDelegate = ::Cast<UPythonCallableForDelegate>(DelegateObj))
				{
					if (PyDelegateUtil::AreCallablesEqual(PythonCallableForDelegate->GetCallable(), PyCallable))
					{
						bContainsCallable = true;
						break;
					}
				}
			}

			if (bContainsCallable)
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* Clear(FPyWrapperMulticastDelegate* InSelf)
		{
			if (!FPyWrapperMulticastDelegate::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			InSelf->DelegateInstance->Clear();

			Py_RETURN_NONE;
		}

		static PyObject* Broadcast(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs)
		{
			return FPyWrapperMulticastDelegate::CallDelegate(InSelf, InArgs);
		}
	};
	// NOTE: The multicast delegate couldn't be strictly type-hinted. The exact param types of the delegate are unknown here. In Python 3.11, we should be able
	//       to use typing.Self for return type of __copy__() and copy() methods.
	//       _T = typing.TypeVar('_T') and Type/Any/Callable/Union are defines by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls: Type[_T], object: object) -> _T -- cast the given object to this Unreal delegate type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> Any -- copy this Unreal delegate" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> Any -- copy this Unreal delegate" },
		{ "is_bound", PyCFunctionCast(&FMethods::IsBound), METH_NOARGS, "is_bound(self) -> bool -- is this Unreal delegate bound to something?" },
		{ "add_function", PyCFunctionCast(&FMethods::AddFunction), METH_VARARGS, "add_function(self, obj: Object, name: Union[Name, str]) -> None -- add a binding to a named Unreal function on the given object instance to the invocation list of this Unreal delegate" },
		{ "add_callable", PyCFunctionCast(&FMethods::AddCallable), METH_VARARGS, "add_callable(self, callable: Callable[..., Any]) -> None -- add a binding to a Python callable to the invocation list of this Unreal delegate" },
		{ "add_function_unique", PyCFunctionCast(&FMethods::AddFunctionUnique), METH_VARARGS, "add_function_unique(self, obj: Object, name: Union[Name, str]) -> None -- add a unique binding to a named Unreal function on the given object instance to the invocation list of this Unreal delegate" },
		{ "add_callable_unique", PyCFunctionCast(&FMethods::AddCallableUnique), METH_VARARGS, "add_callable_unique(self, callable: Callable[..., Any]) -> None -- add a unique binding to a Python callable to the invocation list of this Unreal delegate" },
		{ "remove_function", PyCFunctionCast(&FMethods::RemoveFunction), METH_VARARGS, "remove_function(self, obj: Object, name: Union[Name, str]) -> None -- remove a binding to a named Unreal function on the given object instance from the invocation list of this Unreal delegate" },
		{ "remove_callable", PyCFunctionCast(&FMethods::RemoveCallable), METH_VARARGS, "remove_callable(self, callable: Callable[..., Any]) -> None -- remove a binding to a Python callable from the invocation list of this Unreal delegate" },
		{ "remove_object", PyCFunctionCast(&FMethods::RemoveObject), METH_VARARGS, "remove_object(self, obj: Object) -> None -- remove all bindings for the given object instance from the invocation list of this Unreal delegate" },
		{ "contains_function", PyCFunctionCast(&FMethods::ContainsFunction), METH_VARARGS, "contains_function(self, obj: Object, name: Union[Name, str]) -> bool -- does the invocation list of this Unreal delegate contain a binding to a named Unreal function on the given object instance" },
		{ "contains_callable", PyCFunctionCast(&FMethods::ContainsCallable), METH_VARARGS, "contains_callable(self, callable: Callable[..., Any]) -> bool -- does the invocation list of this Unreal delegate contain a binding to a Python callable" },
		{ "clear", PyCFunctionCast(&FMethods::Clear), METH_NOARGS, "x.clear() -> None -- clear the invocation list of this Unreal delegate" },
		{ "broadcast", PyCFunctionCast(&FMethods::Broadcast), METH_VARARGS, "x.broadcast(*args: Any) -> None -- invoke this Unreal multicast delegate" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"MulticastDelegateBase", /* tp_name */
		sizeof(FPyWrapperMulticastDelegate), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_call = (ternaryfunc)&FFuncs::Call;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed multicast delegate instances";

	static PyNumberMethods PyNumber;
	PyNumber.nb_bool = (inquiry)&FNumberFuncs::Bool;
	PyType.tp_as_number = &PyNumber;

	return PyType;
}

PyTypeObject PyWrapperDelegateType = InitializePyWrapperDelegateType();
PyTypeObject PyWrapperMulticastDelegateType = InitializePyWrapperMulticastDelegateType();

void FPyWrapperDelegateMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperDelegate* Self = static_cast<FPyWrapperDelegate*>(Instance);
	if (Self->DelegateInstance)
	{
		FPyReferenceCollector::AddReferencedObjectsFromDelegate(Collector, *Self->DelegateInstance);
	}
}

void FPyWrapperMulticastDelegateMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperMulticastDelegate* Self = static_cast<FPyWrapperMulticastDelegate*>(Instance);
	if (Self->DelegateInstance)
	{
		FPyReferenceCollector::AddReferencedObjectsFromMulticastDelegate(Collector, *Self->DelegateInstance);
	}
}

#endif	// WITH_PYTHON

