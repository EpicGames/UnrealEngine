// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "PyPtr.h"
#include "PyWrapperOwnerContext.h"
#include "UObject/WeakObjectPtr.h"
#include "PyWrapperDelegate.generated.h"

/**
 * UObject proxy base used to wrap a callable Python object so that it can be used with an Unreal delegate
 * @note This can't go inside the WITH_PYTHON block due to UHT parsing limitations (it doesn't understand that macro)
 */
UCLASS(HideDropDown)
class UPythonCallableForDelegate : public UObject, public IPythonResourceOwner
{
	GENERATED_BODY()

public:
	/** Name given to the generated function that we should bind to the Unreal delegate */
	static const FName GeneratedFuncName;

	/** Native function implementation used by the signature correct Unreal functions added to the derived classes (the ones that are bound to the delegate itself) */
	DECLARE_FUNCTION(CallPythonNative);

#if WITH_PYTHON

public:
	//~ UObject interface
	virtual void BeginDestroy() override;

	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override;

	/** Get the Python callable object on this instance (borrowed reference) */
	PyObject* GetCallable() const;

	/** Set the Python callable object on this instance */
	void SetCallable(PyObject* InCallable);

private:
	/** The callable Python callable object this object wraps (if any) */
	FPyObjectPtr PyCallable;

#else	// WITH_PYTHON

public:
	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override {}

#endif	// WITH_PYTHON
};

#if WITH_PYTHON

/** Python type for FPyWrapperDelegate */
extern PyTypeObject PyWrapperDelegateType;

/** Python type for FPyWrapperMulticastDelegate */
extern PyTypeObject PyWrapperMulticastDelegateType;

/** Initialize the PyWrapperDelegate types and add them to the given Python module */
void InitializePyWrapperDelegate(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Base type for all Unreal exposed delegate instances */
template <typename DelegateType>
struct TPyWrapperDelegate : public FPyWrapperBase
{
	/** The owner of the wrapped delegate instance (if any) */
	FPyWrapperOwnerContext OwnerContext;

	/** Wrapped delegate instance */
	DelegateType* DelegateInstance;

	/** Internal delegate instance (DelegateInstance is set to this when we own the instance) */
	DelegateType InternalDelegateInstance;
};

/** Base meta-data for all Unreal exposed delegate types */
template <typename WrapperType>
struct TPyWrapperDelegateMetaData : public FPyWrapperBaseMetaData
{
	PY_OVERRIDE_GETSET_METADATA(TPyWrapperDelegateMetaData)

	TPyWrapperDelegateMetaData()
		: PythonCallableForDelegateClass(nullptr)
	{
	}

	/** Get the delegate signature from the given type */
	static PyGenUtil::FGeneratedWrappedFunction& GetDelegateSignature(PyTypeObject* PyType)
	{
		TPyWrapperDelegateMetaData* PyWrapperMetaData = TPyWrapperDelegateMetaData::GetMetaData(PyType);
		static PyGenUtil::FGeneratedWrappedFunction NullDelegateSignature = PyGenUtil::FGeneratedWrappedFunction();
		return PyWrapperMetaData ? PyWrapperMetaData->DelegateSignature : NullDelegateSignature;
	}

	/** Get the delegate signature from the type of the given instance */
	static PyGenUtil::FGeneratedWrappedFunction& GetDelegateSignature(WrapperType* Instance)
	{
		return GetDelegateSignature(Py_TYPE(Instance));
	}

	/** Get the generated class type used to wrap Python callables for this delegate type */
	static const UClass* GetPythonCallableForDelegateClass(PyTypeObject* PyType)
	{
		TPyWrapperDelegateMetaData* PyWrapperMetaData = TPyWrapperDelegateMetaData::GetMetaData(PyType);
		return PyWrapperMetaData ? PyWrapperMetaData->PythonCallableForDelegateClass : nullptr;
	}

	/** Get the generated class type used to wrap Python callables for this delegate type */
	static const UClass* GetPythonCallableForDelegateClass(WrapperType* Instance)
	{
		return GetPythonCallableForDelegateClass(Py_TYPE(Instance));
	}

	/** Add object references from this type meta-data to the given collector */
	virtual void AddTypeReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(DelegateSignature.Func);
		Collector.AddReferencedObject(PythonCallableForDelegateClass);
	}

	/** Get the reflection meta data type object associated with this wrapper type if there is one or nullptr if not. */
	virtual const UField* GetMetaType() const override
	{
		return DelegateSignature.Func;
	}

	/** Unreal function representing the signature for the delegate */
	PyGenUtil::FGeneratedWrappedFunction DelegateSignature;

	/** Generated class type used to wrap Python callables for this delegate type */
	TObjectPtr<const UClass> PythonCallableForDelegateClass;
};

/** Type for all Unreal exposed delegate instances */
struct FPyWrapperDelegate : public TPyWrapperDelegate<FScriptDelegate>
{
	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperDelegate* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperDelegate* InSelf);

	/** Initialize this wrapper instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperDelegate* InSelf);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperDelegate* InSelf, const FPyWrapperOwnerContext& InOwnerContext, FScriptDelegate* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperDelegate* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperDelegate* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperDelegate* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperDelegate* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Call the delegate */
	static PyObject* CallDelegate(FPyWrapperDelegate* InSelf, PyObject* InArgs);
};

/** Meta-data for all Unreal exposed delegate types */
struct FPyWrapperDelegateMetaData : public TPyWrapperDelegateMetaData<FPyWrapperDelegate>
{
	PY_METADATA_METHODS(FPyWrapperDelegateMetaData, FGuid(0xCB3D0485, 0x8A3A443E, 0xBEE336F4, 0x82888A81))

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;
};

/** Type for all Unreal exposed multicast delegate instances */
struct FPyWrapperMulticastDelegate : public TPyWrapperDelegate<FMulticastScriptDelegate>
{
	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperMulticastDelegate* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperMulticastDelegate* InSelf);

	/** Initialize this wrapper instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperMulticastDelegate* InSelf);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperMulticastDelegate* InSelf, const FPyWrapperOwnerContext& InOwnerContext, FMulticastScriptDelegate* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperMulticastDelegate* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperMulticastDelegate* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperMulticastDelegate* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperMulticastDelegate* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Call the delegate */
	static PyObject* CallDelegate(FPyWrapperMulticastDelegate* InSelf, PyObject* InArgs);
};

/** Meta-data for all Unreal exposed multicast delegate types */
struct FPyWrapperMulticastDelegateMetaData : public TPyWrapperDelegateMetaData<FPyWrapperMulticastDelegate>
{
	PY_METADATA_METHODS(FPyWrapperMulticastDelegateMetaData, FGuid(0x448FB4DA, 0x38DC4386, 0xBCAFF448, 0x29C0F3A4))

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;
};

typedef TPyPtr<FPyWrapperDelegate> FPyWrapperDelegatePtr;
typedef TPyPtr<FPyWrapperMulticastDelegate> FPyWrapperMulticastDelegatePtr;

#endif	// WITH_PYTHON
