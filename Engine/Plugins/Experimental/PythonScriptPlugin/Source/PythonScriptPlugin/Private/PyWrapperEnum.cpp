// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperEnum.h"
#include "PyWrapperTypeRegistry.h"
#include "PyGIL.h"
#include "PyCore.h"
#include "PyUtil.h"
#include "PyConversion.h"

#include "Containers/ArrayView.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectRedirector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PyWrapperEnum)

#if WITH_PYTHON

typedef TArrayView<FPyWrapperEnum* const> FPyWrapperEnumArrayView;

/** Python metaclass type for FPyWrapperEnum */
extern PyTypeObject PyWrapperEnumMetaclassType;

/** Python type for FPyWrapperEnumIterator */
extern PyTypeObject PyWrapperEnumIteratorType;

/** Iterator used with enums */
struct FPyWrapperEnumIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Array being iterated over */
	FPyWrapperEnumArrayView IterArray;

	/** Current iteration index */
	int32 IterIndex;

	/** New this iterator instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperEnumIterator* New(PyTypeObject* InType)
	{
		FPyWrapperEnumIterator* Self = (FPyWrapperEnumIterator*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			new(&Self->IterArray) FPyWrapperEnumArrayView();
			Self->IterIndex = 0;
		}
		return Self;
	}

	/** Free this iterator instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperEnumIterator* InSelf)
	{
		Deinit(InSelf);

		InSelf->IterArray.~FPyWrapperEnumArrayView();
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this iterator instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperEnumIterator* InSelf, FPyWrapperEnumArrayView InIterArray)
	{
		Deinit(InSelf);

		InSelf->IterArray = InIterArray;
		InSelf->IterIndex = 0;

		return 0;
	}

	/** Deinitialize this iterator instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperEnumIterator* InSelf)
	{
		InSelf->IterArray = FPyWrapperEnumArrayView();
		InSelf->IterIndex = 0;
	}

	/** Get the iterator */
	static FPyWrapperEnumIterator* GetIter(FPyWrapperEnumIterator* InSelf)
	{
		Py_INCREF(InSelf);
		return InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(FPyWrapperEnumIterator* InSelf)
	{
		if (InSelf->IterArray.IsValidIndex(InSelf->IterIndex))
		{
			FPyWrapperEnum* EnumEntry = InSelf->IterArray[InSelf->IterIndex++];
			Py_INCREF(EnumEntry);
			return (PyObject*)EnumEntry;
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}
};

/** Python object for the descriptor of an enum value */
struct FPyWrapperEnumValueDescrObject
{
	/** Common Python Object */
	PyObject_HEAD

	/** The enum entry */
	FPyWrapperEnum* EnumEntry;

	/** The enum entry doc string (may be null) */
	PyObject* EnumEntryDoc;

	/** Set if this enum entry is deprecated and using it should emit a deprecation warning */
	TOptional<FString> DeprecationMessage;

	/** New an instance */
	static FPyWrapperEnumValueDescrObject* New(PyTypeObject* InEnumType, const int64 InEnumEntryValue, const char* InEnumEntryName, const char* InEnumEntryDoc)
	{
		FPyWrapperEnumValueDescrObject* Self = (FPyWrapperEnumValueDescrObject*)PyWrapperEnumValueDescrType.tp_alloc(&PyWrapperEnumValueDescrType, 0);
		if (Self)
		{
			Self->EnumEntry = FPyWrapperEnum::New(InEnumType);
			FPyWrapperEnum::Init(Self->EnumEntry, InEnumEntryValue, InEnumEntryName);
			Self->EnumEntryDoc = InEnumEntryDoc ? PyUnicode_FromString(InEnumEntryDoc) : nullptr;
			new(&Self->DeprecationMessage) TOptional<FString>();
		}
		return Self;
	}

	/** New a deprecated instance */
	static FPyWrapperEnumValueDescrObject* NewDeprecated(FPyWrapperEnum* InEnumEntry, const char* InEnumEntryName, const char* InEnumEntryDoc)
	{
		FPyWrapperEnumValueDescrObject* Self = (FPyWrapperEnumValueDescrObject*)PyWrapperEnumValueDescrType.tp_alloc(&PyWrapperEnumValueDescrType, 0);
		if (Self)
		{
			Py_XINCREF(InEnumEntry);
			Self->EnumEntry = InEnumEntry;
			Self->EnumEntryDoc = InEnumEntryDoc ? PyUnicode_FromString(InEnumEntryDoc) : nullptr;
			new(&Self->DeprecationMessage) TOptional<FString>(FString::Printf(TEXT("Enum entry '%s.%s' is deprecated: Use '%s.%s'"), 
				UTF8_TO_TCHAR(Py_TYPE(Self->EnumEntry)->tp_name), UTF8_TO_TCHAR(InEnumEntryName),
				UTF8_TO_TCHAR(Py_TYPE(Self->EnumEntry)->tp_name), *PyUtil::PyObjectToUEString(Self->EnumEntry->EntryName)
				));
		}
		return Self;
	}

	/** Free this instance */
	static void Free(FPyWrapperEnumValueDescrObject* InSelf)
	{
		Py_XDECREF(InSelf->EnumEntry);
		InSelf->EnumEntry = nullptr;

		Py_XDECREF(InSelf->EnumEntryDoc);
		InSelf->EnumEntryDoc = nullptr;

		InSelf->DeprecationMessage.~TOptional<FString>();

		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}
};

typedef TPyPtr<FPyWrapperEnumValueDescrObject> FPyWrapperEnumValueDescrObjectPtr;

void InitializePyWrapperEnum(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	PyType_Ready(&PyWrapperEnumIteratorType);
	PyType_Ready(&PyWrapperEnumMetaclassType);

	if (PyType_Ready(&PyWrapperEnumValueDescrType) == 0)
	{
		ModuleInfo.AddType(&PyWrapperEnumValueDescrType);
	}

	// Set the metaclass on the enum type
	Py_SET_TYPE(&PyWrapperEnumType, &PyWrapperEnumMetaclassType);
	if (PyType_Ready(&PyWrapperEnumType) == 0)
	{
		static FPyWrapperEnumMetaData MetaData;
		FPyWrapperEnumMetaData::SetMetaData(&PyWrapperEnumType, &MetaData);
		ModuleInfo.AddType(&PyWrapperEnumType);
	}
}

FPyWrapperEnum* FPyWrapperEnum::New(PyTypeObject* InType)
{
	FPyWrapperEnum* Self = (FPyWrapperEnum*)FPyWrapperBase::New(InType);
	if (Self)
	{
		Self->EntryName = nullptr;
		Self->EntryValue = nullptr;
	}
	return Self;
}

void FPyWrapperEnum::Free(FPyWrapperEnum* InSelf)
{
	Deinit(InSelf);

	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperEnum::Init(FPyWrapperEnum* InSelf)
{
	PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Cannot create instances of enum types"));
	return -1;
}

int FPyWrapperEnum::Init(FPyWrapperEnum* InSelf, const int64 InEnumEntryValue, const char* InEnumEntryName)
{
	if (FPyWrapperEnumMetaData::IsEnumFinalized(InSelf))
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Cannot create instances of enum types"));
		return -1;
	}

	InSelf->EntryName = PyUnicode_FromString(InEnumEntryName);
	InSelf->EntryValue = PyConversion::Pythonize(InEnumEntryValue);
	return 0;
}

void FPyWrapperEnum::Deinit(FPyWrapperEnum* InSelf)
{
	Py_XDECREF(InSelf->EntryName);
	InSelf->EntryName = nullptr;

	Py_XDECREF(InSelf->EntryValue);
	InSelf->EntryValue = nullptr;
}

bool FPyWrapperEnum::ValidateInternalState(FPyWrapperEnum* InSelf)
{
	if (!InSelf->EntryName)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - EntryName is null!"));
		return false;
	}

	if (!InSelf->EntryValue)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - EntryValue is null!"));
		return false;
	}

	return true;
}

FPyWrapperEnum* FPyWrapperEnum::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperEnumType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperEnum*)InPyObject;
	}

	return nullptr;
}

FPyWrapperEnum* FPyWrapperEnum::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperEnumType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperEnumType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperEnum*)InPyObject;
	}

	// Allow casting from a different enum type using the same UEnum (for deprecation)
	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperEnumType) == 1)
	{
		const UEnum* RequiredEnum = FPyWrapperEnumMetaData::GetEnum(InType);
		const UEnum* ActualEnum = FPyWrapperEnumMetaData::GetEnum(Py_TYPE(InPyObject));

		if (RequiredEnum == ActualEnum)
		{
			SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

			Py_INCREF(InPyObject);
			return (FPyWrapperEnum*)InPyObject;
		}
	}

	// Allow coerced casting from a numeric value
	if (const FPyWrapperEnumMetaData* EnumMetaData = FPyWrapperEnumMetaData::GetMetaData(InType))
	{
		int64 OtherVal = 0;
		if (PyConversion::Nativize(InPyObject, OtherVal, PyConversion::ESetErrorState::No))
		{
			// Find an enum entry using this value
			for (FPyWrapperEnum* PyEnumEntry : EnumMetaData->EnumEntries)
			{
				const int64 EnumEntryVal = FPyWrapperEnum::GetEnumEntryValue(PyEnumEntry);
				if (EnumEntryVal == OtherVal)
				{
					SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);

					Py_INCREF(PyEnumEntry);
					return (FPyWrapperEnum*)PyEnumEntry;
				}
			}
		}
	}

	return nullptr;
}

FString FPyWrapperEnum::GetEnumEntryName(FPyWrapperEnum* InSelf)
{
	FString EnumEntryNameStr;
	if (InSelf->EntryName)
	{
		PyConversion::Nativize(InSelf->EntryName, EnumEntryNameStr, PyConversion::ESetErrorState::No);
	}
	return EnumEntryNameStr;
}

int64 FPyWrapperEnum::GetEnumEntryValue(FPyWrapperEnum* InSelf)
{
	int64 EnumEntryValueNum = 0;
	if (InSelf->EntryValue)
	{
		PyConversion::Nativize(InSelf->EntryValue, EnumEntryValueNum, PyConversion::ESetErrorState::No);
	}
	return EnumEntryValueNum;
}

FPyWrapperEnum* FPyWrapperEnum::AddEnumEntry(PyTypeObject* InType, const int64 InEnumEntryValue, const char* InEnumEntryName, const char* InEnumEntryDoc)
{
	if (!FPyWrapperEnumMetaData::IsEnumFinalized(InType))
	{
		FPyWrapperEnumValueDescrObjectPtr PyEnumValueDescr = FPyWrapperEnumValueDescrObjectPtr::StealReference(FPyWrapperEnumValueDescrObject::New(InType, InEnumEntryValue, InEnumEntryName, InEnumEntryDoc));
		if (PyEnumValueDescr)
		{
			PyDict_SetItemString(InType->tp_dict, InEnumEntryName, (PyObject*)PyEnumValueDescr.GetPtr());
			return PyEnumValueDescr->EnumEntry;
		}
	}
	return nullptr;
}

FPyWrapperEnum* FPyWrapperEnum::AddDeprecatedEnumEntry(PyTypeObject* InType, FPyWrapperEnum* InEnumEntry, const char* InEnumEntryName, const char* InEnumEntryDoc)
{
	if (!FPyWrapperEnumMetaData::IsEnumFinalized(InType))
	{
		FPyWrapperEnumValueDescrObjectPtr PyEnumValueDescr = FPyWrapperEnumValueDescrObjectPtr::StealReference(FPyWrapperEnumValueDescrObject::NewDeprecated(InEnumEntry, InEnumEntryName, InEnumEntryDoc));
		if (PyEnumValueDescr)
		{
			PyDict_SetItemString(InType->tp_dict, InEnumEntryName, (PyObject*)PyEnumValueDescr.GetPtr());
			return PyEnumValueDescr->EnumEntry;
		}
	}
	return nullptr;
}

PyTypeObject InitializePyWrapperEnumType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperEnum::New(InType);
		}

		static void Dealloc(FPyWrapperEnum* InSelf)
		{
			FPyWrapperEnum::Free(InSelf);
		}

		static int Init(FPyWrapperEnum* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			return FPyWrapperEnum::Init(InSelf);
		}

		static PyObject* Str(FPyWrapperEnum* InSelf)
		{
			if (!FPyWrapperEnum::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyUnicode_FromFormat("<%s.%s: %d>", Py_TYPE(InSelf)->tp_name, TCHAR_TO_UTF8(*FPyWrapperEnum::GetEnumEntryName(InSelf)), FPyWrapperEnum::GetEnumEntryValue(InSelf));
		}

		static PyObject* RichCmp(FPyWrapperEnum* InSelf, PyObject* InOther, int InOp)
		{
			if (!FPyWrapperEnum::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FPyWrapperEnumPtr Other = FPyWrapperEnumPtr::StealReference(FPyWrapperEnum::CastPyObject(InOther, Py_TYPE(InSelf)));
			if (!Other)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (InOp != Py_EQ && InOp != Py_NE)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Only == and != comparison is supported"));
				return nullptr;
			}

			// Compare the objects if both enums are the same type, otherwise compare the values (as cast must have returned a deprecated enum and the entry objects won't match)
			const bool bIsIdentical = (Py_TYPE(InSelf) == Py_TYPE(Other.GetPtr())) ? InSelf->EntryValue == Other->EntryValue : FPyWrapperEnum::GetEnumEntryValue(InSelf) == FPyWrapperEnum::GetEnumEntryValue(Other);
			return PyBool_FromLong(InOp == Py_EQ ? bIsIdentical : !bIsIdentical);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperEnum* InSelf)
		{
			if (!FPyWrapperEnum::ValidateInternalState(InSelf))
			{
				return -1;
			}

			return PyObject_Hash(InSelf->EntryValue);
		}
	};

	struct FMethods
	{
		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:cast", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperEnum::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* StaticEnum(PyTypeObject* InType)
		{
			UEnum* Enum = FPyWrapperEnumMetaData::GetEnum(InType);
			return PyConversion::Pythonize(Enum);
		}

		static PyObject* GetDisplayName(FPyWrapperEnum* InSelf)
		{
			if (!FPyWrapperEnum::ValidateInternalState(InSelf))
			{
				return nullptr;
			}
			int64 EnumEntryValue = FPyWrapperEnum::GetEnumEntryValue(InSelf);
			UEnum* Enum = FPyWrapperEnumMetaData::GetEnum(InSelf);
			return PyConversion::Pythonize(Enum->GetDisplayNameTextByValue(EnumEntryValue));
		}
	};

	static PyMemberDef PyMembers[] = {
		{ PyCStrCast("name"), T_OBJECT, STRUCT_OFFSET(FPyWrapperEnum, EntryName), READONLY, PyCStrCast("The name of this enum entry") },
		{ PyCStrCast("value"), T_OBJECT, STRUCT_OFFSET(FPyWrapperEnum, EntryValue), READONLY, PyCStrCast("The numeric value of this enum entry") },
		{ nullptr, 0, 0, 0, nullptr }
	};

	// NOTE: _T is defines as: _T = typing.TypeVar('_T') and Type is defined by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls: Type[_T], object: object) -> _T -- cast the given object to this Unreal enum type" },
		{ "static_enum", PyCFunctionCast(&FMethods::StaticEnum), METH_NOARGS | METH_CLASS, "static_enum(cls) -> Enum -- get the Unreal enum of this type" },
		{ "get_display_name", PyCFunctionCast(&FMethods::GetDisplayName), METH_NOARGS, "get_display_name(self) -> Text -- get the UMETA display name of this type in the current culture" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"EnumBase", /* tp_name */
		sizeof(FPyWrapperEnum), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;

	PyType.tp_members = PyMembers;
	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;
	PyType.tp_doc = "Type for all Unreal exposed enum instances";

	return PyType;
}

PyTypeObject InitializePyWrapperEnumValueDescrType()
{
	struct FFuncs
	{
		static void Dealloc(FPyWrapperEnumValueDescrObject* InSelf)
		{
			FPyWrapperEnumValueDescrObject::Free(InSelf);
		}

		static PyObject* Str(FPyWrapperEnumValueDescrObject* InSelf)
		{
			return PyUnicode_FromString("<built-in enum value>");
		}

		static PyObject* DescrGet(FPyWrapperEnumValueDescrObject* InSelf, PyObject* InObj, PyObject* InType)
		{
			if (!FPyWrapperEnum::ValidateInternalState(InSelf->EnumEntry))
			{
				return nullptr;
			}

			// Deprecated enums emit a warning
			{
				FString DeprecationMessage;
				if (FPyWrapperEnumMetaData::IsEnumDeprecated(InSelf->EnumEntry, &DeprecationMessage) &&
					PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf->EnumEntry, *FString::Printf(TEXT("Enum '%s' is deprecated: %s"), UTF8_TO_TCHAR(Py_TYPE(InSelf->EnumEntry)->tp_name), *DeprecationMessage)) == -1
					)
				{
					// -1 from SetPythonWarning means the warning should be an exception
					return nullptr;
				}
			}

			// Deprecated enum entries emit a warning
			if (InSelf->DeprecationMessage.IsSet() && PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf->EnumEntry, *InSelf->DeprecationMessage.GetValue()) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return nullptr;
			}

			Py_INCREF(InSelf->EnumEntry);
			return (PyObject*)InSelf->EnumEntry;
		}

		static int DescrSet(FPyWrapperEnumValueDescrObject* InSelf, PyObject* InObj, PyObject* InValue)
		{
			PyErr_SetString(PyExc_Exception, "Enum values are read-only");
			return -1;
		}
	};

	struct FGetSets
	{
		static PyObject* GetName(FPyWrapperEnumValueDescrObject* InSelf, void* InClosure)
		{
			if (InSelf->EnumEntry && InSelf->EnumEntry->EntryName)
			{
				Py_INCREF(InSelf->EnumEntry->EntryName);
				return InSelf->EnumEntry->EntryName;
			}

			Py_RETURN_NONE;
		}

		static PyObject* GetDoc(FPyWrapperEnumValueDescrObject* InSelf, void* InClosure)
		{
			if (InSelf->EnumEntryDoc)
			{
				Py_INCREF(InSelf->EnumEntryDoc);
				return InSelf->EnumEntryDoc;
			}

			Py_RETURN_NONE;
		}
	};

	static PyGetSetDef PyGetSets[] = {
		{ PyCStrCast("__name__"), (getter)&FGetSets::GetName, nullptr, nullptr, nullptr },
		{ PyCStrCast("__doc__"), (getter)&FGetSets::GetDoc, nullptr, nullptr, nullptr },
		{ nullptr, nullptr, nullptr, nullptr, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"_EnumEntry", /* tp_name */
		sizeof(FPyWrapperEnumValueDescrObject), /* tp_basicsize */
	};

	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_descr_get = (descrgetfunc)&FFuncs::DescrGet;
	PyType.tp_descr_set = (descrsetfunc)&FFuncs::DescrSet;
	PyType.tp_getattro = (getattrofunc)&PyObject_GenericGetAttr;

	PyType.tp_getset = PyGetSets;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;

	return PyType;
}

PyTypeObject InitializePyWrapperEnumMetaclassType()
{
	struct FFuncs
	{
		static PyObject* GetIter(PyTypeObject* InSelf)
		{
			typedef TPyPtr<FPyWrapperEnumIterator> FPyWrapperEnumIteratorPtr;

			FPyWrapperEnumArrayView EnumEntriesArray;
			if (const FPyWrapperEnumMetaData* EnumMetaData = FPyWrapperEnumMetaData::GetMetaData(InSelf))
			{
				EnumEntriesArray = EnumMetaData->EnumEntries;
			}
			
			FPyWrapperEnumIteratorPtr NewIter = FPyWrapperEnumIteratorPtr::StealReference(FPyWrapperEnumIterator::New(&PyWrapperEnumIteratorType));
			if (FPyWrapperEnumIterator::Init(NewIter, EnumEntriesArray) != 0)
			{
				return nullptr;
			}
			
			return (PyObject*)NewIter.Release();
		}
	};

	struct FSequenceFuncs
	{
		static Py_ssize_t Len(PyTypeObject* InSelf)
		{
			if (const FPyWrapperEnumMetaData* EnumMetaData = FPyWrapperEnumMetaData::GetMetaData(InSelf))
			{
				return EnumMetaData->EnumEntries.Num();
			}
			return 0;
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"_EnumType", /* tp_name */
		0, /* tp_basicsize */
	};

	PyType.tp_base = &PyType_Type;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;
	PyType.tp_doc = "Metaclass type for all Unreal exposed enum instances";

	static PySequenceMethods PySequence;
	PySequence.sq_length = (lenfunc)&FSequenceFuncs::Len;

	PyType.tp_as_sequence = &PySequence;

	return PyType;
}

PyTypeObject InitializePyWrapperEnumIteratorType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperEnumIterator::New(InType);
		}

		static void Dealloc(FPyWrapperEnumIterator* InSelf)
		{
			FPyWrapperEnumIterator::Free(InSelf);
		}

		static int Init(FPyWrapperEnumIterator* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				return -1;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperEnumType) != 1)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot initialize '%s' with an instance of '%s'"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			FPyWrapperEnumArrayView EnumEntriesArray;
			if (const FPyWrapperEnumMetaData* EnumMetaData = FPyWrapperEnumMetaData::GetMetaData((FPyWrapperEnum*)PyObj))
			{
				EnumEntriesArray = EnumMetaData->EnumEntries;
			}

			return FPyWrapperEnumIterator::Init(InSelf, EnumEntriesArray);
		}

		static FPyWrapperEnumIterator* GetIter(FPyWrapperEnumIterator* InSelf)
		{
			return FPyWrapperEnumIterator::GetIter(InSelf);
		}

		static PyObject* IterNext(FPyWrapperEnumIterator* InSelf)
		{
			return FPyWrapperEnumIterator::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"_EnumIterator", /* tp_name */
		sizeof(FPyWrapperEnumIterator), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed enum iterators";

	return PyType;
}

PyTypeObject PyWrapperEnumType = InitializePyWrapperEnumType();
PyTypeObject PyWrapperEnumValueDescrType = InitializePyWrapperEnumValueDescrType();
PyTypeObject PyWrapperEnumMetaclassType = InitializePyWrapperEnumMetaclassType();
PyTypeObject PyWrapperEnumIteratorType = InitializePyWrapperEnumIteratorType();

FPyWrapperEnumMetaData::FPyWrapperEnumMetaData()
	: Enum(nullptr)
	, bFinalized(false)
{
}

UEnum* FPyWrapperEnumMetaData::GetEnum(PyTypeObject* PyType)
{
	FPyWrapperEnumMetaData* PyWrapperMetaData = FPyWrapperEnumMetaData::GetMetaData(PyType);
	return PyWrapperMetaData ? PyWrapperMetaData->Enum : nullptr;
}

UEnum* FPyWrapperEnumMetaData::GetEnum(FPyWrapperEnum* Instance)
{
	return GetEnum(Py_TYPE(Instance));
}

bool FPyWrapperEnumMetaData::IsEnumDeprecated(PyTypeObject* PyType, FString* OutDeprecationMessage)
{
	if (FPyWrapperEnumMetaData* PyWrapperMetaData = FPyWrapperEnumMetaData::GetMetaData(PyType))
	{
		if (PyWrapperMetaData->DeprecationMessage.IsSet())
		{
			if (OutDeprecationMessage)
			{
				*OutDeprecationMessage = PyWrapperMetaData->DeprecationMessage.GetValue();
			}
			return true;
		}
	}

	return false;
}

bool FPyWrapperEnumMetaData::IsEnumDeprecated(FPyWrapperEnum* Instance, FString* OutDeprecationMessage)
{
	return IsEnumDeprecated(Py_TYPE(Instance), OutDeprecationMessage);
}

bool FPyWrapperEnumMetaData::IsEnumFinalized(PyTypeObject* PyType)
{
	if (FPyWrapperEnumMetaData* PyWrapperMetaData = FPyWrapperEnumMetaData::GetMetaData(PyType))
	{
		return PyWrapperMetaData->bFinalized;
	}

	return false;
}

bool FPyWrapperEnumMetaData::IsEnumFinalized(FPyWrapperEnum* Instance)
{
	return IsEnumFinalized(Py_TYPE(Instance));
}

void FPyWrapperEnumMetaData::AddTypeReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Enum);
}

class FPythonGeneratedEnumBuilder
{
public:
	FPythonGeneratedEnumBuilder(PyTypeObject* InPyType)
		: EnumName()
		, PyType(InPyType)
		, NewEnum(nullptr)
	{
		UObject* EnumOuter = nullptr;
		PyUtil::GetGeneratedTypeOuterAndName(PyType, EnumOuter, EnumName);

		UObjectRedirector* EnumRedirector = CreatePythonTypeLegacyRedirector(PyUtil::GetCleanTypename(InPyType), FTopLevelAssetPath(EnumOuter->GetFName(), *EnumName));

		// Enum instances are re-used if they already exist
		NewEnum = FindObject<UPythonGeneratedEnum>(EnumOuter, *EnumName);
		if (!NewEnum)
		{
			NewEnum = NewObject<UPythonGeneratedEnum>(EnumOuter, *EnumName, RF_Public | RF_Transient);
			NewEnum->SetMetaData(TEXT("DisplayName"), *PyUtil::GetGeneratedTypeDisplayName(PyType));
			NewEnum->SetMetaData(TEXT("BlueprintType"), TEXT("true"));
		}
		NewEnum->EnumValueDefs.Reset();

		NewEnum->EnumRedirector = EnumRedirector;
		if (NewEnum->EnumRedirector)
		{
			NewEnum->EnumRedirector->DestinationObject = NewEnum;
		}
	}

	~FPythonGeneratedEnumBuilder()
	{
		// If NewEnum is still set at this point, if means Finalize wasn't called and we should destroy the partially built enum
		if (NewEnum)
		{
			NewEnum->ClearFlags(RF_Public | RF_Standalone);
			NewEnum = nullptr;

			Py_BEGIN_ALLOW_THREADS
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			Py_END_ALLOW_THREADS
		}
	}

	UPythonGeneratedEnum* Finalize(const TArray<FPyUValueDef*>& InPyValueDefs)
	{
		// Populate the enum with its values, and replace the definitions with real descriptors
		if (!RegisterDescriptors(InPyValueDefs))
		{
			return nullptr;
		}

		// Let Python know that we've changed its type
		PyType_Modified(PyType);

		// Finalize the enum
		NewEnum->Bind();

		// Add the object meta-data to the type
		NewEnum->PyMetaData.Enum = NewEnum;
		NewEnum->PyMetaData.bFinalized = true;
		FPyWrapperEnumMetaData::SetMetaData(PyType, &NewEnum->PyMetaData);

		// Map the Unreal enum to the Python type
		NewEnum->PyType = FPyTypeObjectPtr::NewReference(PyType);
		FPyWrapperTypeRegistry::Get().RegisterWrappedEnumType(NewEnum, PyType, false);

		// Null the NewEnum pointer so the destructor doesn't kill it
		UPythonGeneratedEnum* FinalizedEnum = NewEnum;
		NewEnum = nullptr;
		return FinalizedEnum;
	}

	bool CreateValueFromDefinition(const FString& InFieldName, FPyUValueDef* InPyValueDef)
	{
		int64 EnumValue = 0;
		if (!PyConversion::Nativize(InPyValueDef->Value, EnumValue))
		{
			PyUtil::SetPythonError(PyExc_TypeError, PyType, *FString::Printf(TEXT("Failed to convert enum value for '%s'"), *InFieldName));
			return false;
		}

		// Build the definition data for the new enum value
		UPythonGeneratedEnum::FEnumValueDef& EnumValueDef = *NewEnum->EnumValueDefs.Add_GetRef(MakeShared<UPythonGeneratedEnum::FEnumValueDef>());
		EnumValueDef.PyIndex = NewEnum->EnumValueDefs.Num() - 1;
		EnumValueDef.Value = EnumValue;
		EnumValueDef.Name = InFieldName;

		return true;
	}

private:
	bool RegisterDescriptors(const TArray<FPyUValueDef*>& InPyValueDefs)
	{
		// The enum entries came from a Python dict, so sort them into a consistent order (using the value) before registering them
		NewEnum->EnumValueDefs.Sort([](const TSharedPtr<UPythonGeneratedEnum::FEnumValueDef>& EnumValueDefOne, const TSharedPtr<UPythonGeneratedEnum::FEnumValueDef>& EnumValueDefTwo)
		{
			return EnumValueDefOne->Value < EnumValueDefTwo->Value;
		});

		// Populate the enum with its values
		check(InPyValueDefs.Num() == NewEnum->EnumValueDefs.Num());
		{
			TArray<TPair<FName, int64>> ValueNames;
			for (const TSharedPtr<UPythonGeneratedEnum::FEnumValueDef>& EnumValueDef : NewEnum->EnumValueDefs)
			{
				const FString NamespacedValueName = FString::Printf(TEXT("%s::%s"), *EnumName, *EnumValueDef->Name);
				ValueNames.Emplace(MakeTuple(FName(*NamespacedValueName), EnumValueDef->Value));
			}
			if (!NewEnum->SetEnums(ValueNames, UEnum::ECppForm::Namespaced, UEnum::EUnderlyingType::uint8, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes))
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, TEXT("Failed to set enum values"));
				return false;
			}

			// Can't set the meta-data until SetEnums has been called
			// Note: Beware, InPyValueDefs is not guaranteed to be in the same order due to the sort above - index it via the PyIndex of the value definition!
			for (int32 EnumValueIndex = 0; EnumValueIndex < NewEnum->EnumValueDefs.Num(); ++EnumValueIndex)
			{
				TSharedPtr<UPythonGeneratedEnum::FEnumValueDef>& EnumValueDef = NewEnum->EnumValueDefs[EnumValueIndex];
				FPyUValueDef::ApplyMetaData(InPyValueDefs[EnumValueDef->PyIndex], [this, EnumValueIndex](const FString& InMetaDataKey, const FString& InMetaDataValue)
				{
					NewEnum->SetMetaData(*InMetaDataKey, *InMetaDataValue, EnumValueIndex);
				});
				NewEnum->EnumValueDefs[EnumValueIndex]->DocString = PyGenUtil::GetEnumEntryTooltip(NewEnum, EnumValueIndex);
			}
		}

		// Replace the definitions with real descriptors
		for (const TSharedPtr<UPythonGeneratedEnum::FEnumValueDef>& EnumValueDef : NewEnum->EnumValueDefs)
		{
			FPyWrapperEnum* PyEnumEntry = FPyWrapperEnum::AddEnumEntry(PyType, EnumValueDef->Value, TCHAR_TO_UTF8(*EnumValueDef->Name), TCHAR_TO_UTF8(*EnumValueDef->DocString));
			if (PyEnumEntry)
			{
				NewEnum->PyMetaData.EnumEntries.Add(PyEnumEntry);
			}
		}

		return true;
	}

	FString EnumName;
	PyTypeObject* PyType;
	UPythonGeneratedEnum* NewEnum;
};

void UPythonGeneratedEnum::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPythonGeneratedEnum* This = CastChecked<UPythonGeneratedEnum>(InThis);
	Collector.AddReferencedObject(This->EnumRedirector, This);
}

void UPythonGeneratedEnum::BeginDestroy()
{
	ReleasePythonResources();
	Super::BeginDestroy();
}

void UPythonGeneratedEnum::ReleasePythonResources()
{
	// This may be called after Python has already shut down
	if (Py_IsInitialized())
	{
		FPyScopedGIL GIL;
		UnregisterGeneratedType();
		FPyWrapperEnumMetaData::SetMetaData(PyType.Get(), nullptr);
		PyType.Reset();
	}
	else
	{
		// Release ownership if Python has been shut down to avoid attempting to delete the object (which is already dead)
		PyType.Release();
	}

	EnumValueDefs.Reset();
	PyMetaData = FPyWrapperEnumMetaData();
}

void UPythonGeneratedEnum::UnregisterGeneratedType()
{
	if (PyType)
	{
		FPyWrapperTypeRegistry::Get().UnregisterWrappedEnumType(this, PyType, false);
	}
}

UPythonGeneratedEnum* UPythonGeneratedEnum::GenerateEnum(PyTypeObject* InPyType)
{
	// Builder used to generate the enum
	FPythonGeneratedEnumBuilder PythonEnumBuilder(InPyType);

	// Add the values to this enum
	TArray<FPyUValueDef*> PyValueDefs;
	{
		PyObject* FieldKey = nullptr;
		PyObject* FieldValue = nullptr;
		Py_ssize_t FieldIndex = 0;
		while (PyDict_Next(InPyType->tp_dict, &FieldIndex, &FieldKey, &FieldValue))
		{
			const FString FieldName = PyUtil::PyObjectToUEString(FieldKey);

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyUValueDefType) == 1)
			{
				FPyUValueDef* PyValueDef = (FPyUValueDef*)FieldValue;
				PyValueDefs.Add(PyValueDef);

				if (!PythonEnumBuilder.CreateValueFromDefinition(FieldName, PyValueDef))
				{
					return nullptr;
				}
			}

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyFPropertyDefType) == 1)
			{
				// Properties are not supported on enums
				PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("Enums do not support properties"));
				return nullptr;
			}

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyUFunctionDefType) == 1)
			{
				// Functions are not supported on enums
				PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("Enums do not support methods"));
				return nullptr;
			}
		}
	}

	// Finalize the struct with its value meta-data
	return PythonEnumBuilder.Finalize(PyValueDefs);
}

#endif	// WITH_PYTHON

