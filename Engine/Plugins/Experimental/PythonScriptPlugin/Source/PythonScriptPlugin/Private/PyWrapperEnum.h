// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "UObject/Class.h"
#include "PyWrapperEnum.generated.h"

class UObjectRedirector;

#if WITH_PYTHON

/** Python type for FPyWrapperEnum */
extern PyTypeObject PyWrapperEnumType;

/** Python type for FPyWrapperEnumValueDescrObject */
extern PyTypeObject PyWrapperEnumValueDescrType;

/** Initialize the PyWrapperEnum types and add them to the given Python module */
void InitializePyWrapperEnum(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed enum instances (an instance is created for each entry in the enum, before the enum type is locked for creating new instances) */
struct FPyWrapperEnum : public FPyWrapperBase
{
	/** Name of this enum entry */
	PyObject* EntryName;

	/** Value of this enum entry */
	PyObject* EntryValue;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperEnum* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperEnum* InSelf);

	/** Initialize this wrapper instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperEnum* InSelf);

	/** Initialize this wrapper instance (called directly in C++) */
	static int Init(FPyWrapperEnum* InSelf, const int64 InEnumEntryValue, const char* InEnumEntryName);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperEnum* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperEnum* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperEnum* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperEnum* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Get the name of the enum entry as a string */
	static FString GetEnumEntryName(FPyWrapperEnum* InSelf);

	/** Get the value of the enum entry as an int */
	static int64 GetEnumEntryValue(FPyWrapperEnum* InSelf);

	/** Add the given enum entry on the given enum type (returns borrowed reference) */
	static FPyWrapperEnum* AddEnumEntry(PyTypeObject* InType, const int64 InEnumEntryValue, const char* InEnumEntryName, const char* InEnumEntryDoc);

	/** Add the given deprecated enum entry on the given enum type (returns borrowed reference) */
	static FPyWrapperEnum* AddDeprecatedEnumEntry(PyTypeObject* InType, FPyWrapperEnum* InEnumEntry, const char* InEnumEntryName, const char* InEnumEntryDoc);
};

/** Meta-data for all Unreal exposed enum types */
struct FPyWrapperEnumMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperEnumMetaData, FGuid(0x1D69987C, 0x2F624403, 0x8379FCB5, 0xF896B595))

	FPyWrapperEnumMetaData();

	/** Get the UEnum from the given type */
	static UEnum* GetEnum(PyTypeObject* PyType);

	/** Get the UEnum from the type of the given instance */
	static UEnum* GetEnum(FPyWrapperEnum* Instance);

	/** Check to see if the enum is deprecated, and optionally return its deprecation message */
	static bool IsEnumDeprecated(PyTypeObject* PyType, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the enum is deprecated, and optionally return its deprecation message */
	static bool IsEnumDeprecated(FPyWrapperEnum* Instance, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the enum is finalized */
	static bool IsEnumFinalized(PyTypeObject* PyType);

	/** Check to see if the enum is finalized */
	static bool IsEnumFinalized(FPyWrapperEnum* Instance);

	/** Add object references from this type meta-data to the given collector */
	virtual void AddTypeReferencedObjects(FReferenceCollector& Collector) override;

	/** Get the reflection meta data type object associated with this wrapper type if there is one or nullptr if not. */
	virtual const UField* GetMetaType() const override
	{
		return Enum;
	}

	/** Unreal enum */
	TObjectPtr<UEnum> Enum;

	/** True if this enum type has been finalized after having all of its entries added to it */
	bool bFinalized;

	/** Set if this enum is deprecated and using it should emit a deprecation warning */
	TOptional<FString> DeprecationMessage;

	/** Array of enum entries in the order they were added (enum entries are stored as borrowed references) */
	TArray<FPyWrapperEnum*> EnumEntries;
};

typedef TPyPtr<FPyWrapperEnum> FPyWrapperEnumPtr;

#endif	// WITH_PYTHON

/** An Unreal enum that was generated from a Python type */
UCLASS()
class UPythonGeneratedEnum final : public UEnum, public IPythonResourceOwner
{
	GENERATED_BODY()

#if WITH_PYTHON

public:
	//~ UObject interface
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void BeginDestroy() override;
	virtual bool IsAsset() const override
	{
		return false;
	}

	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override;

	/** Unregister this type from FPyWrapperTypeRegistry */
	void UnregisterGeneratedType();

	/** Generate an Unreal enum from the given Python type */
	static UPythonGeneratedEnum* GenerateEnum(PyTypeObject* InPyType);

private:
	/** Definition data for an Unreal enum value generated from a Python type */
	struct FEnumValueDef
	{
		/** Index of the enum value as it was registered with us from the Python */
		int32 PyIndex = 0;

		/** Numeric value of the enum value */
		int64 Value = 0;

		/** Name of the enum value */
		FString Name;

		/** Documentation string of the enum value */
		FString DocString;
	};

	/** Python type this enum was generated from */
	FPyTypeObjectPtr PyType;

	/** Array of values generated for this enum */
	TArray<TSharedPtr<FEnumValueDef>> EnumValueDefs;

	/** Meta-data for this generated enum that is applied to the Python type */
	FPyWrapperEnumMetaData PyMetaData;

	TObjectPtr<UObjectRedirector> EnumRedirector;

	friend class FPythonGeneratedEnumBuilder;

#else	// WITH_PYTHON

public:
	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override {}

#endif	// WITH_PYTHON
};
