// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperFieldPath.h"
#include "PyWrapperTypeRegistry.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"

#if WITH_PYTHON

void InitializePyWrapperFieldPath(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperFieldPathType) == 0)
	{
		ModuleInfo.AddType(&PyWrapperFieldPathType);
	}
}

void FPyWrapperFieldPath::InitValue(FPyWrapperFieldPath* InSelf, const FFieldPath& InValue)
{
	Super::InitValue(InSelf, InValue);
	FPyWrapperFieldPathFactory::Get().MapInstance(InSelf->Value, InSelf);
}


void FPyWrapperFieldPath::DeinitValue(FPyWrapperFieldPath* InSelf)
{
	FPyWrapperFieldPathFactory::Get().UnmapInstance(InSelf->Value, Py_TYPE(InSelf));
	Super::DeinitValue(InSelf);
}

FPyWrapperFieldPath* FPyWrapperFieldPath::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperFieldPathType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperFieldPath*)InPyObject;
	}

	return nullptr;
}

FPyWrapperFieldPath* FPyWrapperFieldPath::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperFieldPathType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperFieldPathType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperFieldPath*)InPyObject;
	}

	{
		FFieldPath InitValue;
		if (PyConversion::Nativize(InPyObject, InitValue, PyConversion::ESetErrorState::No))
		{
			FPyWrapperFieldPathPtr NewFiedPath = FPyWrapperFieldPathPtr::StealReference(FPyWrapperFieldPath::New(InType));
			if (NewFiedPath)
			{
				if (FPyWrapperFieldPath::Init(NewFiedPath, InitValue) != 0)
				{
					return nullptr;
				}
			}
			SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
			return NewFiedPath.Release();
		}
	}

	{
		FString InitPath;
		if (PyConversion::Nativize(InPyObject, InitPath, PyConversion::ESetErrorState::No))
		{
			FPyWrapperFieldPathPtr NewFiedPath = FPyWrapperFieldPathPtr::StealReference(FPyWrapperFieldPath::New(InType));
			if (NewFiedPath)
			{
				FFieldPath InitValue;
				InitValue.Generate(*InitPath);
				if (FPyWrapperFieldPath::Init(NewFiedPath, InitValue) != 0)
				{
					return nullptr;
				}
			}
			SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
			return NewFiedPath.Release();
		}
	}

	return nullptr;
}

PyTypeObject InitializePyWrapperFieldPathType()
{
	struct FFuncs
	{
		static int Init(FPyWrapperFieldPath* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "|O:call", &PyObj))
			{
				return -1;
			}

			FFieldPath InitValue;
			FString StrValue;
			if (PyObj && PyConversion::Nativize(PyObj, StrValue, PyConversion::ESetErrorState::No)) // Init from a string.
			{
				InitValue.Generate(*StrValue);
			}
			else if (PyObj && !PyConversion::Nativize(PyObj, InitValue)) // Init from another FieldPath.
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert init argument '%s' to 'FieldPath'"), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}
			// else: Default initialization. "f = unreal.FieldPath()".

			return FPyWrapperFieldPath::Init(InSelf, InitValue);
		}

		static PyObject* Str(FPyWrapperFieldPath* InSelf)
		{
			return PyUnicode_FromFormat(TCHAR_TO_UTF8(*InSelf->Value.ToString()));
		}

		static PyObject* Repr(FPyWrapperFieldPath* InSelf)
		{
			return PyUnicode_FromString(TCHAR_TO_UTF8(*FString::Printf(TEXT("FieldPath(\"%s\")"), *InSelf->Value.ToString())));
		}

		static PyObject* RichCmp(FPyWrapperFieldPath* InSelf, PyObject* InOther, int InOp)
		{
			FFieldPath Other;
			if (!PyConversion::Nativize(InOther, Other, PyConversion::ESetErrorState::No))
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (InOp != Py_EQ && InOp != Py_NE)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Only == and != comparison is supported"));
				return nullptr;
			}

			bool bIdentical = InSelf->Value == Other;
			return PyBool_FromLong(InOp == Py_EQ ? bIdentical : !bIdentical);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperFieldPath* InSelf)
		{
			uint32 FieldPathHash = GetTypeHash(InSelf->Value);
			return FieldPathHash != -1 ? FieldPathHash : 0;
		}
	};

	struct FMethods
	{
		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperFieldPath::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperFieldPath* InSelf)
		{
			return PyConversion::Pythonize(InSelf->Value);
		}

		static PyObject* IsValid(FPyWrapperFieldPath* InSelf)
		{
			return PyConversion::Pythonize(InSelf->Value.TryToResolvePath(nullptr) != nullptr);
		}
	};

	static PyMethodDef PyMethods[] = {
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls, obj: object) -> FieldPath -- cast the given object to this Unreal field path type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> FieldPath -- copy this Unreal field path" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> FieldPath -- copy this Unreal field path" },
		{ "is_valid", PyCFunctionCast(&FMethods::IsValid), METH_NOARGS, "is_valid(self) -> bool -- whether this Unreal field path refers to an existing Unreal field" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = InitializePyWrapperBasicType<FPyWrapperFieldPath>("FieldPath", "Type for all Unreal exposed FieldPath instances");

	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Repr;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;

	PyType.tp_methods = PyMethods;

	return PyType;
}

PyTypeObject PyWrapperFieldPathType = InitializePyWrapperFieldPathType();

#endif	// WITH_PYTHON
