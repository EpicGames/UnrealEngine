// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBasic.h"

#if WITH_PYTHON

/** Python type for FPyWrapperFieldPath */
extern PyTypeObject PyWrapperFieldPathType;

/** Initialize the PyWrapperFieldPath types and add them to the given Python module */
void InitializePyWrapperFieldPath(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed FPyWrapperFieldPath instances */
struct FPyWrapperFieldPath : public TPyWrapperBasic<FFieldPath, FPyWrapperFieldPath>
{
	typedef TPyWrapperBasic<FFieldPath, FPyWrapperFieldPath> Super;

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperFieldPath* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperFieldPath* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Initialize the value of this wrapper instance (internal) */
	static void InitValue(FPyWrapperFieldPath* InSelf, const FFieldPath& InValue);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void DeinitValue(FPyWrapperFieldPath* InSelf);
};

typedef TPyPtr<FPyWrapperFieldPath> FPyWrapperFieldPathPtr;

#endif	// WITH_PYTHON