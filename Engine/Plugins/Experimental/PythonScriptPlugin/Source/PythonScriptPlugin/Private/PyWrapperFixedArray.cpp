// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperFixedArray.h"
#include "PyWrapperTypeRegistry.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"

#if WITH_PYTHON

/** Python type for FPyWrapperFixedArrayIterator */
extern PyTypeObject PyWrapperFixedArrayIteratorType;

/** Iterator used with fixed-arrays */
struct FPyWrapperFixedArrayIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Instance being iterated over */
	FPyWrapperFixedArray* IterInstance;

	/** Current iteration index */
	int32 IterIndex;

	/** New this iterator instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperFixedArrayIterator* New(PyTypeObject* InType)
	{
		FPyWrapperFixedArrayIterator* Self = (FPyWrapperFixedArrayIterator*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->IterInstance = nullptr;
			Self->IterIndex = 0;
		}
		return Self;
	}

	/** Free this iterator instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperFixedArrayIterator* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this iterator instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperFixedArrayIterator* InSelf, FPyWrapperFixedArray* InInstance)
	{
		Deinit(InSelf);

		Py_INCREF(InInstance);
		InSelf->IterInstance = InInstance;
		InSelf->IterIndex = 0;

		return 0;
	}

	/** Deinitialize this iterator instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperFixedArrayIterator* InSelf)
	{
		Py_XDECREF(InSelf->IterInstance);
		InSelf->IterInstance = nullptr;
		InSelf->IterIndex = 0;
	}
	
	/** Called to validate the internal state of this iterator instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperFixedArrayIterator* InSelf)
	{
		if (!InSelf->IterInstance)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - IterInstance is null!"));
			return false;
		}

		return true;
	}

	/** Get the iterator */
	static FPyWrapperFixedArrayIterator* GetIter(FPyWrapperFixedArrayIterator* InSelf)
	{
		Py_INCREF(InSelf);
		return InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(FPyWrapperFixedArrayIterator* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		if (InSelf->IterIndex < FPyWrapperFixedArray::Len(InSelf->IterInstance))
		{
			return FPyWrapperFixedArray::GetItem(InSelf->IterInstance, InSelf->IterIndex++);
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}
};

void InitializePyWrapperFixedArray(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperFixedArrayType) == 0)
	{
		static FPyWrapperFixedArrayMetaData MetaData;
		FPyWrapperFixedArrayMetaData::SetMetaData(&PyWrapperFixedArrayType, &MetaData);
		ModuleInfo.AddType(&PyWrapperFixedArrayType);
	}

	PyType_Ready(&PyWrapperFixedArrayIteratorType);
}

FPyWrapperFixedArray* FPyWrapperFixedArray::New(PyTypeObject* InType)
{
	FPyWrapperFixedArray* Self = (FPyWrapperFixedArray*)FPyWrapperBase::New(InType);
	if (Self)
	{
		new(&Self->OwnerContext) FPyWrapperOwnerContext();
		new(&Self->ArrayProp) PyUtil::FConstPropOnScope();
		Self->ArrayInstance = nullptr;
	}
	return Self;
}

void FPyWrapperFixedArray::Free(FPyWrapperFixedArray* InSelf)
{
	Deinit(InSelf);

	InSelf->OwnerContext.~FPyWrapperOwnerContext();
	InSelf->ArrayProp.~TPropOnScope();
	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperFixedArray::Init(FPyWrapperFixedArray* InSelf, const PyUtil::FPropertyDef& InPropDef, const int32 InLen)
{
	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	PyUtil::FConstPropOnScope ArrayProp = PyUtil::FConstPropOnScope::OwnedReference(PyUtil::CreateProperty(InPropDef, InLen));
	if (!ArrayProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Array property was null during init"));
		return -1;
	}

	void* ArrayValue = FMemory::Malloc(ArrayProp->GetSize(), ArrayProp->GetMinAlignment());
	ArrayProp->InitializeValue(ArrayValue);

	InSelf->ArrayProp = MoveTemp(ArrayProp);
	InSelf->ArrayInstance = ArrayValue;

	FPyWrapperFixedArrayFactory::Get().MapInstance(InSelf->ArrayInstance, InSelf);
	return 0;
}

int FPyWrapperFixedArray::Init(FPyWrapperFixedArray* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod)
{
	InOwnerContext.AssertValidConversionMethod(InConversionMethod);

	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	check(InProp && InValue);

	PyUtil::FConstPropOnScope PropToUse;
	void* ArrayInstanceToUse = nullptr;
	switch (InConversionMethod)
	{
	case EPyConversionMethod::Copy:
	case EPyConversionMethod::Steal:
		{
			PropToUse = PyUtil::FConstPropOnScope::OwnedReference(PyUtil::CreateProperty(InProp));
			if (!PropToUse)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to create property from '%s' (%s)"), *InProp->GetName(), *InProp->GetClass()->GetName()));
				return -1;
			}

			ArrayInstanceToUse = FMemory::Malloc(PropToUse->GetSize(), PropToUse->GetMinAlignment());
			PropToUse->InitializeValue(ArrayInstanceToUse);
			PropToUse->CopyCompleteValue(ArrayInstanceToUse, InValue);
		}
		break;

	case EPyConversionMethod::Reference:
		{
			PropToUse = PyUtil::FConstPropOnScope::ExternalReference(InProp);
			ArrayInstanceToUse = InValue;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EPyConversionMethod"));
		break;
	}

	check(PropToUse && ArrayInstanceToUse);

	InSelf->OwnerContext = InOwnerContext;
	InSelf->ArrayProp = MoveTemp(PropToUse);
	InSelf->ArrayInstance = ArrayInstanceToUse;

	FPyWrapperFixedArrayFactory::Get().MapInstance(InSelf->ArrayInstance, InSelf);
	return 0;
}

void FPyWrapperFixedArray::Deinit(FPyWrapperFixedArray* InSelf)
{
	if (InSelf->ArrayInstance)
	{
		FPyWrapperFixedArrayFactory::Get().UnmapInstance(InSelf->ArrayInstance, Py_TYPE(InSelf));
	}

	if (InSelf->OwnerContext.HasOwner())
	{
		InSelf->OwnerContext.Reset();
	}
	else if (InSelf->ArrayInstance)
	{
		if (InSelf->ArrayProp)
		{
			InSelf->ArrayProp->DestroyValue(InSelf->ArrayInstance);
		}
		FMemory::Free(InSelf->ArrayInstance);
	}
	InSelf->ArrayProp.Release();
	InSelf->ArrayInstance = nullptr;
}

bool FPyWrapperFixedArray::ValidateInternalState(FPyWrapperFixedArray* InSelf)
{
	if (!InSelf->ArrayProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ArrayProp is null!"));
		return false;
	}

	if (!InSelf->ArrayInstance)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ArrayInstance is null!"));
		return false;
	}

	return true;
}

FPyWrapperFixedArray* FPyWrapperFixedArray::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperFixedArrayType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperFixedArray*)InPyObject;
	}

	return nullptr;
}

FPyWrapperFixedArray* FPyWrapperFixedArray::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InPropDef, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperFixedArrayType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperFixedArrayType) == 1))
	{
		FPyWrapperFixedArray* Self = (FPyWrapperFixedArray*)InPyObject;
		if (!ValidateInternalState(Self))
		{
			return nullptr;
		}

		const PyUtil::FPropertyDef SelfPropDef = Self->ArrayProp.Get();
		if (SelfPropDef == InPropDef)
		{
			SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

			Py_INCREF(Self);
			return Self;
		}

		FPyWrapperFixedArrayPtr NewArray = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::New(InType));
		if (FPyWrapperFixedArray::Init(NewArray, InPropDef, Self->ArrayProp->ArrayDim) != 0)
		{
			return nullptr;
		}

		// Attempt to convert the entries in the array to the native format of the new array
		FString ExportedEntry;
		for (int32 ArrIndex = 0; ArrIndex < NewArray->ArrayProp->ArrayDim; ++ArrIndex)
		{
			ExportedEntry.Reset();
			if (!Self->ArrayProp->ExportText_Direct(ExportedEntry, GetItemPtr(Self, ArrIndex), GetItemPtr(Self, ArrIndex), nullptr, PPF_None))
			{
				PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to export text for property '%s' (%s) at index %d"), *Self->ArrayProp->GetName(), *Self->ArrayProp->GetClass()->GetName(), ArrIndex));
				return nullptr;
			}

			if (!NewArray->ArrayProp->ImportText_Direct(*ExportedEntry, GetItemPtr(NewArray, ArrIndex), nullptr, PPF_None))
			{
				PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to import text '%s' for property '%s' (%s) at index %d"), *ExportedEntry, *NewArray->ArrayProp->GetName(), *NewArray->ArrayProp->GetClass()->GetName(), ArrIndex));
				return nullptr;
			}
		}

		SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
		return NewArray.Release();
	}

	// Attempt conversion from any iterable sequence type that has a defined length
	if (!PyUtil::IsMappingType(InPyObject))
	{
		const Py_ssize_t SequenceLen = PyObject_Length(InPyObject);
		if (SequenceLen != -1)
		{
			FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(InPyObject));
			if (PyObjIter)
			{
				FPyWrapperFixedArrayPtr NewArray = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::New(InType));
				if (FPyWrapperFixedArray::Init(NewArray, InPropDef, SequenceLen) != 0)
				{
					return nullptr;
				}

				for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
				{
					FPyObjectPtr SequenceItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
					if (!SequenceItem)
					{
						return nullptr;
					}

					if (!PyConversion::NativizeProperty_Direct(SequenceItem, NewArray->ArrayProp, GetItemPtr(NewArray, SequenceIndex)))
					{
						return nullptr;
					}
				}

				SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
				return NewArray.Release();
			}
		}
	}

	return nullptr;
}

void* FPyWrapperFixedArray::GetItemPtr(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex)
{
	check(ValidateInternalState(InSelf));
	check(InIndex >= 0 && InIndex < InSelf->ArrayProp->ArrayDim);

	// This doesn't use ContainerPtrToValuePtr as the ArrayInstance has already been adjusted to 
	// point to the first element in the array and we just need to adjust it for the element size
	return static_cast<uint8*>(InSelf->ArrayInstance) + (InSelf->ArrayProp->GetElementSize() * InIndex);
}

Py_ssize_t FPyWrapperFixedArray::Len(FPyWrapperFixedArray* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	return InSelf->ArrayProp->ArrayDim;
}

PyObject* FPyWrapperFixedArray::GetItem(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, InSelf->ArrayProp->ArrayDim);
	if (PyUtil::ValidateContainerIndexParam(ResolvedIndex, InSelf->ArrayProp->ArrayDim, InSelf->ArrayProp, *PyUtil::GetErrorContext(InSelf)) != 0)
	{
		return nullptr;
	}

	PyObject* PyItemObj = nullptr;
	if (!PyConversion::PythonizeProperty_Direct(InSelf->ArrayProp, GetItemPtr(InSelf, ResolvedIndex), PyItemObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert property '%s' (%s) at index %zd"), *InSelf->ArrayProp->GetName(), *InSelf->ArrayProp->GetClass()->GetName(), ResolvedIndex));
		return nullptr;
	}
	return PyItemObj;
}

int FPyWrapperFixedArray::SetItem(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const Py_ssize_t ResolvedIndex = PyUtil::ResolveContainerIndexParam(InIndex, InSelf->ArrayProp->ArrayDim);
	const int ValidateIndexResult = PyUtil::ValidateContainerIndexParam(ResolvedIndex, InSelf->ArrayProp->ArrayDim, InSelf->ArrayProp, *PyUtil::GetErrorContext(InSelf));
	if (ValidateIndexResult != 0)
	{
		return ValidateIndexResult;
	}

	if (!PyConversion::NativizeProperty_Direct(InValue, InSelf->ArrayProp, GetItemPtr(InSelf, ResolvedIndex)))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert property '%s' (%s) at index %zd"), *InSelf->ArrayProp->GetName(), *InSelf->ArrayProp->GetClass()->GetName(), ResolvedIndex));
		return -1;
	}

	return 0;
}

int FPyWrapperFixedArray::Contains(FPyWrapperFixedArray* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FFixedArrayElementOnScope ContainerEntryValue(InSelf->ArrayProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	for (int32 ArrIndex = 0; ArrIndex < InSelf->ArrayProp->ArrayDim; ++ArrIndex)
	{
		if (ContainerEntryValue.GetProp()->Identical(GetItemPtr(InSelf, ArrIndex), ContainerEntryValue.GetValue()))
		{
			return 1;
		}
	}

	return 0;
}

FPyWrapperFixedArray* FPyWrapperFixedArray::Concat(FPyWrapperFixedArray* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	const PyUtil::FPropertyDef SelfPropDef = InSelf->ArrayProp.Get();
	FPyWrapperFixedArrayPtr Other = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::CastPyObject(InOther, &PyWrapperFixedArrayType, SelfPropDef));
	if (!Other)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot concatenate types '%s' and '%s' together"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(InOther)));
		return nullptr;
	}

	const int32 NewArrayLen = InSelf->ArrayProp->ArrayDim + Other->ArrayProp->ArrayDim;
	FPyWrapperFixedArrayPtr NewArray = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::New(Py_TYPE(InSelf)));
	if (FPyWrapperFixedArray::Init(NewArray, SelfPropDef, NewArrayLen) != 0)
	{
		return nullptr;
	}

	int32 NewArrayIndex = 0;
	for (int32 ArrIndex = 0; ArrIndex < InSelf->ArrayProp->ArrayDim; ++ArrIndex, ++NewArrayIndex)
	{
		InSelf->ArrayProp->CopySingleValue(GetItemPtr(NewArray, NewArrayIndex), GetItemPtr(InSelf, ArrIndex));
	}
	for (int32 ArrIndex = 0; ArrIndex < Other->ArrayProp->ArrayDim; ++ArrIndex, ++NewArrayIndex)
	{
		Other->ArrayProp->CopySingleValue(GetItemPtr(NewArray, NewArrayIndex), GetItemPtr(Other, ArrIndex));
	}
	return NewArray.Release();
}

FPyWrapperFixedArray* FPyWrapperFixedArray::Repeat(FPyWrapperFixedArray* InSelf, Py_ssize_t InMultiplier)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	if (InMultiplier <= 0)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Multiplier must be greater than zero"));
		return nullptr;
	}

	const PyUtil::FPropertyDef SelfPropDef = InSelf->ArrayProp.Get();
	const int32 NewArrayLen = InSelf->ArrayProp->ArrayDim * InMultiplier;
	FPyWrapperFixedArrayPtr NewArray = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::New(Py_TYPE(InSelf)));
	if (FPyWrapperFixedArray::Init(NewArray, SelfPropDef, NewArrayLen) != 0)
	{
		return nullptr;
	}

	int32 NewArrayIndex = 0;
	for (int32 MultipleIndex = 0; MultipleIndex < (int32)InMultiplier; ++MultipleIndex)
	{
		for (int32 ArrIndex = 0; ArrIndex < InSelf->ArrayProp->ArrayDim; ++ArrIndex, ++NewArrayIndex)
		{
			InSelf->ArrayProp->CopySingleValue(GetItemPtr(NewArray, NewArrayIndex), GetItemPtr(InSelf, ArrIndex));
		}
	}
	return NewArray.Release();
}

PyTypeObject InitializePyWrapperFixedArrayType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperFixedArray::New(InType);
		}

		static void Dealloc(FPyWrapperFixedArray* InSelf)
		{
			FPyWrapperFixedArray::Free(InSelf);
		}

		static int Init(FPyWrapperFixedArray* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;
			PyObject* PyLenObj = nullptr;

			static const char *ArgsKwdList[] = { "type", "len", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:call", (char**)ArgsKwdList, &PyTypeObj, &PyLenObj))
			{
				return -1;
			}

			PyUtil::FPropertyDef ArrayPropDef;
			const int ValidateTypeResult = PyUtil::ValidateContainerTypeParam(PyTypeObj, ArrayPropDef, "type", *PyUtil::GetErrorContext(Py_TYPE(InSelf)));
			if (ValidateTypeResult != 0)
			{
				return ValidateTypeResult;
			}

			int32 ArrayLen = 0;
			const int ValidateLenResult = PyUtil::ValidateContainerLenParam(PyLenObj, ArrayLen, "len", *PyUtil::GetErrorContext(Py_TYPE(InSelf)));
			if (ValidateLenResult != 0)
			{
				return ValidateLenResult;
			}

			ArrayLen = FMath::Max(ArrayLen, 1);
			return FPyWrapperFixedArray::Init(InSelf, ArrayPropDef, ArrayLen);
		}

		static PyObject* Str(FPyWrapperFixedArray* InSelf)
		{
			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FString ExportedArray;
			for (int32 ArrayIndex = 0; ArrayIndex < InSelf->ArrayProp->ArrayDim; ++ArrayIndex)
			{
				if (ArrayIndex > 0)
				{
					ExportedArray += TEXT(", ");
				}
				ExportedArray += PyUtil::GetFriendlyPropertyValue(InSelf->ArrayProp, FPyWrapperFixedArray::GetItemPtr(InSelf, ArrayIndex), PPF_Delimited | PPF_IncludeTransient);
			}
			return PyUnicode_FromFormat("[%s]", TCHAR_TO_UTF8(*ExportedArray));
		}

		static PyObject* RichCmp(FPyWrapperFixedArray* InSelf, PyObject* InOther, int InOp)
		{
			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyUtil::FPropertyDef SelfPropDef = InSelf->ArrayProp.Get();
			FPyWrapperFixedArrayPtr Other = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::CastPyObject(InOther, &PyWrapperFixedArrayType, SelfPropDef));
			if (!Other)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (InOp != Py_EQ && InOp != Py_NE)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Only == and != comparison is supported"));
				return nullptr;
			}

			bool bIsIdentical = InSelf->ArrayProp->ArrayDim == Other->ArrayProp->ArrayDim;
			for (int32 ArrayIndex = 0; bIsIdentical && ArrayIndex < InSelf->ArrayProp->ArrayDim; ++ArrayIndex)
			{
				bIsIdentical &= InSelf->ArrayProp->Identical_InContainer(InSelf->ArrayInstance, Other->ArrayInstance, ArrayIndex, PPF_None);
			}

			return PyBool_FromLong(InOp == Py_EQ ? bIsIdentical : !bIsIdentical);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperFixedArray* InSelf)
		{
			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return -1;
			}

			if (InSelf->ArrayProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
			{
				uint32 ArrayHash = 0;
				for (int32 ArrayIndex = 0; ArrayIndex < InSelf->ArrayProp->ArrayDim; ++ArrayIndex)
				{
					ArrayHash = HashCombine(ArrayHash, InSelf->ArrayProp->GetValueTypeHash(FPyWrapperFixedArray::GetItemPtr(InSelf, ArrayIndex)));
				}
				return (PyUtil::FPyHashType)(ArrayHash != -1 ? ArrayHash : 0);
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Type cannot be hashed"));
			return -1;
		}

		static PyObject* GetIter(FPyWrapperFixedArray* InSelf)
		{
			typedef TPyPtr<FPyWrapperFixedArrayIterator> FPyWrapperFixedArrayIteratorPtr;

			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FPyWrapperFixedArrayIteratorPtr NewIter = FPyWrapperFixedArrayIteratorPtr::StealReference(FPyWrapperFixedArrayIterator::New(&PyWrapperFixedArrayIteratorType));
			if (FPyWrapperFixedArrayIterator::Init(NewIter, InSelf) != 0)
			{
				return nullptr;
			}

			return (PyObject*)NewIter.Release();
		}
	};

	struct FSequenceFuncs
	{
		static Py_ssize_t Len(FPyWrapperFixedArray* InSelf)
		{
			return FPyWrapperFixedArray::Len(InSelf);
		}

		static PyObject* GetItem(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex)
		{
			return FPyWrapperFixedArray::GetItem(InSelf, InIndex);
		}

		static int SetItem(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex, PyObject* InValue)
		{
			return FPyWrapperFixedArray::SetItem(InSelf, InIndex, InValue);
		}

		static int Contains(FPyWrapperFixedArray* InSelf, PyObject* InValue)
		{
			return FPyWrapperFixedArray::Contains(InSelf, InValue);
		}

		static PyObject* Concat(FPyWrapperFixedArray* InSelf, PyObject* InOther)
		{
			return (PyObject*)FPyWrapperFixedArray::Concat(InSelf, InOther);
		}

		static PyObject* Repeat(FPyWrapperFixedArray* InSelf, Py_ssize_t InMultiplier)
		{
			return (PyObject*)FPyWrapperFixedArray::Repeat(InSelf, InMultiplier);
		}

		static PyObject* GetSlice(FPyWrapperFixedArray* InSelf, Py_ssize_t InSliceStart, Py_ssize_t InSliceStop)
		{
			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const Py_ssize_t ResolvedSliceStart = PyUtil::ResolveContainerIndexParam(InSliceStart, InSelf->ArrayProp->ArrayDim);
			const Py_ssize_t ResolvedSliceStop = PyUtil::ResolveContainerIndexParam(InSliceStop, InSelf->ArrayProp->ArrayDim);
			const Py_ssize_t SliceLen = ResolvedSliceStop - ResolvedSliceStart;
			if (SliceLen <= 0)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Slice length must be greater than zero"));
				return nullptr;
			}

			const PyUtil::FPropertyDef SelfPropDef = InSelf->ArrayProp.Get();
			FPyWrapperFixedArrayPtr NewArray = FPyWrapperFixedArrayPtr::StealReference(FPyWrapperFixedArray::New(Py_TYPE(InSelf)));
			if (FPyWrapperFixedArray::Init(NewArray, SelfPropDef, SliceLen) != 0)
			{
				return nullptr;
			}

			for (Py_ssize_t ArrIndex = ResolvedSliceStart; ArrIndex < ResolvedSliceStop; ++ArrIndex)
			{
				InSelf->ArrayProp->CopySingleValue(FPyWrapperFixedArray::GetItemPtr(NewArray, ArrIndex - ResolvedSliceStart), FPyWrapperFixedArray::GetItemPtr(InSelf, ArrIndex));
			}
			return (PyObject*)NewArray.Release();
		}
	};

	struct FMappingFuncs
	{
		static PyObject* GetItem(FPyWrapperFixedArray* InSelf, PyObject* InIndexer)
		{
			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			// Array types support numeric and slice based indexing
			int32 Index = 0;
			if (PyConversion::Nativize(InIndexer, Index, PyConversion::ESetErrorState::No))
			{
				return FPyWrapperFixedArray::GetItem(InSelf, (Py_ssize_t)Index);
			}

			if (PySlice_Check(InIndexer))
			{
				Py_ssize_t SliceStart = 0;
				Py_ssize_t SliceStop = 0;
				Py_ssize_t SliceStep = 0;
				Py_ssize_t SliceLen = 0;
				if (PySlice_GetIndicesEx((PyUtil::FPySliceType*)InIndexer, FPyWrapperFixedArray::Len(InSelf), &SliceStart, &SliceStop, &SliceStep, &SliceLen) != 0)
				{
					return nullptr;
				}
				if (SliceStep == 1)
				{
					return FSequenceFuncs::GetSlice(InSelf, SliceStart, SliceStop);
				}
				else
				{
					PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Slice step must be 1"));
					return nullptr;
				}
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Indexer '%s' was invalid for '%s' (%s)"), *PyUtil::GetFriendlyTypename(InIndexer), *InSelf->ArrayProp->GetName(), *InSelf->ArrayProp->GetClass()->GetName()));
			return nullptr;
		}
	};

	struct FMethods
	{
		// This is for type hinting, to make the class appears as a generic (https://peps.python.org/pep-0560/) and support hinting like 'a: unreal.FixedArray[int] = unreal.FixedArray(int)'
		static PyObject* GetClassItem(PyObject* Type, PyObject* Item)
		{
			Py_INCREF(Type);
			return Type;
		}

		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;
			PyObject* PyObj = nullptr;

			static const char *ArgsKwdList[] = { "type", "obj", nullptr };
			if (PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:call", (char**)ArgsKwdList, &PyTypeObj, &PyObj))
			{
				PyUtil::FPropertyDef ArrayPropDef;
				if (PyUtil::ValidateContainerTypeParam(PyTypeObj, ArrayPropDef, "type", *PyUtil::GetErrorContext(InType)) != 0)
				{
					return nullptr;
				}

				PyObject* PyCastResult = (PyObject*)FPyWrapperFixedArray::CastPyObject(PyObj, InType, ArrayPropDef);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperFixedArray* InSelf)
		{
			if (!FPyWrapperFixedArray::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperFixedArrayFactory::Get().CreateInstance(InSelf->ArrayInstance, InSelf->ArrayProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}
	};

	// NOTE: _ElemType = typing.TypeVar('_ElemType') and TYpe is defined in the typing Python module.
	static PyMethodDef PyMethods[] = {
		{ "__class_getitem__", PyCFunctionCast(FMethods::GetClassItem), METH_O|METH_CLASS, "__class_getitem__(cls, item: _ElemType) -- implemented for type hinting purpose only" },
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_KEYWORDS | METH_CLASS, "cast(cls, type: Type[_ElemType], obj: object) -> FixedArray[_ElemType] -- cast the given object to this Unreal fixed-array type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> FixedArray[_ElemType] -- copy this Unreal fixed-array" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> FixedArray[_ElemType] -- copy this Unreal fixed-array" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"FixedArray", /* tp_name */
		sizeof(FPyWrapperFixedArray), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed fixed-array instances";

	static PySequenceMethods PySequence;
	PySequence.sq_length = (lenfunc)&FSequenceFuncs::Len;
	PySequence.sq_item = (ssizeargfunc)&FSequenceFuncs::GetItem;
	PySequence.sq_ass_item = (ssizeobjargproc)&FSequenceFuncs::SetItem;
	PySequence.sq_contains = (objobjproc)&FSequenceFuncs::Contains;
	PySequence.sq_concat = (binaryfunc)&FSequenceFuncs::Concat;
	PySequence.sq_repeat = (ssizeargfunc)&FSequenceFuncs::Repeat;

	static PyMappingMethods PyMapping;
	PyMapping.mp_subscript = (binaryfunc)&FMappingFuncs::GetItem;

	PyType.tp_as_sequence = &PySequence;
	PyType.tp_as_mapping = &PyMapping;

	return PyType;
}

PyTypeObject InitializePyWrapperFixedArrayIteratorType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperFixedArrayIterator::New(InType);
		}

		static void Dealloc(FPyWrapperFixedArrayIterator* InSelf)
		{
			FPyWrapperFixedArrayIterator::Free(InSelf);
		}

		static int Init(FPyWrapperFixedArrayIterator* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				return -1;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperFixedArrayType) != 1)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot initialize '%s' with an instance of '%s'"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return FPyWrapperFixedArrayIterator::Init(InSelf, (FPyWrapperFixedArray*)PyObj);
		}

		static FPyWrapperFixedArrayIterator* GetIter(FPyWrapperFixedArrayIterator* InSelf)
		{
			return FPyWrapperFixedArrayIterator::GetIter(InSelf);
		}

		static PyObject* IterNext(FPyWrapperFixedArrayIterator* InSelf)
		{
			return FPyWrapperFixedArrayIterator::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"FixedArrayIterator", /* tp_name */
		sizeof(FPyWrapperFixedArrayIterator), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed fixed-array iterators";

	return PyType;
}

PyTypeObject PyWrapperFixedArrayType = InitializePyWrapperFixedArrayType();
PyTypeObject PyWrapperFixedArrayIteratorType = InitializePyWrapperFixedArrayIteratorType();

void FPyWrapperFixedArrayMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperFixedArray* Self = static_cast<FPyWrapperFixedArray*>(Instance);
	if (Self->ArrayProp && Self->ArrayInstance && !Self->OwnerContext.HasOwner())
	{
		Self->ArrayProp.AddReferencedObjects(Collector);
		FPyReferenceCollector::AddReferencedObjectsFromProperty(Collector, Self->ArrayProp, Self->ArrayInstance);
	}
}

#endif	// WITH_PYTHON
