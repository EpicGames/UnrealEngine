// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "PyWrapperOwnerContext.h"
#include "PyUtil.h"

#if WITH_PYTHON

/** Python type for FPyWrapperFixedArray */
extern PyTypeObject PyWrapperFixedArrayType;

/** Initialize the PyWrapperFixedArray types and add them to the given Python module */
void InitializePyWrapperFixedArray(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed fixed-array instances */
struct FPyWrapperFixedArray : public FPyWrapperBase
{
	/** The owner of the wrapped fixed-array instance (if any) */
	FPyWrapperOwnerContext OwnerContext;

	/** Property describing the fixed-array */
	PyUtil::FConstPropOnScope ArrayProp;

	/** Wrapped fixed-array instance */
	void* ArrayInstance;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperFixedArray* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperFixedArray* InSelf);

	/** Initialize this wrapper (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperFixedArray* InSelf, const PyUtil::FPropertyDef& InPropDef, const int32 InLen);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperFixedArray* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperFixedArray* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperFixedArray* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperFixedArray* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperFixedArray* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InPropDef, FPyConversionResult* OutCastResult = nullptr);

	/** Get the raw pointer to the element at index N (negative indexing not supported) */
	static void* GetItemPtr(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex);

	/** Get the length of this container (equivalent to 'len(x)' in Python) */
	static Py_ssize_t Len(FPyWrapperFixedArray* InSelf);

	/** Get the item at index N (equivalent to 'x[N]' in Python, returns new reference) */
	static PyObject* GetItem(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex);

	/** Set the item at index N (equivalent to 'x[N] = v' in Python) */
	static int SetItem(FPyWrapperFixedArray* InSelf, Py_ssize_t InIndex, PyObject* InValue);

	/** Does this container have an entry with the given value? (equivalent to 'v in x' in Python) */
	static int Contains(FPyWrapperFixedArray* InSelf, PyObject* InValue);

	/** Concatenate the other object to this one, returning a new container (equivalent to 'x + o' in Python, returns new reference) */
	static FPyWrapperFixedArray* Concat(FPyWrapperFixedArray* InSelf, PyObject* InOther);

	/** Repeat this container by N, returning a new container (equivalent to 'x * N' in Python, returns new reference) */
	static FPyWrapperFixedArray* Repeat(FPyWrapperFixedArray* InSelf, Py_ssize_t InMultiplier);
};

/** Meta-data for all Unreal exposed fixed-array types */
struct FPyWrapperFixedArrayMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperFixedArrayMetaData, FGuid(0x8A8CA89E, 0xC8F04627, 0xA7D2B568, 0xB086C02E))

	FPyWrapperFixedArrayMetaData()
	{
	}

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;
};

typedef TPyPtr<FPyWrapperFixedArray> FPyWrapperFixedArrayPtr;

#endif	// WITH_PYTHON
