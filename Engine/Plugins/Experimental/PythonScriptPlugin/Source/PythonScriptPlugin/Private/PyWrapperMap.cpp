// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperMap.h"
#include "PyWrapperTypeRegistry.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"

#if WITH_PYTHON

/** Python type for FPyWrapperMapItemIterator */
extern PyTypeObject PyWrapperMapItemIteratorType;

/** Python type for FPyWrapperMapKeyIterator */
extern PyTypeObject PyWrapperMapKeyIteratorType;

/** Python type for FPyWrapperMapValueIterator */
extern PyTypeObject PyWrapperMapValueIteratorType;

/** Python type for FPyWrapperMapItemView */
extern PyTypeObject PyWrapperMapItemViewType;

/** Python type for FPyWrapperMapKeyView */
extern PyTypeObject PyWrapperMapKeyViewType;

/** Python type for FPyWrapperMapValueView */
extern PyTypeObject PyWrapperMapValueViewType;

/** Helper function to make a new map iterator */
template <typename TImpl>
PyObject* MakeMapIter(FPyWrapperMap* InSelf)
{
	typedef TPyPtr<TImpl> FMapIteratorPtr;

	if (!FPyWrapperMap::ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FMapIteratorPtr NewIter = FMapIteratorPtr::StealReference(TImpl::New(TImpl::GetType()));
	if (TImpl::Init(NewIter, InSelf) != 0)
	{
		return nullptr;
	}

	return (PyObject*)NewIter.Release();
}

/** Helper function to make a new Python list from a map iterator */
template <typename TImpl>
PyObject* MakeListFromMapIter(FPyWrapperMap* InSelf)
{
	FPyObjectPtr IterObj = FPyObjectPtr::StealReference(MakeMapIter<TImpl>(InSelf));
	if (!IterObj)
	{
		return nullptr;
	}

	const Py_ssize_t MapLen = FPyWrapperMap::Len(InSelf);
	FPyObjectPtr ListObj = FPyObjectPtr::StealReference(PyList_New(MapLen));

	for (Py_ssize_t MapIndex = 0; MapIndex < MapLen; ++MapIndex)
	{
		PyObject* MapItem = PyIter_Next(IterObj);
		if (!MapItem)
		{
			return nullptr;
		}

		PyList_SetItem(ListObj, MapIndex, MapItem); // PyList_SetItem steals the new reference returned by PyIter_Next
	}

	return ListObj.Release();
}

/** Helper function to make a new map view */
template <typename TImpl>
PyObject* MakeMapView(FPyWrapperMap* InSelf)
{
	typedef TPyPtr<TImpl> FMapViewPtr;

	if (!FPyWrapperMap::ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FMapViewPtr NewView = FMapViewPtr::StealReference(TImpl::New(TImpl::GetType()));
	if (TImpl::Init(NewView, InSelf) != 0)
	{
		return nullptr;
	}

	return (PyObject*)NewView.Release();
}

/** Iterator used with maps */
template <typename TImpl>
struct TPyWrapperMapIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Instance being iterated over */
	FPyWrapperMap* IterInstance;

	/** Current iteration index */
	int32 IterIndex;

	/** New this iterator instance (called via tp_new for Python, or directly in C++) */
	static TImpl* New(PyTypeObject* InType)
	{
		TImpl* Self = (TImpl*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->IterInstance = nullptr;
			Self->IterIndex = 0;
		}
		return Self;
	}

	/** Free this iterator instance (called via tp_dealloc for Python) */
	static void Free(TImpl* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this iterator instance (called via tp_init for Python, or directly in C++) */
	static int Init(TImpl* InSelf, FPyWrapperMap* InInstance)
	{
		Deinit(InSelf);

		Py_INCREF(InInstance);
		InSelf->IterInstance = InInstance;
		InSelf->IterIndex = GetElementIndex(InSelf, 0);

		return 0;
	}

	/** Deinitialize this iterator instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(TImpl* InSelf)
	{
		Py_XDECREF(InSelf->IterInstance);
		InSelf->IterInstance = nullptr;
		InSelf->IterIndex = 0;
	}

	/** Called to validate the internal state of this iterator instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(TImpl* InSelf)
	{
		if (!InSelf->IterInstance)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - IterInstance is null!"));
			return false;
		}

		return true;
	}

	/** Given a sparse index, get the first element index from this point in the map (including the given index) */
	static int32 GetElementIndex(TImpl* InSelf, int32 InSparseIndex)
	{
		FScriptMapHelper ScriptMapHelper(InSelf->IterInstance->MapProp, InSelf->IterInstance->MapInstance);
		const int32 SparseCount = ScriptMapHelper.GetMaxIndex();

		int32 ElementIndex = InSparseIndex;
		for (; ElementIndex < SparseCount; ++ElementIndex)
		{
			if (ScriptMapHelper.IsValidIndex(ElementIndex))
			{
				break;
			}
		}

		return ElementIndex;
	}

	/** Get the iterator */
	static PyObject* GetIter(TImpl* InSelf)
	{
		Py_INCREF(InSelf);
		return (PyObject*)InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(TImpl* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		FScriptMapHelper ScriptMapHelper(InSelf->IterInstance->MapProp, InSelf->IterInstance->MapInstance);
		const int32 SparseCount = ScriptMapHelper.GetMaxIndex();

		if (InSelf->IterIndex < SparseCount)
		{
			const int32 ElementIndex = InSelf->IterIndex;
			InSelf->IterIndex = GetElementIndex(InSelf, InSelf->IterIndex + 1);

			if (!ScriptMapHelper.IsValidIndex(ElementIndex))
			{
				PyUtil::SetPythonError(PyExc_IndexError, InSelf, TEXT("Iterator was on an invalid element index! Was the map changed while iterating?"));
				return nullptr;
			}

			return TImpl::GetItem(InSelf, ScriptMapHelper, ElementIndex);
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}
};

/** Iterator used with map items (key->value pairs) */
struct FPyWrapperMapItemIterator : public TPyWrapperMapIterator<FPyWrapperMapItemIterator>
{
	static PyTypeObject* GetType()
	{
		return &PyWrapperMapItemIteratorType;
	}

	static PyObject* GetItem(FPyWrapperMapItemIterator* InSelf, FScriptMapHelper& InScriptMapHelper, const int32 InValidatedInternalIndex)
	{
		FPyObjectPtr PyKeyObj;
		if (!PyConversion::PythonizeProperty(InScriptMapHelper.GetKeyProperty(), InScriptMapHelper.GetKeyPtr(InValidatedInternalIndex), PyKeyObj.Get()))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert key property '%s' (%s)"), *InScriptMapHelper.GetKeyProperty()->GetName(), *InScriptMapHelper.GetKeyProperty()->GetClass()->GetName()));
			return nullptr;
		}

		FPyObjectPtr PyValueObj;
		if (!PyConversion::PythonizeProperty(InScriptMapHelper.GetValueProperty(), InScriptMapHelper.GetValuePtr(InValidatedInternalIndex), PyValueObj.Get()))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert value property '%s' (%s)"), *InScriptMapHelper.GetValueProperty()->GetName(), *InScriptMapHelper.GetValueProperty()->GetClass()->GetName()));
			return nullptr;
		}

		return PyTuple_Pack(2, PyKeyObj.Get(), PyValueObj.Get());
	}
};

/** Iterator used with map keys */
struct FPyWrapperMapKeyIterator : public TPyWrapperMapIterator<FPyWrapperMapKeyIterator>
{
	static PyTypeObject* GetType()
	{
		return &PyWrapperMapKeyIteratorType;
	}

	static PyObject* GetItem(FPyWrapperMapKeyIterator* InSelf, FScriptMapHelper& InScriptMapHelper, const int32 InValidatedInternalIndex)
	{
		PyObject* PyItemObj = nullptr;
		if (!PyConversion::PythonizeProperty(InScriptMapHelper.GetKeyProperty(), InScriptMapHelper.GetKeyPtr(InValidatedInternalIndex), PyItemObj))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert key property '%s' (%s)"), *InScriptMapHelper.GetKeyProperty()->GetName(), *InScriptMapHelper.GetKeyProperty()->GetClass()->GetName()));
			return nullptr;
		}
		return PyItemObj;
	}
};

/** Iterator used with map values */
struct FPyWrapperMapValueIterator : public TPyWrapperMapIterator<FPyWrapperMapValueIterator>
{
	static PyTypeObject* GetType()
	{
		return &PyWrapperMapValueIteratorType;
	}

	static PyObject* GetItem(FPyWrapperMapValueIterator* InSelf, FScriptMapHelper& InScriptMapHelper, const int32 InValidatedInternalIndex)
	{
		PyObject* PyItemObj = nullptr;
		if (!PyConversion::PythonizeProperty(InScriptMapHelper.GetValueProperty(), InScriptMapHelper.GetValuePtr(InValidatedInternalIndex), PyItemObj))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert value property '%s' (%s)"), *InScriptMapHelper.GetValueProperty()->GetName(), *InScriptMapHelper.GetValueProperty()->GetClass()->GetName()));
			return nullptr;
		}
		return PyItemObj;
	}
};

/** View used with maps */
template <typename TImpl>
struct TPyWrapperMapView
{
	/** Common Python Object */
	PyObject_HEAD

	/** Instance being viewed */
	FPyWrapperMap* ViewInstance;

	/** New this view instance (called via tp_new for Python, or directly in C++) */
	static TImpl* New(PyTypeObject* InType)
	{
		TImpl* Self = (TImpl*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->ViewInstance = nullptr;
		}
		return Self;
	}

	/** Free this view instance (called via tp_dealloc for Python) */
	static void Free(TImpl* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this view instance (called via tp_init for Python, or directly in C++) */
	static int Init(TImpl* InSelf, FPyWrapperMap* InInstance)
	{
		Deinit(InSelf);

		Py_INCREF(InInstance);
		InSelf->ViewInstance = InInstance;

		return 0;
	}

	/** Deinitialize this view instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(TImpl* InSelf)
	{
		Py_XDECREF(InSelf->ViewInstance);
		InSelf->ViewInstance = nullptr;
	}

	/** Called to validate the internal state of this view instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(TImpl* InSelf)
	{
		if (!InSelf->ViewInstance)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ViewInstance is null!"));
			return false;
		}

		return true;
	}

	/** Called to get the length of the map we're a view to */
	static Py_ssize_t Len(TImpl* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return -1;
		}

		return FPyWrapperMap::Len(InSelf->ViewInstance);
	}
};

/** View used with map items (key->value pairs) */
struct FPyWrapperMapItemView : public TPyWrapperMapView<FPyWrapperMapItemView>
{
	static PyTypeObject* GetType()
	{
		return &PyWrapperMapItemViewType;
	}

	static PyObject* GetIter(FPyWrapperMapItemView* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		return MakeMapIter<FPyWrapperMapItemIterator>(InSelf->ViewInstance);
	}
};

/** View used with map keys */
struct FPyWrapperMapKeyView : public TPyWrapperMapView<FPyWrapperMapKeyView>
{
	static PyTypeObject* GetType()
	{
		return &PyWrapperMapKeyViewType;
	}

	static PyObject* GetIter(FPyWrapperMapKeyView* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		return MakeMapIter<FPyWrapperMapKeyIterator>(InSelf->ViewInstance);
	}
};

/** View used with map values */
struct FPyWrapperMapValueView : public TPyWrapperMapView<FPyWrapperMapValueView>
{
	static PyTypeObject* GetType()
	{
		return &PyWrapperMapValueViewType;
	}

	static PyObject* GetIter(FPyWrapperMapValueView* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		return MakeMapIter<FPyWrapperMapValueIterator>(InSelf->ViewInstance);
	}
};

void InitializePyWrapperMap(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperMapType) == 0)
	{
		static FPyWrapperMapMetaData MetaData;
		FPyWrapperMapMetaData::SetMetaData(&PyWrapperMapType, &MetaData);
		ModuleInfo.AddType(&PyWrapperMapType);
	}

	PyType_Ready(&PyWrapperMapItemIteratorType);
	PyType_Ready(&PyWrapperMapKeyIteratorType);
	PyType_Ready(&PyWrapperMapValueIteratorType);

	PyType_Ready(&PyWrapperMapItemViewType);
	PyType_Ready(&PyWrapperMapKeyViewType);
	PyType_Ready(&PyWrapperMapValueViewType);
}

FPyWrapperMap* FPyWrapperMap::New(PyTypeObject* InType)
{
	FPyWrapperMap* Self = (FPyWrapperMap*)FPyWrapperBase::New(InType);
	if (Self)
	{
		new(&Self->OwnerContext) FPyWrapperOwnerContext();
		new(&Self->MapProp) PyUtil::FConstMapPropOnScope();
		Self->MapInstance = nullptr;
	}
	return Self;
}

void FPyWrapperMap::Free(FPyWrapperMap* InSelf)
{
	Deinit(InSelf);

	InSelf->OwnerContext.~FPyWrapperOwnerContext();
	InSelf->MapProp.~TPropOnScope();
	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperMap::Init(FPyWrapperMap* InSelf, const PyUtil::FPropertyDef& InKeyDef, const PyUtil::FPropertyDef& InValueDef)
{
	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	PyUtil::FPropOnScope MapKeyProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InKeyDef, 1));
	if (!MapKeyProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Map key property was null during init"));
		return -1;
	}

	PyUtil::FPropOnScope MapValueProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InValueDef, 1));
	if (!MapValueProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Map value property was null during init"));
		return -1;
	}

	PyUtil::FMapPropOnScope MapProp = PyUtil::FMapPropOnScope::OwnedReference(new FMapProperty(FFieldVariant(), PyUtil::DefaultPythonPropertyName));
	MapProp->KeyProp = MapKeyProp.Release();
	MapProp->ValueProp = MapValueProp.Release();

	// Need to manually call Link to fix-up some data (such as the C++ property flags and the map layout) that are only set during Link
	{
		FArchive Ar;
		MapProp->LinkWithoutChangingOffset(Ar);
	}

	void* MapValue = FMemory::Malloc(MapProp->GetSize(), MapProp->GetMinAlignment());
	MapProp->InitializeValue(MapValue);

	InSelf->MapProp = MoveTemp(MapProp);
	InSelf->MapInstance = MapValue;

	FPyWrapperMapFactory::Get().MapInstance(InSelf->MapInstance, InSelf);
	return 0;
}

int FPyWrapperMap::Init(FPyWrapperMap* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FMapProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod)
{
	InOwnerContext.AssertValidConversionMethod(InConversionMethod);

	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	check(InProp && InValue);

	PyUtil::FConstMapPropOnScope PropToUse;
	void* MapInstanceToUse = nullptr;
	switch (InConversionMethod)
	{
	case EPyConversionMethod::Copy:
	case EPyConversionMethod::Steal:
		{
			PyUtil::FPropOnScope MapKeyProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InProp->KeyProp, 1));
			if (!MapKeyProp)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to create key property from '%s' (%s)"), *InProp->KeyProp->GetName(), *InProp->KeyProp->GetClass()->GetName()));
				return -1;
			}

			PyUtil::FPropOnScope MapValueProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InProp->ValueProp, 1));
			if (!MapValueProp)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to create value property from '%s' (%s)"), *InProp->ValueProp->GetName(), *InProp->ValueProp->GetClass()->GetName()));
				return -1;
			}

			PyUtil::FMapPropOnScope MapProp = PyUtil::FMapPropOnScope::OwnedReference(new FMapProperty(FFieldVariant(), PyUtil::DefaultPythonPropertyName));
			MapProp->KeyProp = MapKeyProp.Release();
			MapProp->ValueProp = MapValueProp.Release();

			// Need to manually call Link to fix-up some data (such as the C++ property flags and the map layout) that are only set during Link
			{
				FArchive Ar;
				MapProp->LinkWithoutChangingOffset(Ar);
			}

			PropToUse = MoveTemp(MapProp);

			MapInstanceToUse = FMemory::Malloc(PropToUse->GetSize(), PropToUse->GetMinAlignment());
			PropToUse->InitializeValue(MapInstanceToUse);
			if (InConversionMethod == EPyConversionMethod::Steal)
			{
				FScriptMapHelper SelfScriptMapHelper(PropToUse, MapInstanceToUse);
				SelfScriptMapHelper.MoveAssign(InValue);
			}
			else
			{
				PropToUse->CopyCompleteValue(MapInstanceToUse, InValue);
			}
		}
		break;

	case EPyConversionMethod::Reference:
		{
			PropToUse = PyUtil::FConstMapPropOnScope::ExternalReference(InProp);
			MapInstanceToUse = InValue;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EPyConversionMethod"));
		break;
	}

	check(PropToUse && MapInstanceToUse);

	InSelf->OwnerContext = InOwnerContext;
	InSelf->MapProp = MoveTemp(PropToUse);
	InSelf->MapInstance = MapInstanceToUse;

	FPyWrapperMapFactory::Get().MapInstance(InSelf->MapInstance, InSelf);
	return 0;
}

void FPyWrapperMap::Deinit(FPyWrapperMap* InSelf)
{
	if (InSelf->MapInstance)
	{
		FPyWrapperMapFactory::Get().UnmapInstance(InSelf->MapInstance, Py_TYPE(InSelf));
	}

	if (InSelf->OwnerContext.HasOwner())
	{
		InSelf->OwnerContext.Reset();
	}
	else if (InSelf->MapInstance)
	{
		if (InSelf->MapProp)
		{
			InSelf->MapProp->DestroyValue(InSelf->MapInstance);
		}
		FMemory::Free(InSelf->MapInstance);
	}
	InSelf->MapProp.Reset();
	InSelf->MapInstance = nullptr;
}

bool FPyWrapperMap::ValidateInternalState(FPyWrapperMap* InSelf)
{
	if (!InSelf->MapProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - MapProp is null!"));
		return false;
	}

	if (!InSelf->MapInstance)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - MapInstance is null!"));
		return false;
	}

	return true;
}

FPyWrapperMap* FPyWrapperMap::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperMapType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperMap*)InPyObject;
	}

	return nullptr;
}

FPyWrapperMap* FPyWrapperMap::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InKeyDef, const PyUtil::FPropertyDef& InValueDef, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperMapType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperMapType) == 1))
	{
		FPyWrapperMap* Self = (FPyWrapperMap*)InPyObject;

		if (!ValidateInternalState(Self))
		{
			return nullptr;
		}

		const PyUtil::FPropertyDef SelfKeyPropDef = Self->MapProp->KeyProp;
		const PyUtil::FPropertyDef SelfValuePropDef = Self->MapProp->ValueProp;
		if (SelfKeyPropDef == InKeyDef && SelfValuePropDef == InValueDef)
		{
			SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

			Py_INCREF(Self);
			return Self;
		}

		FPyWrapperMapPtr NewMap = FPyWrapperMapPtr::StealReference(FPyWrapperMap::New(InType));
		if (FPyWrapperMap::Init(NewMap, InKeyDef, InValueDef) != 0)
		{
			return nullptr;
		}

		// Attempt to convert the entries in the map to the native format of the new map
		{
			const FScriptMapHelper SelfScriptMapHelper(Self->MapProp, Self->MapInstance);
			FScriptMapHelper NewScriptMapHelper(NewMap->MapProp, NewMap->MapInstance);

			FString ExportedKey;
			FString ExportedValue;
			for (FScriptMapHelper::FIterator It(SelfScriptMapHelper); It; ++It)
			{
				ExportedKey.Reset();
				if (!SelfScriptMapHelper.GetKeyProperty()->ExportText_Direct(ExportedKey, SelfScriptMapHelper.GetKeyPtr(It), SelfScriptMapHelper.GetKeyPtr(It), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to export text for key property '%s' (%s) at index %d"), *SelfScriptMapHelper.GetKeyProperty()->GetName(), *SelfScriptMapHelper.GetKeyProperty()->GetClass()->GetName(), It.GetLogicalIndex()));
					return nullptr;
				}

				ExportedValue.Reset();
				if (!SelfScriptMapHelper.GetValueProperty()->ExportText_Direct(ExportedValue, SelfScriptMapHelper.GetValuePtr(It), SelfScriptMapHelper.GetValuePtr(It), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to export text for value property '%s' (%s) at index %d"), *SelfScriptMapHelper.GetValueProperty()->GetName(), *SelfScriptMapHelper.GetValueProperty()->GetClass()->GetName(), It.GetLogicalIndex()));
					return nullptr;
				}

				const int32 NewElementIndex = NewScriptMapHelper.AddDefaultValue_Invalid_NeedsRehash();
				if (!NewScriptMapHelper.GetKeyProperty()->ImportText_Direct(*ExportedKey, NewScriptMapHelper.GetKeyPtr(NewElementIndex), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to import text '%s' key for property '%s' (%s) at index %d"), *ExportedKey, *NewScriptMapHelper.GetKeyProperty()->GetName(), *NewScriptMapHelper.GetKeyProperty()->GetClass()->GetName(), NewElementIndex));
					return nullptr;
				}
				if (!NewScriptMapHelper.GetValueProperty()->ImportText_Direct(*ExportedValue, NewScriptMapHelper.GetValuePtr(NewElementIndex), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to import text '%s' value for property '%s' (%s) at index %d"), *ExportedValue, *NewScriptMapHelper.GetValueProperty()->GetName(), *NewScriptMapHelper.GetValueProperty()->GetClass()->GetName(), NewElementIndex));
					return nullptr;
				}
			}

			NewScriptMapHelper.Rehash();
		}

		SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
		return NewMap.Release();
	}

	// Attempt conversion from any iterable type that has a defined length
	if (PyUtil::HasLength(InPyObject))
	{
		const Py_ssize_t SequenceLen = PyObject_Length(InPyObject);
		if (SequenceLen != -1)
		{
			FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(InPyObject));
			if (PyObjIter)
			{
				FPyWrapperMapPtr NewMap = FPyWrapperMapPtr::StealReference(FPyWrapperMap::New(InType));
				if (FPyWrapperMap::Init(NewMap, InKeyDef, InValueDef) != 0)
				{
					return nullptr;
				}

				FScriptMapHelper NewScriptMapHelper(NewMap->MapProp, NewMap->MapInstance);

				if (PyUtil::IsMappingType(InPyObject))
				{
					// Conversion from a mapping type
					for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
					{
						FPyObjectPtr KeyItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
						if (!KeyItem)
						{
							return nullptr;
						}

						FPyObjectPtr ValueItem = FPyObjectPtr::StealReference(PyObject_GetItem(InPyObject, KeyItem));
						if (!ValueItem)
						{
							return nullptr;
						}

						const int32 NewElementIndex = NewScriptMapHelper.AddDefaultValue_Invalid_NeedsRehash();
						if (!PyConversion::NativizeProperty(KeyItem, NewScriptMapHelper.GetKeyProperty(), NewScriptMapHelper.GetKeyPtr(NewElementIndex)))
						{
							return nullptr;
						}
						if (!PyConversion::NativizeProperty(ValueItem, NewScriptMapHelper.GetValueProperty(), NewScriptMapHelper.GetValuePtr(NewElementIndex)))
						{
							return nullptr;
						}
					}
				}
				else
				{
					// Conversion from a sequence of pairs
					for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
					{
						FPyObjectPtr PairItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
						if (!PairItem)
						{
							return nullptr;
						}

						FPyObjectPtr PairSequence = FPyObjectPtr::StealReference(PySequence_Fast(PairItem, ""));
						if (!PairSequence)
						{
							PyUtil::SetPythonError(PyExc_TypeError, NewMap.Get(), *FString::Printf(TEXT("Failed to convert item at index %zd to a sequence"), SequenceIndex));
							return nullptr;
						}

						const Py_ssize_t PairLen = PySequence_Fast_GET_SIZE(PairSequence.Get());
						if (PairLen != 2)
						{
							PyUtil::SetPythonError(PyExc_TypeError, NewMap.Get(), *FString::Printf(TEXT("Failed to convert item at index %zd as it was not a pair (len != 2)"), SequenceIndex));
							return nullptr;
						}

						const int32 NewElementIndex = NewScriptMapHelper.AddDefaultValue_Invalid_NeedsRehash();
						if (!PyConversion::NativizeProperty(PySequence_Fast_GET_ITEM(PairSequence.Get(), 0), NewScriptMapHelper.GetKeyProperty(), NewScriptMapHelper.GetKeyPtr(NewElementIndex)))
						{
							return nullptr;
						}
						if (!PyConversion::NativizeProperty(PySequence_Fast_GET_ITEM(PairSequence.Get(), 1), NewScriptMapHelper.GetValueProperty(), NewScriptMapHelper.GetValuePtr(NewElementIndex)))
						{
							return nullptr;
						}
					}
				}

				NewScriptMapHelper.Rehash();

				SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
				return NewMap.Release();
			}
		}
	}

	return nullptr;
}

Py_ssize_t FPyWrapperMap::Len(FPyWrapperMap* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);
	return SelfScriptMapHelper.Num();
}

PyObject* FPyWrapperMap::GetItem(FPyWrapperMap* InSelf, PyObject* InKey)
{
	return GetItem(InSelf, InKey, nullptr);
}

PyObject* FPyWrapperMap::GetItem(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InDefault)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	PyUtil::FMapKeyOnScope MapKey(InSelf->MapProp);
	if (!MapKey.IsValid())
	{
		return nullptr;
	}

	if (!MapKey.SetValue(InKey, *PyUtil::GetErrorContext(InSelf)))
	{
		return nullptr;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);

	const void* ValuePtr = SelfScriptMapHelper.FindValueFromHash(MapKey.GetValue());
	if (!ValuePtr)
	{
		if (!InDefault)
		{
			PyUtil::SetPythonError(PyExc_KeyError, InSelf, *FString::Printf(TEXT("Key '%s' was not found in the map"), *PyUtil::PyObjectToUEString(InKey)));
			return nullptr;
		}

		Py_INCREF(InDefault);
		return InDefault;
	}

	PyObject* PyItemObj = nullptr;
	if (!PyConversion::PythonizeProperty(SelfScriptMapHelper.GetValueProperty(), ValuePtr, PyItemObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert value property '%s' (%s) for key '%s'"), *SelfScriptMapHelper.GetValueProperty()->GetName(), *SelfScriptMapHelper.GetValueProperty()->GetClass()->GetName(), *PyUtil::PyObjectToUEString(InKey)));
		return nullptr;
	}
	return PyItemObj;
}

int FPyWrapperMap::SetItem(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FMapKeyOnScope MapKey(InSelf->MapProp);
	if (!MapKey.IsValid())
	{
		return -1;
	}

	if (!MapKey.SetValue(InKey, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);
	if (InValue)
	{
		PyUtil::FMapValueOnScope MapValue(InSelf->MapProp);
		if (!MapValue.IsValid())
		{
			return -1;
		}

		if (!MapValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
		{
			return -1;
		}

		SelfScriptMapHelper.AddPair(MapKey.GetValue(), MapValue.GetValue());
	}
	else
	{
		SelfScriptMapHelper.RemovePair(MapKey.GetValue());
	}

	return 0;
}

PyObject* FPyWrapperMap::SetDefault(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	PyUtil::FMapKeyOnScope MapKey(InSelf->MapProp);
	if (!MapKey.IsValid())
	{
		return nullptr;
	}

	if (!MapKey.SetValue(InKey, *PyUtil::GetErrorContext(InSelf)))
	{
		return nullptr;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);

	const void* ValuePtr = SelfScriptMapHelper.FindValueFromHash(MapKey.GetValue());
	if (!ValuePtr)
	{
		PyUtil::FMapValueOnScope MapValue(InSelf->MapProp);
		if (!MapValue.IsValid())
		{
			return nullptr;
		}

		if (InValue && !MapValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
		{
			return nullptr;
		}

		SelfScriptMapHelper.AddPair(MapKey.GetValue(), MapValue.GetValue());
		
		ValuePtr = SelfScriptMapHelper.FindValueFromHash(MapKey.GetValue());
		check(ValuePtr);
	}

	PyObject* PyItemObj = nullptr;
	if (!PyConversion::PythonizeProperty(SelfScriptMapHelper.GetValueProperty(), ValuePtr, PyItemObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert value property '%s' (%s) for key '%s'"), *SelfScriptMapHelper.GetValueProperty()->GetName(), *SelfScriptMapHelper.GetValueProperty()->GetClass()->GetName(), *PyUtil::PyObjectToUEString(InKey)));
		return nullptr;
	}
	return PyItemObj;
}

int FPyWrapperMap::Contains(FPyWrapperMap* InSelf, PyObject* InKey)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FMapKeyOnScope MapKey(InSelf->MapProp);
	if (!MapKey.IsValid())
	{
		return -1;
	}

	if (!MapKey.SetValue(InKey, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);
	return SelfScriptMapHelper.FindValueFromHash(MapKey.GetValue()) ? 1 : 0;
}

int FPyWrapperMap::Clear(FPyWrapperMap* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);
	SelfScriptMapHelper.EmptyValues();
	
	return 0;
}

PyObject* FPyWrapperMap::Pop(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InDefault)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	PyUtil::FMapKeyOnScope MapKey(InSelf->MapProp);
	if (!MapKey.IsValid())
	{
		return nullptr;
	}

	if (!MapKey.SetValue(InKey, *PyUtil::GetErrorContext(InSelf)))
	{
		return nullptr;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);

	const void* ValuePtr = SelfScriptMapHelper.FindValueFromHash(MapKey.GetValue());
	if (!ValuePtr)
	{
		if (!InDefault)
		{
			PyUtil::SetPythonError(PyExc_KeyError, InSelf, *FString::Printf(TEXT("Key '%s' was not found in the map"), *PyUtil::PyObjectToUEString(InKey)));
			return nullptr;
		}

		Py_INCREF(InDefault);
		return InDefault;
	}

	PyObject* PyItemObj = nullptr;
	if (!PyConversion::PythonizeProperty(SelfScriptMapHelper.GetValueProperty(), ValuePtr, PyItemObj))
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert value property '%s' (%s) for key '%s'"), *SelfScriptMapHelper.GetValueProperty()->GetName(), *SelfScriptMapHelper.GetValueProperty()->GetClass()->GetName(), *PyUtil::PyObjectToUEString(InKey)));
		return nullptr;
	}

	SelfScriptMapHelper.RemovePair(MapKey.GetValue());
	
	return PyItemObj;
}

PyObject* FPyWrapperMap::PopItem(FPyWrapperMap* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);
	const FScriptMapHelper::FIterator It(SelfScriptMapHelper);
	if (It)
	{
		FPyObjectPtr PyReturnKey;
		if (!PyConversion::PythonizeProperty(SelfScriptMapHelper.GetKeyProperty(), SelfScriptMapHelper.GetKeyPtr(It), PyReturnKey.Get()))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert key property '%s' (%s) at index 0"), *SelfScriptMapHelper.GetKeyProperty()->GetName(), *SelfScriptMapHelper.GetKeyProperty()->GetClass()->GetName()));
			return nullptr;
		}

		FPyObjectPtr PyReturnValue;
		if (!PyConversion::PythonizeProperty(SelfScriptMapHelper.GetValueProperty(), SelfScriptMapHelper.GetValuePtr(It), PyReturnValue.Get()))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert value property '%s' (%s) at index 0"), *SelfScriptMapHelper.GetValueProperty()->GetName(), *SelfScriptMapHelper.GetValueProperty()->GetClass()->GetName()));
			return nullptr;
		}

		SelfScriptMapHelper.RemoveAt(It.GetInternalIndex());
		
		return PyTuple_Pack(2, PyReturnKey.Get(), PyReturnValue.Get());
	}

	PyUtil::SetPythonError(PyExc_KeyError, InSelf, TEXT("Cannot pop from an empty map"));
	return nullptr;
}

int FPyWrapperMap::Update(FPyWrapperMap* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfKeyDef = InSelf->MapProp->KeyProp;
	const PyUtil::FPropertyDef SelfValueDef = InSelf->MapProp->ValueProp;
	FPyWrapperMapPtr Other = FPyWrapperMapPtr::StealReference(FPyWrapperMap::CastPyObject(InOther, &PyWrapperMapType, SelfKeyDef, SelfValueDef));
	if (!Other)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument (%s) to '%s'"), *PyUtil::GetFriendlyTypename(InOther), *PyUtil::GetFriendlyTypename(InSelf)));
		return -1;
	}

	FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);
	const FScriptMapHelper OtherScriptMapHelper(Other->MapProp, Other->MapInstance);

	for (FScriptMapHelper::FIterator It(OtherScriptMapHelper); It; ++It)
	{
		SelfScriptMapHelper.AddPair(OtherScriptMapHelper.GetKeyPtr(It), OtherScriptMapHelper.GetValuePtr(It));
	}

	return 0;
}

PyObject* FPyWrapperMap::Items(FPyWrapperMap* InSelf)
{
	return MakeListFromMapIter<FPyWrapperMapItemIterator>(InSelf);
}

PyObject* FPyWrapperMap::Keys(FPyWrapperMap* InSelf)
{
	return MakeListFromMapIter<FPyWrapperMapKeyIterator>(InSelf);
}

PyObject* FPyWrapperMap::Values(FPyWrapperMap* InSelf)
{
	return MakeListFromMapIter<FPyWrapperMapValueIterator>(InSelf);
}

FPyWrapperMap* FPyWrapperMap::FromKeys(PyObject* InSequence, PyObject* InValue, PyTypeObject* InType)
{
	const Py_ssize_t SequenceLen = PyObject_Length(InSequence);
	if (SequenceLen <= 0)
	{
		PyUtil::SetPythonError(PyExc_Exception, InType, *FString::Printf(TEXT("'sequence' (%s) cannot be empty"), *PyUtil::GetFriendlyTypename(InSequence)));
		return nullptr;
	}

	FPyObjectPtr PySequenceIter = FPyObjectPtr::StealReference(PyObject_GetIter(InSequence));
	if (!PySequenceIter)
	{
		PyUtil::SetPythonError(PyExc_Exception, InType, *FString::Printf(TEXT("'sequence' (%s) must be iterable"), *PyUtil::GetFriendlyTypename(InSequence)));
		return nullptr;
	}

	PyUtil::FPropertyDef MapKeyDef;
	PyUtil::FPropertyDef MapValueDef;
	if (PyUtil::ValidateContainerTypeParam((PyObject*)Py_TYPE(InValue), MapValueDef, "value", *PyUtil::GetErrorContext(InType)) != 0)
	{
		return nullptr;
	}

	FPyWrapperMapPtr NewMap;
	for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
	{
		FPyObjectPtr PyKeyItem = FPyObjectPtr::StealReference(PyIter_Next(PySequenceIter));
		if (!PyKeyItem)
		{
			return nullptr;
		}

		if (!NewMap)
		{
			if (PyUtil::ValidateContainerTypeParam((PyObject*)Py_TYPE(PyKeyItem), MapKeyDef, "sequence[0]", *PyUtil::GetErrorContext(InType)) != 0)
			{
				return nullptr;
			}

			NewMap = FPyWrapperMapPtr::StealReference(FPyWrapperMap::New(InType));
			if (FPyWrapperMap::Init(NewMap, MapKeyDef, MapValueDef) != 0)
			{
				return nullptr;
			}
		}

		FScriptMapHelper NewScriptMapHelper(NewMap->MapProp, NewMap->MapInstance);

		const int32 NewElementIndex = NewScriptMapHelper.AddDefaultValue_Invalid_NeedsRehash();
		if (!PyConversion::NativizeProperty(PyKeyItem, NewScriptMapHelper.GetKeyProperty(), NewScriptMapHelper.GetKeyPtr(NewElementIndex)))
		{
			return nullptr;
		}
		if (!PyConversion::NativizeProperty(InValue, NewScriptMapHelper.GetValueProperty(), NewScriptMapHelper.GetValuePtr(NewElementIndex)))
		{
			return nullptr;
		}
	}

	FScriptMapHelper NewScriptMapHelper(NewMap->MapProp, NewMap->MapInstance);
	NewScriptMapHelper.Rehash();

	return NewMap.Release();
}

PyTypeObject InitializePyWrapperMapType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperMap::New(InType);
		}

		static void Dealloc(FPyWrapperMap* InSelf)
		{
			FPyWrapperMap::Free(InSelf);
		}

		static int Init(FPyWrapperMap* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyKeyObj = nullptr;
			PyObject* PyValueObj = nullptr;

			static const char *ArgsKwdList[] = { "key", "value", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:call", (char**)ArgsKwdList, &PyKeyObj, &PyValueObj))
			{
				return -1;
			}

			PyUtil::FPropertyDef MapKeyDef;
			const int ValidateKeyResult = PyUtil::ValidateContainerTypeParam(PyKeyObj, MapKeyDef, "key", *PyUtil::GetErrorContext(Py_TYPE(InSelf)));
			if (ValidateKeyResult != 0)
			{
				return ValidateKeyResult;
			}

			PyUtil::FPropertyDef MapValueDef;
			const int ValidateValueResult = PyUtil::ValidateContainerTypeParam(PyValueObj, MapValueDef, "value", *PyUtil::GetErrorContext(Py_TYPE(InSelf)));
			if (ValidateValueResult != 0)
			{
				return ValidateValueResult;
			}

			return FPyWrapperMap::Init(InSelf, MapKeyDef, MapValueDef);
		}

		static PyObject* Str(FPyWrapperMap* InSelf)
		{
			if (!FPyWrapperMap::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const FScriptMapHelper SelfScriptMapHelper(InSelf->MapProp, InSelf->MapInstance);

			FString ExportedMap;
			for (FScriptMapHelper::FIterator It(SelfScriptMapHelper); It; ++It)
			{
				if (It.GetLogicalIndex() > 0)
				{
					ExportedMap += TEXT(", ");
				}
				ExportedMap += PyUtil::GetFriendlyPropertyValue(SelfScriptMapHelper.GetKeyProperty(), SelfScriptMapHelper.GetKeyPtr(It), PPF_Delimited | PPF_IncludeTransient);
				ExportedMap += TEXT(": ");
				ExportedMap += PyUtil::GetFriendlyPropertyValue(SelfScriptMapHelper.GetValueProperty(), SelfScriptMapHelper.GetValuePtr(It), PPF_Delimited | PPF_IncludeTransient);
			}
			return PyUnicode_FromFormat("{%s}", TCHAR_TO_UTF8(*ExportedMap));
		}

		static PyObject* RichCmp(FPyWrapperMap* InSelf, PyObject* InOther, int InOp)
		{
			if (!FPyWrapperMap::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyUtil::FPropertyDef SelfKeyDef = InSelf->MapProp->KeyProp;
			const PyUtil::FPropertyDef SelfValueDef = InSelf->MapProp->ValueProp;
			FPyWrapperMapPtr Other = FPyWrapperMapPtr::StealReference(FPyWrapperMap::CastPyObject(InOther, &PyWrapperMapType, SelfKeyDef, SelfValueDef));
			if (!Other)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (InOp != Py_EQ && InOp != Py_NE)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Only == and != comparison is supported"));
				return nullptr;
			}

			const bool bIsIdentical = InSelf->MapProp->Identical(InSelf->MapInstance, Other->MapInstance, PPF_None);
			return PyBool_FromLong(InOp == Py_EQ ? bIsIdentical : !bIsIdentical);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperMap* InSelf)
		{
			if (!FPyWrapperMap::ValidateInternalState(InSelf))
			{
				return -1;
			}

			if (InSelf->MapProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
			{
				const uint32 MapHash = InSelf->MapProp->GetValueTypeHash(InSelf->MapInstance);
				return (PyUtil::FPyHashType)(MapHash != -1 ? MapHash : 0);
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Type cannot be hashed"));
			return -1;
		}

		static PyObject* GetIter(FPyWrapperMap* InSelf)
		{
			return MakeMapIter<FPyWrapperMapKeyIterator>(InSelf);
		}
	};

	struct FSequenceFuncs
	{
		static int Contains(FPyWrapperMap* InSelf, PyObject* InValue)
		{
			return FPyWrapperMap::Contains(InSelf, InValue);
		}
	};

	struct FMappingFuncs
	{
		static Py_ssize_t Len(FPyWrapperMap* InSelf)
		{
			return FPyWrapperMap::Len(InSelf);
		}

		static PyObject* GetItem(FPyWrapperMap* InSelf, PyObject* InKey)
		{
			return FPyWrapperMap::GetItem(InSelf, InKey);
		}

		static int SetItem(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InValue)
		{
			return FPyWrapperMap::SetItem(InSelf, InKey, InValue);
		}
	};

	struct FMethods
	{
		// This is for type hinting, to make the class appears as a generic (https://peps.python.org/pep-0560/) and support hinting like 'm: unreal.Map[int, str] = unreal.Map(int, str)'
		static PyObject* GetClassItem(PyObject* Type, PyObject* Item)
		{
			Py_INCREF(Type);
			return Type;
		}

		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyKeyObj = nullptr;
			PyObject* PyValueObj = nullptr;
			PyObject* PyObj = nullptr;

			static const char *ArgsKwdList[] = { "key", "value", "obj", nullptr };
			if (PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OOO:call", (char**)ArgsKwdList, &PyKeyObj, &PyValueObj, &PyObj))
			{
				PyUtil::FPropertyDef MapKeyDef;
				if (PyUtil::ValidateContainerTypeParam(PyKeyObj, MapKeyDef, "key", *PyUtil::GetErrorContext(InType)) != 0)
				{
					return nullptr;
				}

				PyUtil::FPropertyDef MapValueDef;
				if (PyUtil::ValidateContainerTypeParam(PyValueObj, MapValueDef, "value", *PyUtil::GetErrorContext(InType)) != 0)
				{
					return nullptr;
				}

				PyObject* PyCastResult = (PyObject*)FPyWrapperMap::CastPyObject(PyObj, InType, MapKeyDef, MapValueDef);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperMap* InSelf)
		{
			if (!FPyWrapperMap::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperMapFactory::Get().CreateInstance(InSelf->MapInstance, InSelf->MapProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}

		static PyObject* Clear(FPyWrapperMap* InSelf)
		{
			if (FPyWrapperMap::Clear(InSelf) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* FromKeys(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PySequenceObj = nullptr;
			PyObject* PyValueObj = nullptr;

			static const char *ArgsKwdList[] = { "sequence", "value", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:fromkeys", (char**)ArgsKwdList, &PySequenceObj, &PyValueObj))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperMap::FromKeys(PySequenceObj, PyValueObj ? PyValueObj : Py_None, InType);
		}

		static PyObject* Get(FPyWrapperMap* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyKeyObj = nullptr;
			PyObject* PyDefaultObj = nullptr;

			static const char *ArgsKwdList[] = { "key", "default", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:get", (char**)ArgsKwdList, &PyKeyObj, &PyDefaultObj))
			{
				return nullptr;
			}

			return FPyWrapperMap::GetItem(InSelf, PyKeyObj, PyDefaultObj ? PyDefaultObj : Py_None);
		}

		static PyObject* SetDefault(FPyWrapperMap* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyKeyObj = nullptr;
			PyObject* PyDefaultObj = nullptr;

			static const char *ArgsKwdList[] = { "key", "default", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:setdefault", (char**)ArgsKwdList, &PyKeyObj, &PyDefaultObj))
			{
				return nullptr;
			}

			return FPyWrapperMap::SetDefault(InSelf, PyKeyObj, PyDefaultObj);
		}

		static PyObject* Pop(FPyWrapperMap* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyKeyObj = nullptr;
			PyObject* PyDefaultObj = nullptr;

			static const char *ArgsKwdList[] = { "key", "default", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|O:pop", (char**)ArgsKwdList, &PyKeyObj, &PyDefaultObj))
			{
				return nullptr;
			}

			return FPyWrapperMap::Pop(InSelf, PyKeyObj, PyDefaultObj);
		}

		static PyObject* PopItem(FPyWrapperMap* InSelf)
		{
			return FPyWrapperMap::PopItem(InSelf);
		}

		static PyObject* Update(FPyWrapperMap* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "|O:update", &PyObj))
			{
				return nullptr;
			}

			if (PyObj && FPyWrapperMap::Update(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			if (InKwds && FPyWrapperMap::Update(InSelf, InKwds) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Items(FPyWrapperMap* InSelf)
		{
			return MakeMapView<FPyWrapperMapItemView>(InSelf);
		}

		static PyObject* Keys(FPyWrapperMap* InSelf)
		{
			return MakeMapView<FPyWrapperMapKeyView>(InSelf);
		}

		static PyObject* Values(FPyWrapperMap* InSelf)
		{
			return MakeMapView<FPyWrapperMapValueView>(InSelf);
		}
	};

	// NOTE: _KeyType = typing.TypeVar('_KeyType'), _ValueType = typing.TypeVar('_ValueType'), Type/Iterable/Tuple/Union/List/ItemsView/KeysView/ValuesView are defines in the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "__class_getitem__", PyCFunctionCast(FMethods::GetClassItem), METH_O|METH_CLASS, "__class_getitem__(cls, item) -- implemented for type hinting purpose only" },
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_KEYWORDS | METH_CLASS, "cast(cls, key: Type[_KeyType], value: Type[_ValueType], obj: object) -> Map[_KeyType, _ValueType] -- cast the given object to this Unreal map type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> Map[_KeyType, _ValueType] -- copy this Unreal map" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> Map[_KeyType, _ValueType] -- copy this Unreal map" },
		{ "clear", PyCFunctionCast(&FMethods::Clear), METH_NOARGS, "clear(self) -> None -- remove all items from this Unreal map" },
		{ "fromkeys", PyCFunctionCast(&FMethods::FromKeys), METH_VARARGS | METH_KEYWORDS | METH_CLASS, "fromkeys(cls, sequence: Iterable[_KeyType], value: _ValueType) -> Map[_KeyType, _ValueType] -- returns a new Unreal map of keys from the sequence using the given value (types are inferred)" },
		{ "get", PyCFunctionCast(&FMethods::Get), METH_VARARGS | METH_KEYWORDS, "get(self, key: _KeyType, default: _ValueType=...) -> _ValueType -- x[key] if key in x, otherwise default" },
		{ "setdefault", PyCFunctionCast(&FMethods::SetDefault), METH_VARARGS | METH_KEYWORDS, "setdefault(self, key: _KeyType, default: _ValueType=...) -> _ValueType -- set key to default if key not in x and return the value of key" },
		{ "pop", PyCFunctionCast(&FMethods::Pop), METH_VARARGS | METH_KEYWORDS, "pop(self, key: _KeyType, default: _ValueType=...) -> _ValueType -- remove key and return its value, or default if key not present, or raise KeyError if no default" },
		{ "popitem", PyCFunctionCast(&FMethods::PopItem), METH_NOARGS, "popitem(self) -> Tuple[_KeyType, _ValueType] -- remove and return an arbitrary pair from this Unreal map, or raise KeyError if the map is empty" },
		{ "update", PyCFunctionCast(&FMethods::Update), METH_VARARGS | METH_KEYWORDS, "update(self, pairs: Union[Mapping[_KeyType, _ValueType], Iterable[Tuple[_KeyType, _ValueType]], Iterable[List[Union[_KeyType, _ValueType]]]]) -> None -- update this Unreal map from the given mapping or sequence pairs type or key->value arguments" },
		{ "items", PyCFunctionCast(&FMethods::Items), METH_NOARGS, "items(self) -> ItemsView[_KeyType, _ValueType] -- a set-like view of the key->value pairs of this Unreal map" },
		{ "keys", PyCFunctionCast(&FMethods::Keys), METH_NOARGS, "keys(self) -> KeysView[_KeyType] -- a set-like view of the keys of this Unreal map" },
		{ "values", PyCFunctionCast(&FMethods::Values), METH_NOARGS, "values(self) -> ValuesView[_ValueType] -- a view of the values of this Unreal map" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"Map", /* tp_name */
		sizeof(FPyWrapperMap), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed map instances";

	static PySequenceMethods PySequence;
	PySequence.sq_contains = (objobjproc)&FSequenceFuncs::Contains;

	static PyMappingMethods PyMapping;
	PyMapping.mp_length = (lenfunc)&FMappingFuncs::Len;
	PyMapping.mp_subscript = (binaryfunc)&FMappingFuncs::GetItem;
	PyMapping.mp_ass_subscript = (objobjargproc)&FMappingFuncs::SetItem;

	PyType.tp_as_sequence = &PySequence;
	PyType.tp_as_mapping = &PyMapping;

	return PyType;
}

template <typename TImpl>
PyTypeObject InitializePyWrapperMapIteratorType(const char* InName)
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)TImpl::New(InType);
		}

		static void Dealloc(TImpl* InSelf)
		{
			TImpl::Free(InSelf);
		}

		static int Init(TImpl* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				return -1;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperMapType) != 1)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot initialize '%s' with an instance of '%s'"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return TImpl::Init(InSelf, (FPyWrapperMap*)PyObj);
		}

		static PyObject* GetIter(TImpl* InSelf)
		{
			return TImpl::GetIter(InSelf);
		}

		static PyObject* IterNext(TImpl* InSelf)
		{
			return TImpl::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		InName, /* tp_name */
		sizeof(TImpl), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed map iterators";

	return PyType;
}

template <typename TImpl>
PyTypeObject InitializePyWrapperMapViewType(const char* InName)
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)TImpl::New(InType);
		}

		static void Dealloc(TImpl* InSelf)
		{
			TImpl::Free(InSelf);
		}

		static int Init(TImpl* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				return -1;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperMapType) != 1)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot initialize '%s' with an instance of '%s'"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return TImpl::Init(InSelf, (FPyWrapperMap*)PyObj);
		}

		static PyObject* GetIter(TImpl* InSelf)
		{
			return TImpl::GetIter(InSelf);
		}
	};

	struct FSequenceFuncs
	{
		static Py_ssize_t Len(TImpl* InSelf)
		{
			return TImpl::Len(InSelf);
		}

		static int Contains(TImpl* InSelf, PyObject* InValue)
		{
			FPyObjectPtr PySet = FPyObjectPtr::StealReference(PySet_New((PyObject*)InSelf));
			if (!PySet)
			{
				return -1;
			}

			return PySet_Contains(PySet, InValue);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		InName, /* tp_name */
		sizeof(TImpl), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed map views";

	static PySequenceMethods PySequence;
	PySequence.sq_length = (lenfunc)&FSequenceFuncs::Len;
	PySequence.sq_contains = (objobjproc)&FSequenceFuncs::Contains;

	PyType.tp_as_sequence = &PySequence;

	return PyType;
}

template <typename TImpl>
PyTypeObject InitializePyWrapperMapSetViewType(const char* InName)
{
	struct FNumberFuncs
	{
		static PyObject* Sub(PyObject* InLHS, PyObject* InRHS)
		{
			FPyObjectPtr LHS = FPyObjectPtr::StealReference(PySet_New(InLHS));
			FPyObjectPtr RHS = FPyObjectPtr::StealReference(PySet_New(InRHS));
			if (!LHS || !RHS)
			{
				return nullptr;
			}

			return PyNumber_Subtract(LHS, RHS);
		}

		static PyObject* And(PyObject* InLHS, PyObject* InRHS)
		{
			FPyObjectPtr LHS = FPyObjectPtr::StealReference(PySet_New(InLHS));
			FPyObjectPtr RHS = FPyObjectPtr::StealReference(PySet_New(InRHS));
			if (!LHS || !RHS)
			{
				return nullptr;
			}

			return PyNumber_And(LHS, RHS);
		}

		static PyObject* Xor(PyObject* InLHS, PyObject* InRHS)
		{
			FPyObjectPtr LHS = FPyObjectPtr::StealReference(PySet_New(InLHS));
			FPyObjectPtr RHS = FPyObjectPtr::StealReference(PySet_New(InRHS));
			if (!LHS || !RHS)
			{
				return nullptr;
			}

			return PyNumber_Xor(LHS, RHS);
		}

		static PyObject* Or(PyObject* InLHS, PyObject* InRHS)
		{
			FPyObjectPtr LHS = FPyObjectPtr::StealReference(PySet_New(InLHS));
			FPyObjectPtr RHS = FPyObjectPtr::StealReference(PySet_New(InRHS));
			if (!LHS || !RHS)
			{
				return nullptr;
			}

			return PyNumber_Or(LHS, RHS);
		}
	};

	PyTypeObject PyType = InitializePyWrapperMapViewType<TImpl>(InName);

	static PyNumberMethods PyNumber;
	PyNumber.nb_subtract = (binaryfunc)&FNumberFuncs::Sub;
	PyNumber.nb_and = (binaryfunc)&FNumberFuncs::And;
	PyNumber.nb_xor = (binaryfunc)&FNumberFuncs::Xor;
	PyNumber.nb_or = (binaryfunc)&FNumberFuncs::Or;

	PyType.tp_as_number = &PyNumber;

	return PyType;
}

PyTypeObject PyWrapperMapType = InitializePyWrapperMapType();
PyTypeObject PyWrapperMapItemIteratorType = InitializePyWrapperMapIteratorType<FPyWrapperMapItemIterator>("MapItemIterator");
PyTypeObject PyWrapperMapKeyIteratorType = InitializePyWrapperMapIteratorType<FPyWrapperMapKeyIterator>("MapKeyIterator");
PyTypeObject PyWrapperMapValueIteratorType = InitializePyWrapperMapIteratorType<FPyWrapperMapValueIterator>("MapValueIterator");
PyTypeObject PyWrapperMapItemViewType = InitializePyWrapperMapSetViewType<FPyWrapperMapItemView>("MapItemView");
PyTypeObject PyWrapperMapKeyViewType = InitializePyWrapperMapSetViewType<FPyWrapperMapKeyView>("MapKeyView");
PyTypeObject PyWrapperMapValueViewType = InitializePyWrapperMapViewType<FPyWrapperMapValueView>("MapValueView");

void FPyWrapperMapMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperMap* Self = static_cast<FPyWrapperMap*>(Instance);
	if (Self->MapProp && Self->MapInstance && !Self->OwnerContext.HasOwner())
	{
		Self->MapProp.AddReferencedObjects(Collector);
		FPyReferenceCollector::AddReferencedObjectsFromProperty(Collector, Self->MapProp, Self->MapInstance);
	}
}

#endif	// WITH_PYTHON
