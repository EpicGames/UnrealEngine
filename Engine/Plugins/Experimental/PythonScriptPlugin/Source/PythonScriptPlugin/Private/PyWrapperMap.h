// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "PyWrapperOwnerContext.h"
#include "PyUtil.h"

#if WITH_PYTHON

/** Python type for FPyWrapperMap */
extern PyTypeObject PyWrapperMapType;

/** Initialize the PyWrapperMap types and add them to the given Python module */
void InitializePyWrapperMap(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed map instances */
struct FPyWrapperMap : public FPyWrapperBase
{
	/** The owner of the wrapped map instance (if any) */
	FPyWrapperOwnerContext OwnerContext;

	/** Property describing the map */
	PyUtil::FConstMapPropOnScope MapProp;

	/** Wrapped map instance */
	void* MapInstance;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperMap* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperMap* InSelf);

	/** Initialize this wrapper (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperMap* InSelf, const PyUtil::FPropertyDef& InKeyDef, const PyUtil::FPropertyDef& InValueDef);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperMap* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FMapProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperMap* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperMap* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperMap* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperMap* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InKeyDef, const PyUtil::FPropertyDef& InValueDef, FPyConversionResult* OutCastResult = nullptr);

	/** Get the length of this container (equivalent to 'len(x)' in Python) */
	static Py_ssize_t Len(FPyWrapperMap* InSelf);

	/** Get the item with key K (equivalent to 'x[K]' in Python, returns new reference) */
	static PyObject* GetItem(FPyWrapperMap* InSelf, PyObject* InKey);

	/** Get the item with key K (equivalent to 'x.get(K, D)' in Python, returns new reference) */
	static PyObject* GetItem(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InDefault);

	/** Set the item with key K (equivalent to 'x[K] = v' in Python) */
	static int SetItem(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InValue);

	/** Set the item with key K if K isn't in the map and return the value of K (equivalent to 'x.setdefault(K, v)' in Python) */
	static PyObject* SetDefault(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InValue);

	/** Does this container have an entry with the given value? (equivalent to 'v in x' in Python) */
	static int Contains(FPyWrapperMap* InSelf, PyObject* InKey);

	/** Remove all values from this container (equivalent to 'x.clear()' in Python) */
	static int Clear(FPyWrapperMap* InSelf);

	/** Remove and return the value for key K if present, otherwise return the default, or raise KeyError if no default is given (equivalent to 'x.popitem()' in Python, returns new reference) */
	static PyObject* Pop(FPyWrapperMap* InSelf, PyObject* InKey, PyObject* InDefault);

	/** Remove and return an arbitrary pair from this map (equivalent to 'x.popitem()' in Python, returns new reference) */
	static PyObject* PopItem(FPyWrapperMap* InSelf);

	/** Update this map from another (equivalent to 'x.update(o)' in Python) */
	static int Update(FPyWrapperMap* InSelf, PyObject* InOther);

	/** Get a Python list containing the items from this map as key->value pairs */
	static PyObject* Items(FPyWrapperMap* InSelf);

	/** Get a Python list containing the keys from this map */
	static PyObject* Keys(FPyWrapperMap* InSelf);

	/** Get a Python list containing the values from this map */
	static PyObject* Values(FPyWrapperMap* InSelf);

	/** Create a new map of keys from the sequence using the given value (types are inferred) */
	static FPyWrapperMap* FromKeys(PyObject* InSequence, PyObject* InValue, PyTypeObject* InType);
};

/** Meta-data for all Unreal exposed map types */
struct FPyWrapperMapMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperMapMetaData, FGuid(0xA6AAB38C, 0x5C174F81, 0xB80FA903, 0xBF50C8A2))

	FPyWrapperMapMetaData()
	{
	}

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;
};

typedef TPyPtr<FPyWrapperMap> FPyWrapperMapPtr;

#endif	// WITH_PYTHON
