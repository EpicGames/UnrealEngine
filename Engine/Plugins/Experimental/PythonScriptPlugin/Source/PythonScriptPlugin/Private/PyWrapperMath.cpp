// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperMath.h"
#include "PyWrapperTypeRegistry.h"
#include "PyCore.h"
#include "PyUtil.h"
#include "PyConstant.h"
#include "PyConversion.h"

#if WITH_PYTHON

void InitializePyWrapperMath(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FVector>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FVector2D>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FVector4>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FQuat>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FRotator>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FTransform>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FPlane>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FBox2D>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FColor>>());
	FPyWrapperTypeRegistry::Get().RegisterInlineStructFactory(MakeShared<TPyWrapperInlineStructFactory<FLinearColor>>());
	// Register more math types as inline as needed.
}

#endif	// WITH_PYTHON
