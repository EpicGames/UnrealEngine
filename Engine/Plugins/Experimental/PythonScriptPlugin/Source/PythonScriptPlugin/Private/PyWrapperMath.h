// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperStruct.h"
#include "PyUtil.h"

#if WITH_PYTHON

/** Initialize the PyWrapperMath types and add them to the given Python module */
void InitializePyWrapperMath(PyGenUtil::FNativePythonModule& ModuleInfo);

#endif	// WITH_PYTHON
