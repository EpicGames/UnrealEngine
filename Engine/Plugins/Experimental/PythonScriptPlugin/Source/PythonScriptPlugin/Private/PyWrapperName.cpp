// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperName.h"
#include "PyWrapperTypeRegistry.h"
#include "PyUtil.h"
#include "PyConversion.h"

#if WITH_PYTHON

void InitializePyWrapperName(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperNameType) == 0)
	{
		ModuleInfo.AddType(&PyWrapperNameType);
	}
}

void FPyWrapperName::InitValue(FPyWrapperName* InSelf, const FName InValue)
{
	Super::InitValue(InSelf, InValue);
	FPyWrapperNameFactory::Get().MapInstance(InSelf->Value, InSelf);
}

void FPyWrapperName::DeinitValue(FPyWrapperName* InSelf)
{
	FPyWrapperNameFactory::Get().UnmapInstance(InSelf->Value, Py_TYPE(InSelf));
	Super::DeinitValue(InSelf);
}

FPyWrapperName* FPyWrapperName::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperNameType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperName*)InPyObject;
	}

	return nullptr;
}

FPyWrapperName* FPyWrapperName::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperNameType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperNameType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperName*)InPyObject;
	}

	{
		FName InitValue;
		if (PyConversion::Nativize(InPyObject, InitValue))
		{
			FPyWrapperNamePtr NewName = FPyWrapperNamePtr::StealReference(FPyWrapperName::New(InType));
			if (NewName)
			{
				if (FPyWrapperName::Init(NewName, InitValue) != 0)
				{
					return nullptr;
				}
			}
			SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
			return NewName.Release();
		}
	}

	return nullptr;
}

PyTypeObject InitializePyWrapperNameType()
{
	struct FFuncs
	{
		static int Init(FPyWrapperName* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			FName InitValue;

			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "|O:call", &PyObj))
			{
				return -1;
			}
			
			if (PyObj && !PyConversion::Nativize(PyObj, InitValue))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert init argument '%s' to 'Name'"), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return FPyWrapperName::Init(InSelf, InitValue);
		}

		static PyObject* Str(FPyWrapperName* InSelf)
		{
			return PyUnicode_FromString(TCHAR_TO_UTF8(*InSelf->Value.ToString()));
		}

		static PyObject* Repr(FPyWrapperName* InSelf)
		{
			return PyUnicode_FromString(TCHAR_TO_UTF8(*FString::Printf(TEXT("Name(\"%s\")"), *InSelf->Value.ToString())));
		}

		static PyObject* RichCmp(FPyWrapperName* InSelf, PyObject* InOther, int InOp)
		{
			FName Other;
			if (!PyConversion::Nativize(InOther, Other, PyConversion::ESetErrorState::No))
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			return PyUtil::PyRichCmp(InSelf->Value.Compare(Other), 0, InOp);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperName* InSelf)
		{
			const PyUtil::FPyHashType PyHash = (PyUtil::FPyHashType)GetTypeHash(InSelf->Value);
			return PyHash != -1 ? PyHash : 0;
		}
	};

	struct FMethods
	{
		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperName::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* IsValid(FPyWrapperName* InSelf)
		{
			if (InSelf->Value.IsValid())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsNone(FPyWrapperName* InSelf)
		{
			if (InSelf->Value.IsNone())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}
	};

	static PyMethodDef PyMethods[] = {
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls, object: object) -> Name -- cast the given object to this Unreal name type" },
		{ "is_valid", PyCFunctionCast(&FMethods::IsValid), METH_NOARGS, "is_valid(self) -> bool -- is this Unreal name valid?" },
		{ "is_none", PyCFunctionCast(&FMethods::IsNone), METH_NOARGS, "is_none(self) -> bool -- is this Unreal name set to NAME_None?" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = InitializePyWrapperBasicType<FPyWrapperName>("Name", "Type for all Unreal exposed name instances");

	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Repr;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;

	PyType.tp_methods = PyMethods;

	return PyType;
}

PyTypeObject PyWrapperNameType = InitializePyWrapperNameType();

#endif	// WITH_PYTHON
