// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBasic.h"

#if WITH_PYTHON

/** Python type for FPyWrapperName */
extern PyTypeObject PyWrapperNameType;

/** Initialize the FPyWrapperName types and add them to the given Python module */
void InitializePyWrapperName(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed FName instances */
struct FPyWrapperName : public TPyWrapperBasic<FName, FPyWrapperName>
{
	typedef TPyWrapperBasic<FName, FPyWrapperName> Super;

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperName* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperName* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Initialize the value of this wrapper instance (internal) */
	static void InitValue(FPyWrapperName* InSelf, const FName InValue);

	/** Deinitialize the value of this wrapper instance (internal) */
	static void DeinitValue(FPyWrapperName* InSelf);
};

typedef TPyPtr<FPyWrapperName> FPyWrapperNamePtr;

#endif	// WITH_PYTHON
