// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperObject.h"
#include "PyWrapperOwnerContext.h"
#include "PyWrapperTypeRegistry.h"
#include "PyGIL.h"
#include "PyCore.h"
#include "PyUtil.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/Package.h"
#include "UObject/Class.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "UObject/StructOnScope.h"
#include "UObject/PropertyPortFlags.h"
#include "Engine/World.h"
#include "Misc/ScopeExit.h"
#include "Templates/Casts.h"
#include "BlueprintActionDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PyWrapperObject)

#if WITH_PYTHON

void InitializePyWrapperObject(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperObjectType) == 0)
	{
		static FPyWrapperObjectMetaData MetaData;
		FPyWrapperObjectMetaData::SetMetaData(&PyWrapperObjectType, &MetaData);
		ModuleInfo.AddType(&PyWrapperObjectType);
	}
}

FPyWrapperObject* FPyWrapperObject::New(PyTypeObject* InType)
{
	FPyWrapperObject* Self = (FPyWrapperObject*)FPyWrapperBase::New(InType);
	if (Self)
	{
		Self->ObjectInstance = nullptr;
	}
	return Self;
}

void FPyWrapperObject::Free(FPyWrapperObject* InSelf)
{
	Deinit(InSelf);

	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperObject::Init(FPyWrapperObject* InSelf, UObject* InValue)
{
	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	check(InValue);

	InSelf->ObjectInstance = InValue;
	FPyWrapperObjectFactory::Get().MapInstance(InSelf->ObjectInstance, InSelf);
	return 0;
}

void FPyWrapperObject::Deinit(FPyWrapperObject* InSelf)
{
	if (InSelf->ObjectInstance)
	{
		FPyWrapperObjectFactory::Get().UnmapInstance(InSelf->ObjectInstance, Py_TYPE(InSelf));
	}
	InSelf->ObjectInstance = nullptr;
}

bool FPyWrapperObject::ValidateInternalState(FPyWrapperObject* InSelf)
{
	if (!InSelf->ObjectInstance)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ObjectInstance is null!"));
		return false;
	}

	return true;
}

FPyWrapperObject* FPyWrapperObject::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperObjectType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperObject*)InPyObject;
	}

	return nullptr;
}

FPyWrapperObject* FPyWrapperObject::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperObjectType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperObjectType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperObject*)InPyObject;
	}

	return nullptr;
}

PyObject* FPyWrapperObject::GetPropertyValue(FPyWrapperObject* InSelf, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	return PyGenUtil::GetPropertyValue(InSelf->ObjectInstance->GetClass(), InSelf->ObjectInstance, InPropDef, InPythonAttrName, (PyObject*)InSelf, *PyUtil::GetErrorContext(InSelf));
}

int FPyWrapperObject::SetPropertyValue(FPyWrapperObject* InSelf, PyObject* InValue, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName, const EPropertyAccessChangeNotifyMode InNotifyMode, const uint64 InReadOnlyFlags)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = FPyWrapperOwnerContext((PyObject*)InSelf, InPropDef.Prop).BuildChangeNotify(InNotifyMode);

	// If the object is a template, gather instances (including subclass CDOs) inheriting their property value from this template.
	// Those instances are passed into PyGenUtil::SetPropertyValue so that they will receive the value change as well.
	TArray<void*> ArchetypeInsts;
	const bool bIsObjectTemplate = PropertyAccessUtil::IsObjectTemplate(InSelf->ObjectInstance);
	if (bIsObjectTemplate)
	{
		PropertyAccessUtil::GetArchetypeInstancesInheritingPropertyValue_AsContainerData(InPropDef.Prop, InSelf->ObjectInstance, ArchetypeInsts);
	}

	return PyGenUtil::SetPropertyValue(InSelf->ObjectInstance->GetClass(), InSelf->ObjectInstance, InValue, InPropDef, InPythonAttrName, ChangeNotify.Get(), InReadOnlyFlags, bIsObjectTemplate, *PyUtil::GetErrorContext(InSelf), ArchetypeInsts);
}

PyObject* FPyWrapperObject::CallGetterFunction(FPyWrapperObject* InSelf, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	return CallFunction_Impl(InSelf->ObjectInstance, InFuncDef, InFuncDef.Func ? TCHAR_TO_UTF8(*InFuncDef.Func->GetName()) : "null", *PyUtil::GetErrorContext(InSelf));
}

int FPyWrapperObject::CallSetterFunction(FPyWrapperObject* InSelf, PyObject* InValue, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	if (ensureAlways(InFuncDef.Func))
	{
		// Deprecated functions emit a warning
		if (InFuncDef.DeprecationMessage.IsSet())
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf, *FString::Printf(TEXT("Function '%s.%s' is deprecated: %s"), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.Func->GetName(), *InFuncDef.DeprecationMessage.GetValue())) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return -1;
			}
		}

		// Setter functions should have a single input parameter and no output parameters
		if (InFuncDef.InputParams.Num() != 1 || InFuncDef.OutputParams.Num() != 0)
		{
			PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Setter function '%s.%s' on '%s' has the incorrect number of parameters (expected 1 input and 0 output, got %d input and %d output)"), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.Func->GetName(), *InSelf->ObjectInstance->GetName(), InFuncDef.InputParams.Num(), InFuncDef.OutputParams.Num()));
			return -1;
		}

		PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
		if (InValue)
		{
			if (!PyConversion::NativizeProperty_InContainer(InValue, InFuncDef.InputParams[0].ParamProp, FuncParams.GetMemory(), 0))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert input parameter when calling function '%s.%s' on '%s'"), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.Func->GetName(), *InSelf->ObjectInstance->GetName()));
				return -1;
			}
		}
		if (!PyUtil::InvokeFunctionCall(InSelf->ObjectInstance, InFuncDef.Func, FuncParams.GetMemory(), *PyUtil::GetErrorContext(InSelf)))
		{
			return -1;
		}
	}

	return 0;
}

PyObject* FPyWrapperObject::CallFunction(PyTypeObject* InType, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName)
{
	UClass* Class = FPyWrapperObjectMetaData::GetClass(InType);
	UObject* Obj = Class ? Class->GetDefaultObject() : nullptr;
	return CallFunction_Impl(Obj, InFuncDef, InPythonFuncName, *PyUtil::GetErrorContext(InType));
}

PyObject* FPyWrapperObject::CallFunction(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName)
{
	UClass* Class = FPyWrapperObjectMetaData::GetClass(InType);
	UObject* Obj = Class ? Class->GetDefaultObject() : nullptr;
	return CallFunction_Impl(Obj, InArgs, InKwds, InFuncDef, InPythonFuncName, *PyUtil::GetErrorContext(InType));
}

PyObject* FPyWrapperObject::CallFunction(FPyWrapperObject* InSelf, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	return CallFunction_Impl(InSelf->ObjectInstance, InFuncDef, InPythonFuncName, *PyUtil::GetErrorContext(InSelf));
}

PyObject* FPyWrapperObject::CallFunction(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	return CallFunction_Impl(InSelf->ObjectInstance, InArgs, InKwds, InFuncDef, InPythonFuncName, *PyUtil::GetErrorContext(InSelf));
}

PyObject* FPyWrapperObject::CallFunction_Impl(UObject* InObj, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName, const TCHAR* InErrorCtxt)
{
	if (InObj && ensureAlways(InFuncDef.Func))
	{
		// Deprecated functions emit a warning
		if (InFuncDef.DeprecationMessage.IsSet())
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FString::Printf(TEXT("Function '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(InPythonFuncName), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.DeprecationMessage.GetValue())) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return nullptr;
			}
		}

		if (InFuncDef.Func->ChildProperties == nullptr)
		{
			// No return value
			if (!PyUtil::InvokeFunctionCall(InObj, InFuncDef.Func, nullptr, InErrorCtxt))
			{
				return nullptr;
			}
		}
		else
		{
			// Return value requires that we create a params struct to hold the result
			PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
			if (!PyUtil::InvokeFunctionCall(InObj, InFuncDef.Func, FuncParams.GetMemory(), InErrorCtxt))
			{
				return nullptr;
			}
			return PyGenUtil::PackReturnValues(FuncParams.GetMemory(), InFuncDef.OutputParams, InErrorCtxt, *FString::Printf(TEXT("function '%s.%s' on '%s'"), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.Func->GetName(), *InObj->GetName()));
		}
	}

	Py_RETURN_NONE;
}

PyObject* FPyWrapperObject::CallFunction_Impl(UObject* InObj, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName, const TCHAR* InErrorCtxt)
{
	TArray<PyObject*> Params;
	if (!PyGenUtil::ParseMethodParameters(InArgs, InKwds, InFuncDef.InputParams, InPythonFuncName, Params))
	{
		return nullptr;
	}

	if (InObj && ensureAlways(InFuncDef.Func))
	{
		// Deprecated functions emit a warning
		if (InFuncDef.DeprecationMessage.IsSet())
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InErrorCtxt, *FString::Printf(TEXT("Function '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(InPythonFuncName), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.DeprecationMessage.GetValue())) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return nullptr;
			}
		}

		PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
		PyGenUtil::ApplyParamDefaults(FuncParams.GetMemory(), InFuncDef.InputParams);
		for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ++ParamIndex)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ParamDef = InFuncDef.InputParams[ParamIndex];

			PyObject* PyValue = Params[ParamIndex];
			if (PyValue)
			{
				if (!PyConversion::NativizeProperty_InContainer(PyValue, ParamDef.ParamProp, FuncParams.GetMemory(), 0))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InErrorCtxt, *FString::Printf(TEXT("Failed to convert parameter '%s' when calling function '%s.%s' on '%s'"), UTF8_TO_TCHAR(ParamDef.ParamName.GetData()), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.Func->GetName(), *InObj->GetName()));
					return nullptr;
				}
			}
		}
		if (!PyUtil::InvokeFunctionCall(InObj, InFuncDef.Func, FuncParams.GetMemory(), InErrorCtxt))
		{
			return nullptr;
		}
		return PyGenUtil::PackReturnValues(FuncParams.GetMemory(), InFuncDef.OutputParams, InErrorCtxt, *FString::Printf(TEXT("function '%s.%s' on '%s'"), *InFuncDef.Func->GetOwnerClass()->GetName(), *InFuncDef.Func->GetName(), *InObj->GetName()));
	}

	Py_RETURN_NONE;
}

PyObject* FPyWrapperObject::CallClassMethodNoArgs_Impl(PyTypeObject* InType, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedMethod* Closure = (PyGenUtil::FGeneratedWrappedMethod*)InClosure;
	return CallFunction(InType, Closure->MethodFunc, Closure->MethodName.GetData());
}

PyObject* FPyWrapperObject::CallClassMethodWithArgs_Impl(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedMethod* Closure = (PyGenUtil::FGeneratedWrappedMethod*)InClosure;
	return CallFunction(InType, InArgs, InKwds, Closure->MethodFunc, Closure->MethodName.GetData());
}

PyObject* FPyWrapperObject::CallMethodNoArgs_Impl(FPyWrapperObject* InSelf, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedMethod* Closure = (PyGenUtil::FGeneratedWrappedMethod*)InClosure;
	return CallFunction(InSelf, Closure->MethodFunc, Closure->MethodName.GetData());
}

PyObject* FPyWrapperObject::CallMethodWithArgs_Impl(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedMethod* Closure = (PyGenUtil::FGeneratedWrappedMethod*)InClosure;
	return CallFunction(InSelf, InArgs, InKwds, Closure->MethodFunc, Closure->MethodName.GetData());
}

PyObject* FPyWrapperObject::CallDynamicFunction_Impl(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const PyGenUtil::FGeneratedWrappedMethodParameter& InSelfParam, const char* InPythonFuncName)
{
	TArray<PyObject*> Params;
	if ((InArgs || InKwds) && !PyGenUtil::ParseMethodParameters(InArgs, InKwds, InFuncDef.InputParams, InPythonFuncName, Params))
	{
		return nullptr;
	}

	if (ensureAlways(InFuncDef.Func))
	{
		UClass* Class = InFuncDef.Func->GetOwnerClass();
		UObject* Obj = Class->GetDefaultObject();

		// Deprecated functions emit a warning
		if (InFuncDef.DeprecationMessage.IsSet())
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf, *FString::Printf(TEXT("Function '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(InPythonFuncName), *Class->GetName(), *InFuncDef.DeprecationMessage.GetValue())) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return nullptr;
			}
		}

		PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
		PyGenUtil::ApplyParamDefaults(FuncParams.GetMemory(), InFuncDef.InputParams);
		if (ensureAlways(CastField<FObjectPropertyBase>(InSelfParam.ParamProp)))
		{
			void* SelfArgInstance = InSelfParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
			CastField<FObjectPropertyBase>(InSelfParam.ParamProp)->SetObjectPropertyValue(SelfArgInstance, InSelf->ObjectInstance);
		}
		for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ++ParamIndex)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ParamDef = InFuncDef.InputParams[ParamIndex];

			PyObject* PyValue = Params[ParamIndex];
			if (PyValue)
			{
				if (!PyConversion::NativizeProperty_InContainer(PyValue, ParamDef.ParamProp, FuncParams.GetMemory(), 0))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert parameter '%s' when calling function '%s.%s' on '%s'"), UTF8_TO_TCHAR(ParamDef.ParamName.GetData()), *Class->GetName(), *InFuncDef.Func->GetName(), *Obj->GetName()));
					return nullptr;
				}
			}
		}
		const FString ErrorCtxt = PyUtil::GetErrorContext(InSelf);
		if (!PyUtil::InvokeFunctionCall(Obj, InFuncDef.Func, FuncParams.GetMemory(), *ErrorCtxt))
		{
			return nullptr;
		}
		return PyGenUtil::PackReturnValues(FuncParams.GetMemory(), InFuncDef.OutputParams, *ErrorCtxt, *FString::Printf(TEXT("function '%s.%s' on '%s'"), *Class->GetName(), *InFuncDef.Func->GetName(), *Obj->GetName()));
	}

	Py_RETURN_NONE;
}

PyObject* FPyWrapperObject::CallDynamicMethodNoArgs_Impl(FPyWrapperObject* InSelf, void* InClosure)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	const PyGenUtil::FGeneratedWrappedDynamicMethod* Closure = (PyGenUtil::FGeneratedWrappedDynamicMethod*)InClosure;
	return CallDynamicFunction_Impl(InSelf, nullptr, nullptr, Closure->MethodFunc, Closure->SelfParam, Closure->MethodName.GetData());
}

PyObject* FPyWrapperObject::CallDynamicMethodWithArgs_Impl(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, void* InClosure)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	const PyGenUtil::FGeneratedWrappedDynamicMethod* Closure = (PyGenUtil::FGeneratedWrappedDynamicMethod*)InClosure;
	return CallDynamicFunction_Impl(InSelf, InArgs, InKwds, Closure->MethodFunc, Closure->SelfParam, Closure->MethodName.GetData());
}

PyObject* FPyWrapperObject::Getter_Impl(FPyWrapperObject* InSelf, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedGetSet* Closure = (PyGenUtil::FGeneratedWrappedGetSet*)InClosure;
	return Closure->GetFunc.Func
		? CallGetterFunction(InSelf, Closure->GetFunc)
		: GetPropertyValue(InSelf, Closure->Prop, Closure->GetSetName.GetData());
}

int FPyWrapperObject::Setter_Impl(FPyWrapperObject* InSelf, PyObject* InValue, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedGetSet* Closure = (PyGenUtil::FGeneratedWrappedGetSet*)InClosure;
	return Closure->SetFunc.Func
		? CallSetterFunction(InSelf, InValue, Closure->SetFunc)
		: SetPropertyValue(InSelf, InValue, Closure->Prop, Closure->GetSetName.GetData());
}

PyTypeObject InitializePyWrapperObjectType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperObject::New(InType);
		}

		static void Dealloc(FPyWrapperObject* InSelf)
		{
			FPyWrapperObject::Free(InSelf);
		}

		static int Init(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			UObject* InitValue = nullptr;

			UObject* ObjectOuter = GetTransientPackage();
			FName ObjectName;

			// Parse the args
			{
				PyObject* PyOuterObj = nullptr;
				PyObject* PyNameObj = nullptr;

				static const char *ArgsKwdList[] = { "outer", "name", nullptr };
				if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "|OO:call", (char**)ArgsKwdList, &PyOuterObj, &PyNameObj))
				{
					return -1;
				}

				if (PyOuterObj && !PyConversion::Nativize(PyOuterObj, ObjectOuter))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'outer' (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyOuterObj)));
					return -1;
				}

				if (PyNameObj && !PyConversion::Nativize(PyNameObj, ObjectName))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
					return -1;
				}
			}

			// Deprecated classes emit a warning
			{
				FString DeprecationMessage;
				if (FPyWrapperObjectMetaData::IsClassDeprecated(InSelf, &DeprecationMessage) &&
					PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf, *FString::Printf(TEXT("Class '%s' is deprecated: %s"), UTF8_TO_TCHAR(Py_TYPE(InSelf)->tp_name), *DeprecationMessage)) == -1
					)
				{
					// -1 from SetPythonWarning means the warning should be an exception
					return -1;
				}
			}

			// Create the instance
			UClass* ObjClass = FPyWrapperObjectMetaData::GetClass(InSelf);
			InitValue = PyUtil::NewObject(ObjClass, ObjectOuter, ObjectName, nullptr, *PyUtil::GetErrorContext(InSelf));
			if (!InitValue)
			{
				// PyUtil::NewObject should have set an error state
				return -1;
			}

			return FPyWrapperObject::Init(InSelf, InitValue);
		}

		static PyObject* Str(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyUnicode_FromFormat("<Object '%s' (%p) Class '%s'>", TCHAR_TO_UTF8(*InSelf->ObjectInstance->GetPathName()), InSelf->ObjectInstance.Get(), TCHAR_TO_UTF8(*InSelf->ObjectInstance->GetClass()->GetName()));
		}

		static PyUtil::FPyHashType Hash(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return -1;
			}

			const PyUtil::FPyHashType PyHash = (PyUtil::FPyHashType)GetTypeHash(InSelf->ObjectInstance);
			return PyHash != -1 ? PyHash : 0;
		}
	};

	struct FMethods
	{
		static PyObject* PostInit(FPyWrapperObject* InSelf)
		{
			Py_RETURN_NONE;
		}

		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:cast", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperObject::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* GetDefaultObject(PyTypeObject* InType)
		{
			UClass* Class = FPyWrapperObjectMetaData::GetClass(InType);
			UObject* CDO = Class ? Class->GetDefaultObject() : nullptr;
			return PyConversion::Pythonize(CDO);
		}

		static PyObject* StaticClass(PyTypeObject* InType)
		{
			UClass* Class = FPyWrapperObjectMetaData::GetClass(InType);
			return PyConversion::PythonizeClass(Class);
		}

		static PyObject* GetClass(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::PythonizeClass(InSelf->ObjectInstance->GetClass());
		}

		static PyObject* GetOuter(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetOuter());
		}

		static PyObject* GetTypedOuter(FPyWrapperObject* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyOuterType = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:get_typed_outer", &PyOuterType))
			{
				return nullptr;
			}

			UClass* OuterType = nullptr;
			if (!PyConversion::NativizeClass(PyOuterType, OuterType, UObject::StaticClass()))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetTypedOuter(OuterType));
		}

		static PyObject* GetOutermost(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetOutermost());
		}

		static PyObject* IsPackageExternal(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const bool bIsPackageExternal = InSelf->ObjectInstance->IsPackageExternal();
			return PyConversion::Pythonize(bIsPackageExternal);
		}

		static PyObject* GetPackage(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetPackage());
		}

		static PyObject* GetName(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetName());
		}

		static PyObject* GetFName(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetFName());
		}

		static PyObject* GetFullName(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetFullName());
		}

		static PyObject* GetPathName(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetPathName());
		}

		static PyObject* GetWorld(FPyWrapperObject* InSelf)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return PyConversion::Pythonize(InSelf->ObjectInstance->GetWorld());
		}

		static PyObject* Modify(FPyWrapperObject* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyAlwaysMarkDirty = nullptr;
			if (!PyArg_ParseTuple(InArgs, "|O:modify", &PyAlwaysMarkDirty))
			{
				return nullptr;
			}

			bool bAlwaysMarkDirty = true;
			if (PyAlwaysMarkDirty && !PyConversion::Nativize(PyAlwaysMarkDirty, bAlwaysMarkDirty))
			{
				return nullptr;
			}

			const bool bResult = InSelf->ObjectInstance->Modify(bAlwaysMarkDirty);
			return PyConversion::Pythonize(bResult);
		}

		static PyObject* Rename(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyNameObj = nullptr;
			PyObject* PyOuterObj = nullptr;

			static const char *ArgsKwdList[] = { "name", "outer", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "|OO:rename", (char**)ArgsKwdList, &PyNameObj, &PyOuterObj))
			{
				return nullptr;
			}

			FName NewName;
			if (PyNameObj && PyNameObj != Py_None && !PyConversion::Nativize(PyNameObj, NewName))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
				return nullptr;
			}

			UObject* NewOuter = nullptr;
			if (PyOuterObj && !PyConversion::Nativize(PyOuterObj, NewOuter))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'outer' (%s) to 'Object'"), *PyUtil::GetFriendlyTypename(PyOuterObj)));
				return nullptr;
			}

			const bool bResult = InSelf->ObjectInstance->Rename(NewName.IsNone() ? nullptr : *NewName.ToString(), NewOuter);

			return PyConversion::Pythonize(bResult);
		}

		static PyGenUtil::FGeneratedWrappedProperty FindEditorPropertyImpl(FPyWrapperObject* InSelf, const FName InPythonPropName)
		{
			PyGenUtil::FGeneratedWrappedProperty WrappedPropDef;

			const UClass* Class = InSelf->ObjectInstance->GetClass();
			const FName ResolvedName = FPyWrapperObjectMetaData::ResolvePropertyName(InSelf, InPythonPropName);

			const FProperty* ResolvedProp = Class->FindPropertyByName(ResolvedName);
			if (InSelf->ObjectInstance->HasAnyFlags(RF_ClassDefaultObject) && (!ResolvedProp || ResolvedProp->HasAnyPropertyFlags(CPF_Deprecated)))
			{
				// Look for a sparse member of the same name - the sparse data is treated as an extension of the class default object by the details panel
				if (const UStruct* SparseDataStruct = Class->GetSparseClassDataStruct())
				{
					if (const FProperty* SparseProp = SparseDataStruct->FindPropertyByName(ResolvedName))
					{
						ResolvedProp = SparseProp;
					}
				}
			}

			if (!ResolvedProp)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Failed to find property '%s' for attribute '%s' on '%s'"), *ResolvedName.ToString(), *InPythonPropName.ToString(), *Class->GetName()));
				return WrappedPropDef;
			}

			// If the owner class is set then this property is from 'self', otherwise it's from the sparse class data
			const bool bPropertyIsOwnedBySelf = ResolvedProp->GetOwnerClass() != nullptr;

			TOptional<FString> PropDeprecationMessage;
			if (bPropertyIsOwnedBySelf)
			{
				FString PropDeprecationMessageStr;
				if (FPyWrapperObjectMetaData::IsPropertyDeprecated(InSelf, InPythonPropName, &PropDeprecationMessageStr))
				{
					PropDeprecationMessage = MoveTemp(PropDeprecationMessageStr);
				}
			}
			
			if (PropDeprecationMessage.IsSet())
			{
				WrappedPropDef.SetProperty(ResolvedProp, PyGenUtil::FGeneratedWrappedProperty::SPF_None);
				WrappedPropDef.DeprecationMessage = MoveTemp(PropDeprecationMessage);
			}
			else
			{
				WrappedPropDef.SetProperty(ResolvedProp);
			}

			return WrappedPropDef;
		}

		static PyObject* GetEditorProperty(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyNameObj = nullptr;

			static const char *ArgsKwdList[] = { "name", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:get_editor_property", (char**)ArgsKwdList, &PyNameObj))
			{
				return nullptr;
			}

			FName Name;
			if (!PyConversion::Nativize(PyNameObj, Name))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedProperty WrappedPropDef = FindEditorPropertyImpl(InSelf, Name);
			if (!WrappedPropDef.Prop)
			{
				return nullptr;
			}

			// If the owner class is set then this property is from 'self', otherwise it's from the sparse class data
			const bool bPropertyIsOwnedBySelf = WrappedPropDef.Prop->GetOwnerClass() != nullptr;
			UClass* Class = InSelf->ObjectInstance->GetClass();
			if (bPropertyIsOwnedBySelf)
			{
				return PyGenUtil::GetPropertyValue(Class, InSelf->ObjectInstance, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), (PyObject*)InSelf, *PyUtil::GetErrorContext(InSelf));
			}
			else
			{
				const UScriptStruct* SparseDataStruct = Class->GetSparseClassDataStruct();
				const void* SparseData = Class->GetSparseClassData(EGetSparseClassDataMethod::ArchetypeIfNull);
				if (SparseDataStruct && SparseData)
				{
					return PyGenUtil::GetPropertyValue(SparseDataStruct, SparseData, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), (PyObject*)InSelf, *PyUtil::GetErrorContext(InSelf));
				}
				else
				{
					PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Failed to get sparse data for '%s'"), *Class->GetName()));
					return nullptr;
				}
			}
		}

		static PyObject* SetEditorProperty(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyNameObj = nullptr;
			PyObject* PyValueObj = nullptr;
			PyObject* PyNotifyModeObj = nullptr;

			static const char *ArgsKwdList[] = { "name", "value", "notify_mode", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO|O:set_editor_property", (char**)ArgsKwdList, &PyNameObj, &PyValueObj, &PyNotifyModeObj))
			{
				return nullptr;
			}

			FName Name;
			if (!PyConversion::Nativize(PyNameObj, Name))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
				return nullptr;
			}

			static const UEnum* PropertyAccessChangeNotifyModeEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/CoreUObject.EPropertyAccessChangeNotifyMode"));

			EPropertyAccessChangeNotifyMode NotifyMode = EPropertyAccessChangeNotifyMode::Default;
			if (PyNotifyModeObj && !PyConversion::NativizeEnumEntry(PyNotifyModeObj, PropertyAccessChangeNotifyModeEnum, NotifyMode))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'notify_mode' (%s) to 'PropertyAccessChangeNotifyMode'"), *PyUtil::GetFriendlyTypename(PyNotifyModeObj)));
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedProperty WrappedPropDef = FindEditorPropertyImpl(InSelf, Name);
			if (!WrappedPropDef.Prop)
			{
				return nullptr;
			}

			// If the object is a template, gather instances (including subclass CDOs) inheriting their property value from this template.
			// Those instances are passed into PyGenUtil::SetPropertyValue so that they will receive the value change as well.
			TArray<void*> ArchetypeInsts;
			const bool bIsObjectTemplate = PropertyAccessUtil::IsObjectTemplate(InSelf->ObjectInstance);
			if (bIsObjectTemplate)
			{
				PropertyAccessUtil::GetArchetypeInstancesInheritingPropertyValue_AsContainerData(WrappedPropDef.Prop, InSelf->ObjectInstance, ArchetypeInsts);
			}

			// If the owner class is set then this property is from 'self', otherwise it's from the sparse class data
			const bool bPropertyIsOwnedBySelf = WrappedPropDef.Prop->GetOwnerClass() != nullptr;
			UClass* Class = InSelf->ObjectInstance->GetClass();
			if (bPropertyIsOwnedBySelf)
			{
				const TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = FPyWrapperOwnerContext((PyObject*)InSelf, WrappedPropDef.Prop).BuildChangeNotify(NotifyMode);
				const int Result = PyGenUtil::SetPropertyValue(Class, InSelf->ObjectInstance, PyValueObj, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), ChangeNotify.Get(), PropertyAccessUtil::EditorReadOnlyFlags, bIsObjectTemplate, *PyUtil::GetErrorContext(InSelf), ArchetypeInsts);
				if (Result != 0)
				{
					return nullptr;
				}
			}
			else
			{
				const UScriptStruct* SparseDataStruct = Class->GetSparseClassDataStruct();
				void* SparseData = Class->GetOrCreateSparseClassData();
				if (SparseDataStruct && SparseData)
				{
					const TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = FPyWrapperOwnerContext((PyObject*)InSelf, WrappedPropDef.Prop).BuildChangeNotify(NotifyMode);
					const int Result = PyGenUtil::SetPropertyValue(SparseDataStruct, SparseData, PyValueObj, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), ChangeNotify.Get(), PropertyAccessUtil::EditorReadOnlyFlags, bIsObjectTemplate, *PyUtil::GetErrorContext(InSelf), ArchetypeInsts);
					if (Result != 0)
					{
						return nullptr;
					}
				}
				else
				{
					PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Failed to get sparse data for '%s'"), *Class->GetName()));
					return nullptr;
				}
			}

			Py_RETURN_NONE;
		}

		static PyObject* SetEditorProperties(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyPropertyInfoDict = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:set_editor_properties", &PyPropertyInfoDict))
			{
				return nullptr;
			}

			if (!PyUtil::IsMappingType(PyPropertyInfoDict))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("'property_info' (%s) is expected to be a mapping type (eg, dict)"), *PyUtil::GetFriendlyTypename(PyPropertyInfoDict)));
				return nullptr;
			}

			// Build up the list of properties and their new values
			typedef TTuple<FName, PyGenUtil::FGeneratedWrappedProperty, FPyObjectPtr> FPropertyInfoPair;
			TArray<FPropertyInfoPair> PropertyInfos;
			{
				FPyObjectPtr PyPropertyInfoDictIter = FPyObjectPtr::StealReference(PyObject_GetIter(PyPropertyInfoDict));
				if (PyPropertyInfoDictIter)
				{
					FPyObjectPtr PyKeyItem;
					while ((PyKeyItem = FPyObjectPtr::StealReference(PyIter_Next(PyPropertyInfoDictIter))))
					{
						FName Name;
						if (!PyConversion::Nativize(PyKeyItem, Name))
						{
							PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert dict key (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyKeyItem)));
							return nullptr;
						}

						PyGenUtil::FGeneratedWrappedProperty WrappedPropDef = FindEditorPropertyImpl(InSelf, Name);
						if (!WrappedPropDef.Prop)
						{
							return nullptr;
						}

						FPyObjectPtr PyValueItem = FPyObjectPtr::StealReference(PyObject_GetItem(PyPropertyInfoDict, PyKeyItem));
						if (ensure(PyValueItem))
						{
							PropertyInfos.Add(MakeTuple(Name, MoveTemp(WrappedPropDef), MoveTemp(PyValueItem)));
						}
					}

					// Iteration error?
					if (PyErr_Occurred())
					{
						return nullptr;
					}
				}
			}

			// At this point we know that every property we were asked to set was found, but not that the values are going to be compatible
			// Checking value coercion is tricky without actually doing the conversion, so we'll assume that this point that we need to do the change notifies
			InSelf->ObjectInstance->PreEditChange(nullptr);
			ON_SCOPE_EXIT
			{
				InSelf->ObjectInstance->PostEditChange();
			};

			// Try and set the value of each property
			UClass* Class = InSelf->ObjectInstance->GetClass();
			const bool bIsObjectTemplate = PropertyAccessUtil::IsObjectTemplate(InSelf->ObjectInstance);
			for (const FPropertyInfoPair& PropertyInfo : PropertyInfos)
			{
				const FName Name = PropertyInfo.Get<0>();
				const PyGenUtil::FGeneratedWrappedProperty& WrappedPropDef = PropertyInfo.Get<1>();
				PyObject* PyValueObj = PropertyInfo.Get<2>().GetPtr();

				// If the object is a template, gather instances (including subclass CDOs) inheriting their property value from this template.
				// Those instances are passed into PyGenUtil::SetPropertyValue so that they will receive the value change as well.
				// Each property can have a different set of archetype instances that are inheriting the value.
				TArray<void*> ArchetypeInsts;
				if (bIsObjectTemplate)
				{
					PropertyAccessUtil::GetArchetypeInstancesInheritingPropertyValue_AsContainerData(WrappedPropDef.Prop, InSelf->ObjectInstance, ArchetypeInsts);
				}

				// If the owner class is set then this property is from 'self', otherwise it's from the sparse class data
				const bool bPropertyIsOwnedBySelf = WrappedPropDef.Prop->GetOwnerClass() != nullptr;
				if (bPropertyIsOwnedBySelf)
				{
					const int Result = PyGenUtil::SetPropertyValue(Class, InSelf->ObjectInstance, PyValueObj, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), nullptr, PropertyAccessUtil::EditorReadOnlyFlags, bIsObjectTemplate, *PyUtil::GetErrorContext(InSelf), ArchetypeInsts);
					if (Result != 0)
					{
						return nullptr;
					}
				}
				else
				{
					const UScriptStruct* SparseDataStruct = Class->GetSparseClassDataStruct();
					void* SparseData = Class->GetOrCreateSparseClassData();
					if (SparseDataStruct && SparseData)
					{
						const int Result = PyGenUtil::SetPropertyValue(SparseDataStruct, SparseData, PyValueObj, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), nullptr, PropertyAccessUtil::EditorReadOnlyFlags, bIsObjectTemplate, *PyUtil::GetErrorContext(InSelf), ArchetypeInsts);
						if (Result != 0)
						{
							return nullptr;
						}
					}
					else
					{
						PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Failed to get sparse data for '%s'"), *Class->GetName()));
						return nullptr;
					}
				}
			}

			Py_RETURN_NONE;
		}

		static PyObject* CallMethod(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperObject::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyNameObj = nullptr;
			PyObject* PyArgsObj = nullptr;
			PyObject* PyKwargsObj = nullptr;

			static const char* ArgsKwdList[] = { "name", "args", "kwargs", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O|OO:call_method", (char**)ArgsKwdList, &PyNameObj, &PyArgsObj, &PyKwargsObj))
			{
				return nullptr;
			}

			FName Name;
			if (!PyConversion::Nativize(PyNameObj, Name))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyNameObj)));
				return nullptr;
			}

			// Args must be a tuple
			if (PyArgsObj && !PyTuple_Check(PyArgsObj))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("'args' (%s) must be 'tuple'"), *PyUtil::GetFriendlyTypename(PyArgsObj)));
				return nullptr;
			}

			// Kwargs must be a dict
			if (PyKwargsObj && !PyDict_Check(PyKwargsObj))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("'kwargs' (%s) must be 'dict'"), *PyUtil::GetFriendlyTypename(PyKwargsObj)));
				return nullptr;
			}

			// Find the named function we want to call
			const UFunction* Func = InSelf->ObjectInstance->GetClass()->FindFunctionByName(Name);
			if (!Func)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Failed to find function '%s' on '%s'"), *Name.ToString(), *InSelf->ObjectInstance->GetClass()->GetName()));
				return nullptr;
			}

			// Args must always be set to a tuple, even if empty, for the parameter validation to work
			FPyObjectPtr EmptyArgs;
			if (!PyArgsObj)
			{
				EmptyArgs = FPyObjectPtr::StealReference(PyTuple_New(0));
				PyArgsObj = EmptyArgs;
			}

			// Build temporary glue for this function and call it
			const FString PythonFunctionName = PyGenUtil::PythonizeName(Func->GetName(), PyGenUtil::EPythonizeNameCase::Lower);
			PyGenUtil::FGeneratedWrappedFunction GeneratedWrappedFunction;
			GeneratedWrappedFunction.SetFunction(Func);
			return FPyWrapperObject::CallFunction(InSelf, PyArgsObj, PyKwargsObj, GeneratedWrappedFunction, TCHAR_TO_UTF8(*PythonFunctionName));
		}
	};

	// NOTE: _T = typing.TypeVar('_T') and Any/Type/Union/Mapping/Optional are defines by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ PyGenUtil::PostInitFuncName, PyCFunctionCast(&FMethods::PostInit), METH_NOARGS, "_post_init(self) -> None -- called during Unreal object initialization (equivalent to PostInitProperties in C++)" },
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls: Type[_T], object: object) -> _T -- cast the given object to this Unreal object type or raise an exception if the cast is not possible" },
		{ "get_default_object", PyCFunctionCast(&FMethods::GetDefaultObject), METH_NOARGS | METH_CLASS, "get_default_object(cls: Type[_T]) -> _T -- get the Unreal class default object (CDO) of this type" },
		{ "static_class", PyCFunctionCast(&FMethods::StaticClass), METH_NOARGS | METH_CLASS, "static_class(cls) -> Class -- get the Unreal class of this type" },
		{ "get_class", PyCFunctionCast(&FMethods::GetClass), METH_NOARGS, "get_class(self) -> Class -- get the Unreal class of this instance" },
		{ "get_outer", PyCFunctionCast(&FMethods::GetOuter), METH_NOARGS, "get_outer(self) -> Any -- get the outer object from this instance (if any)" },
		{ "get_typed_outer", PyCFunctionCast(&FMethods::GetTypedOuter), METH_VARARGS, "get_typed_outer(self, type: Union[Class, type]) -> Any -- get the first outer object of the given type from this instance (if any)" },
		{ "get_outermost", PyCFunctionCast(&FMethods::GetOutermost), METH_NOARGS, "get_outermost(self) -> Package -- get the outermost object (the package) from this instance" },
		{ "is_package_external", PyCFunctionCast(&FMethods::IsPackageExternal), METH_NOARGS, "is_package_external(self) -> bool -- returns true if this instance has a different package than its outer's package" },
		{ "get_package", PyCFunctionCast(&FMethods::GetPackage), METH_NOARGS, "get_package(self) -> Package -- get the package directly associated with this instance" },
		{ "get_name", PyCFunctionCast(&FMethods::GetName), METH_NOARGS, "get_name(self) -> str -- get the name of this instance" },
		{ "get_fname", PyCFunctionCast(&FMethods::GetFName), METH_NOARGS, "get_fname(self) -> Name -- get the name of this instance" },
		{ "get_full_name", PyCFunctionCast(&FMethods::GetFullName), METH_NOARGS, "get_full_name(self) -> str -- get the full name (class name + full path) of this instance" },
		{ "get_path_name", PyCFunctionCast(&FMethods::GetPathName), METH_NOARGS, "get_path_name(self) -> str -- get the path name of this instance" },
		{ "get_world", PyCFunctionCast(&FMethods::GetWorld), METH_NOARGS, "get_world(self) -> Optional[World] -- get the world associated with this instance (if any)" },
		{ "modify", PyCFunctionCast(&FMethods::Modify), METH_VARARGS, "modify(self, always_mark_dirty: bool = True) -> bool -- inform that this instance is about to be modified (tracks changes for undo/redo if transactional)" },
		{ "rename", PyCFunctionCast(&FMethods::Rename), METH_VARARGS | METH_KEYWORDS, "rename(self, name: Union[Name, str]=\"None\", outer: Optional[Object]=None) -> bool -- rename this instance and/or change its outer" },
		{ "get_editor_property", PyCFunctionCast(&FMethods::GetEditorProperty), METH_VARARGS | METH_KEYWORDS, "get_editor_property(self, name: str) -> object -- get the value of any property visible to the editor" },
		{ "set_editor_property", PyCFunctionCast(&FMethods::SetEditorProperty), METH_VARARGS | METH_KEYWORDS, "set_editor_property(self, name: str, value: object, notify_mode: PropertyAccessChangeNotifyMode=PropertyAccessChangeNotifyMode.DEFAULT) -> None -- set the value of any property visible to the editor, ensuring that the pre/post change notifications are called" },
		{ "set_editor_properties", PyCFunctionCast(&FMethods::SetEditorProperties), METH_VARARGS, "set_editor_properties(self, properties: Mapping[str, object]) -> None -- set the value of any properties visible to the editor (from a name->value dict), ensuring that the pre/post change notifications are called" },
		{ "call_method", PyCFunctionCast(&FMethods::CallMethod), METH_VARARGS | METH_KEYWORDS, "call_method(self, name: str, *args: Tuple[object, ...], **kwargs: Mapping[str, object]) -> Any -- call a method on this object via Unreal reflection using the given ordered (a tuple passed as args) or named (a dict passed as kwargs) argument data - allows calling methods that don't have Python glue" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"_ObjectBase", /* tp_name */
		sizeof(FPyWrapperObject), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;
	PyType.tp_doc = "Type for all Unreal exposed object instances";

	return PyType;
}

PyTypeObject PyWrapperObjectType = InitializePyWrapperObjectType();

void FPyWrapperObjectMetaData::AddTypeReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Class);
}

void FPyWrapperObjectMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperObject* Self = static_cast<FPyWrapperObject*>(Instance);

	UObject* OldInstance = Self->ObjectInstance;
	Collector.AddReferencedObject(Self->ObjectInstance);

	// Update the wrapped instance in the object factory
	if (Self->ObjectInstance != OldInstance)
	{
		FPyWrapperObjectFactory::Get().UnmapInstance(OldInstance, Py_TYPE(Self));
	}

	// Update the object type
	if (Self->ObjectInstance != OldInstance && Self->ObjectInstance)
	{
		// Object instance has been re-pointed, make sure we're still the correct type
		PyTypeObject* NewPyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(Self->ObjectInstance->GetClass());
		if (PyType_IsSubtype(NewPyType, &PyWrapperObjectType) && NewPyType->tp_basicsize == Py_TYPE(Self)->tp_basicsize)
		{
			PyTypeObject* OldPyType = Py_TYPE(Self);
			Py_INCREF(NewPyType);   // balance the future Py_DECREF in subtype_dealloc
			Py_SET_TYPE(Self, NewPyType);
			Py_DECREF(OldPyType);   // balance the Py_INCREF from tp_alloc
		}
		else
		{
			Self->ObjectInstance = nullptr;
		}
	}

	// Update the wrapped instance in the object factory
	if (Self->ObjectInstance != OldInstance && Self->ObjectInstance)
	{
		FPyWrapperObjectFactory::Get().MapInstance(Self->ObjectInstance, Self);
	}

	// We also need to ARO delegates on this object to catch ones that are wrapping Python callables (also recursing into nested structs and containers)
	if (Self->ObjectInstance)
	{
		FPyReferenceCollector::AddReferencedObjectsFromStruct(Collector, Self->ObjectInstance->GetClass(), Self->ObjectInstance, EPyReferenceCollectorFlags::IncludeDelegates | EPyReferenceCollectorFlags::IncludeStructs | EPyReferenceCollectorFlags::IncludeContainers);
	}
}

UClass* FPyWrapperObjectMetaData::GetClass(PyTypeObject* PyType)
{
	FPyWrapperObjectMetaData* PyWrapperMetaData = FPyWrapperObjectMetaData::GetMetaData(PyType);
	return PyWrapperMetaData ? PyWrapperMetaData->Class : nullptr;
}

UClass* FPyWrapperObjectMetaData::GetClass(FPyWrapperObject* Instance)
{
	return GetClass(Py_TYPE(Instance));
}

FName FPyWrapperObjectMetaData::ResolvePropertyName(PyTypeObject* PyType, const FName InPythonPropertyName)
{
	if (FPyWrapperObjectMetaData* PyWrapperMetaData = FPyWrapperObjectMetaData::GetMetaData(PyType))
	{
		if (const FName* MappedPropName = PyWrapperMetaData->PythonProperties.Find(InPythonPropertyName))
		{
			return *MappedPropName;
		}

		if (const UClass* SuperClass = PyWrapperMetaData->Class ? PyWrapperMetaData->Class->GetSuperClass() : nullptr)
		{
			PyTypeObject* SuperClassPyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(SuperClass);
			return ResolvePropertyName(SuperClassPyType, InPythonPropertyName);
		}
	}

	return InPythonPropertyName;
}

FName FPyWrapperObjectMetaData::ResolvePropertyName(FPyWrapperObject* Instance, const FName InPythonPropertyName)
{
	return ResolvePropertyName(Py_TYPE(Instance), InPythonPropertyName);
}

bool FPyWrapperObjectMetaData::IsPropertyDeprecated(PyTypeObject* PyType, const FName InPythonPropertyName, FString* OutDeprecationMessage)
{
	if (FPyWrapperObjectMetaData* PyWrapperMetaData = FPyWrapperObjectMetaData::GetMetaData(PyType))
	{
		if (const FString* DeprecationMessage = PyWrapperMetaData->PythonDeprecatedProperties.Find(InPythonPropertyName))
		{
			if (OutDeprecationMessage)
			{
				*OutDeprecationMessage = *DeprecationMessage;
			}
			return true;
		}

		if (const UClass* SuperClass = PyWrapperMetaData->Class ? PyWrapperMetaData->Class->GetSuperClass() : nullptr)
		{
			PyTypeObject* SuperClassPyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(SuperClass);
			return IsPropertyDeprecated(SuperClassPyType, InPythonPropertyName, OutDeprecationMessage);
		}
	}

	return false;
}

bool FPyWrapperObjectMetaData::IsPropertyDeprecated(FPyWrapperObject* Instance, const FName InPythonPropertyName, FString* OutDeprecationMessage)
{
	return IsPropertyDeprecated(Py_TYPE(Instance), InPythonPropertyName, OutDeprecationMessage);
}

FName FPyWrapperObjectMetaData::ResolveFunctionName(PyTypeObject* PyType, const FName InPythonMethodName)
{
	if (FPyWrapperObjectMetaData* PyWrapperMetaData = FPyWrapperObjectMetaData::GetMetaData(PyType))
	{
		if (const FName* MappedFuncName = PyWrapperMetaData->PythonMethods.Find(InPythonMethodName))
		{
			return *MappedFuncName;
		}

		if (const UClass* SuperClass = PyWrapperMetaData->Class ? PyWrapperMetaData->Class->GetSuperClass() : nullptr)
		{
			PyTypeObject* SuperClassPyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(SuperClass);
			return ResolveFunctionName(SuperClassPyType, InPythonMethodName);
		}
	}

	return InPythonMethodName;
}

FName FPyWrapperObjectMetaData::ResolveFunctionName(FPyWrapperObject* Instance, const FName InPythonMethodName)
{
	return ResolveFunctionName(Py_TYPE(Instance), InPythonMethodName);
}

bool FPyWrapperObjectMetaData::IsFunctionDeprecated(PyTypeObject* PyType, const FName InPythonMethodName, FString* OutDeprecationMessage)
{
	if (FPyWrapperObjectMetaData* PyWrapperMetaData = FPyWrapperObjectMetaData::GetMetaData(PyType))
	{
		if (const FString* DeprecationMessage = PyWrapperMetaData->PythonDeprecatedMethods.Find(InPythonMethodName))
		{
			if (OutDeprecationMessage)
			{
				*OutDeprecationMessage = *DeprecationMessage;
			}
			return true;
		}

		if (const UClass* SuperClass = PyWrapperMetaData->Class ? PyWrapperMetaData->Class->GetSuperClass() : nullptr)
		{
			PyTypeObject* SuperClassPyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(SuperClass);
			return IsFunctionDeprecated(SuperClassPyType, InPythonMethodName, OutDeprecationMessage);
		}
	}

	return false;
}

bool FPyWrapperObjectMetaData::IsFunctionDeprecated(FPyWrapperObject* Instance, const FName InPythonMethodName, FString* OutDeprecationMessage)
{
	return IsFunctionDeprecated(Py_TYPE(Instance), InPythonMethodName, OutDeprecationMessage);
}

bool FPyWrapperObjectMetaData::IsClassDeprecated(PyTypeObject* PyType, FString* OutDeprecationMessage)
{
	if (FPyWrapperObjectMetaData* PyWrapperMetaData = FPyWrapperObjectMetaData::GetMetaData(PyType))
	{
		if (PyWrapperMetaData->DeprecationMessage.IsSet())
		{
			if (OutDeprecationMessage)
			{
				*OutDeprecationMessage = PyWrapperMetaData->DeprecationMessage.GetValue();
			}
			return true;
		}
	}

	return false;
}

bool FPyWrapperObjectMetaData::IsClassDeprecated(FPyWrapperObject* Instance, FString* OutDeprecationMessage)
{
	return IsClassDeprecated(Py_TYPE(Instance), OutDeprecationMessage);
}

class FPythonGeneratedClassBuilder
{
public:
	FPythonGeneratedClassBuilder(UClass* InSuperClass, PyTypeObject* InPyType)
		: ClassName()
		, PyType(InPyType)
		, OldClass(nullptr)
		, NewClass(nullptr)
	{
		UObject* ClassOuter = nullptr;
		PyUtil::GetGeneratedTypeOuterAndName(PyType, ClassOuter, ClassName);

		// Find any existing class with the name we want to use
		OldClass = FindObject<UPythonGeneratedClass>(ClassOuter, *ClassName);

		// Create a new class with a temporary name; we will rename it as part of Finalize
		const FString NewClassName = MakeUniqueObjectName(ClassOuter, UPythonGeneratedClass::StaticClass(), *FString::Printf(TEXT("%s_NEWINST"), *ClassName)).ToString();
		NewClass = NewObject<UPythonGeneratedClass>(ClassOuter, *NewClassName, RF_Public | RF_Transient);
		NewClass->SetMetaData(TEXT("DisplayName"), *PyUtil::GetGeneratedTypeDisplayName(PyType));
		NewClass->SetMetaData(TEXT("BlueprintType"), TEXT("true"));
		NewClass->SetSuperStruct(InSuperClass);
	}

	FPythonGeneratedClassBuilder(UPythonGeneratedClass* InOldClass, UClass* InSuperClass)
		: ClassName(InOldClass->GetName())
		, PyType(InOldClass->PyType)
		, OldClass(InOldClass)
		, NewClass(nullptr)
	{
		UObject* ClassOuter = InOldClass->GetOuter();

		// Create a new class with a temporary name; we will rename it as part of Finalize
		const FString NewClassName = MakeUniqueObjectName(ClassOuter, UPythonGeneratedClass::StaticClass(), *FString::Printf(TEXT("%s_NEWINST"), *ClassName)).ToString();
		NewClass = NewObject<UPythonGeneratedClass>(ClassOuter, *NewClassName, RF_Public | RF_Transient);
		NewClass->SetMetaData(TEXT("DisplayName"), *PyUtil::GetGeneratedTypeDisplayName(PyType));
		NewClass->SetMetaData(TEXT("BlueprintType"), TEXT("true"));
		NewClass->SetSuperStruct(InSuperClass);
	}

	~FPythonGeneratedClassBuilder()
	{
		// If NewClass is still set at this point, if means Finalize wasn't called and we should destroy the partially built class
		if (NewClass)
		{
			NewClass->ClearFlags(RF_Public | RF_Standalone);
			NewClass = nullptr;

			Py_BEGIN_ALLOW_THREADS
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			Py_END_ALLOW_THREADS
		}
	}

	UPythonGeneratedClass* Finalize(FPyObjectPtr InPyPostInitFunction)
	{
		// Set the post-init function
		NewClass->PyPostInitFunction = InPyPostInitFunction;
		if (!NewClass->PyPostInitFunction)
		{
			return nullptr;
		}

		// Replace the definitions with real descriptors
		if (!RegisterDescriptors())
		{
			return nullptr;
		}

		// Generate ToolTip using the Python doc string
		if (PyType->tp_doc)
		{
			static const FName ToolTipKey = "ToolTip";
			NewClass->SetMetaData(ToolTipKey, UTF8_TO_TCHAR(PyType->tp_doc));
		}

		// Let Python know that we've changed its type
		PyType_Modified(PyType);

		// We can no longer fail, so prepare the old class for removal and set the correct name on the new class
		if (OldClass)
		{
			PrepareOldClassForReinstancing();
		}

		const FString CleanPyTypeName = PyUtil::GetCleanTypename(PyType);

		FNameBuilder CleanCDOName;
		CleanCDOName.Append(DEFAULT_OBJECT_PREFIX);
		CleanCDOName.Append(CleanPyTypeName);

		Py_BEGIN_ALLOW_THREADS
		NewClass->ClassRedirector = CreatePythonTypeLegacyRedirector(CleanPyTypeName, FTopLevelAssetPath(NewClass->GetOuter()->GetFName(), *ClassName));
		if (NewClass->ClassRedirector)
		{
			NewClass->ClassRedirector->DestinationObject = NewClass;
		}
		else
		{
			// If the class doesn't have a redirector, then we also need to remove any existing redirector for the CDO (before the real CDO is created below)
			if (UObjectRedirector* ExistingCDORedirector = FindObject<UObjectRedirector>(GetPythonTypeContainer(), *CleanCDOName))
			{
				ExistingCDORedirector->DestinationObject = nullptr;
				ExistingCDORedirector->ClearFlags(RF_Public | RF_Standalone);
				ExistingCDORedirector->Rename(*FString::Printf(TEXT("%s_DELETED"), *CleanCDOName), nullptr, REN_DontCreateRedirectors);
			}
		}
		NewClass->Rename(*ClassName, nullptr, REN_DontCreateRedirectors);

		// Finalize the class
		NewClass->Bind();
		NewClass->StaticLink(true);
		NewClass->AssembleReferenceTokenStream();
		Py_END_ALLOW_THREADS

		// Add the object meta-data to the type
		NewClass->PyMetaData.Class = NewClass;
		FPyWrapperObjectMetaData::SetMetaData(PyType, &NewClass->PyMetaData);

		// Map the Unreal class to the Python type
		NewClass->PyType = FPyTypeObjectPtr::NewReference(PyType);
		FPyWrapperTypeRegistry::Get().RegisterWrappedClassType(NewClass, PyType, false);

		// Ensure the CDO exists and add a redirector for it if needed
		Py_BEGIN_ALLOW_THREADS
		if (UObject* NewClassCDO = NewClass->GetDefaultObject())
		{
			if (NewClass->ClassRedirector)
			{
				// If the class has a redirector, then we also need to redirect the CDO
				NewClass->CDORedirector = NewObject<UObjectRedirector>(GetPythonTypeContainer(), FName(CleanCDOName), RF_Public | RF_Transient);
				NewClass->CDORedirector->DestinationObject = NewClassCDO;
			}
		}
		Py_END_ALLOW_THREADS

		// Re-instance the old class and re-parent any derived classes to this new type
		if (OldClass)
		{
			FPyWrapperTypeReinstancer::Get().AddPendingClass(OldClass, NewClass);
			UPythonGeneratedClass::ReparentDerivedClasses(OldClass, NewClass);
		}	

		if (FBlueprintActionDatabase* ActionDB = FBlueprintActionDatabase::TryGet())
		{
			Py_BEGIN_ALLOW_THREADS
			// Notify Blueprints that there is a new class to add to the action list
			ActionDB->RefreshClassActions(NewClass);

			if (OldClass)
			{
				ActionDB->RefreshClassActions(OldClass);
			}
			Py_END_ALLOW_THREADS
		}

		// Null the NewClass pointer so the destructor doesn't kill it
		UPythonGeneratedClass* FinalizedClass = NewClass;
		NewClass = nullptr;
		return FinalizedClass;
	}

	bool CreatePropertyFromDefinition(const FString& InFieldName, FPyFPropertyDef* InPyPropDef)
	{
		UClass* SuperClass = NewClass->GetSuperClass();

		// Resolve the property name to match any previously exported properties from the parent type
		const FName PropName = FPyWrapperObjectMetaData::ResolvePropertyName(PyType->tp_base, *InFieldName);
		if (SuperClass->FindPropertyByName(PropName))
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Property '%s' (%s) cannot override a property from the base type"), *InFieldName, *PyUtil::GetFriendlyTypename(InPyPropDef->PropType)));
			return false;
		}

		// Create the property from its definition
		FProperty* Prop = PyUtil::CreateProperty(InPyPropDef->PropType, 1, NewClass, PropName);
		if (!Prop)
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create property for '%s' (%s)"), *InFieldName, *PyUtil::GetFriendlyTypename(InPyPropDef->PropType)));
			return false;
		}
		Prop->PropertyFlags |= (CPF_Edit | CPF_BlueprintVisible);
		FPyFPropertyDef::ApplyMetaData(InPyPropDef, Prop);
		NewClass->AddCppProperty(Prop);

		// Resolve any getter/setter function names
		const FName GetterFuncName = FPyWrapperObjectMetaData::ResolveFunctionName(PyType->tp_base, *InPyPropDef->GetterFuncName);
		const FName SetterFuncName = FPyWrapperObjectMetaData::ResolveFunctionName(PyType->tp_base, *InPyPropDef->SetterFuncName);
		if (!GetterFuncName.IsNone())
		{
			Prop->SetMetaData(PyGenUtil::BlueprintGetterMetaDataKey, *GetterFuncName.ToString());
		}
		if (!SetterFuncName.IsNone())
		{
			Prop->SetMetaData(PyGenUtil::BlueprintSetterMetaDataKey, *SetterFuncName.ToString());
		}

		// Build the definition data for the new property accessor
		PyGenUtil::FPropertyDef& PropDef = *NewClass->PropertyDefs.Add_GetRef(MakeShared<PyGenUtil::FPropertyDef>());
		PropDef.GeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*InFieldName);
		PropDef.GeneratedWrappedGetSet.GetSetDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("type: %s\n%s"), *PyGenUtil::GetPropertyPythonType(Prop), *PyGenUtil::GetFieldTooltip(Prop)));
		PropDef.GeneratedWrappedGetSet.Prop.SetProperty(Prop);
		PropDef.GeneratedWrappedGetSet.GetFunc.SetFunction(NewClass->FindFunctionByName(GetterFuncName));
		PropDef.GeneratedWrappedGetSet.SetFunc.SetFunction(NewClass->FindFunctionByName(SetterFuncName));
		PropDef.GeneratedWrappedGetSet.GetCallback = (getter)&FPyWrapperObject::Getter_Impl;
		PropDef.GeneratedWrappedGetSet.SetCallback = (setter)&FPyWrapperObject::Setter_Impl;
		PropDef.GeneratedWrappedGetSet.ToPython(PropDef.PyGetSet);

		// If this property has a getter or setter, also make an internal version with the get/set function cleared so that Python can read/write the internal property value
		if (PropDef.GeneratedWrappedGetSet.GetFunc.Func || PropDef.GeneratedWrappedGetSet.SetFunc.Func)
		{
			PyGenUtil::FPropertyDef& InternalPropDef = *NewClass->PropertyDefs.Add_GetRef(MakeShared<PyGenUtil::FPropertyDef>());
			InternalPropDef.GeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("_%s"), *InFieldName));
			InternalPropDef.GeneratedWrappedGetSet.GetSetDoc = PropDef.GeneratedWrappedGetSet.GetSetDoc;
			InternalPropDef.GeneratedWrappedGetSet.Prop.SetProperty(Prop);
			InternalPropDef.GeneratedWrappedGetSet.GetCallback = (getter)&FPyWrapperObject::Getter_Impl;
			InternalPropDef.GeneratedWrappedGetSet.SetCallback = (setter)&FPyWrapperObject::Setter_Impl;
			InternalPropDef.GeneratedWrappedGetSet.ToPython(InternalPropDef.PyGetSet);
		}

		return true;
	}

	bool CreateFunctionFromDefinition(const FString& InFieldName, FPyUFunctionDef* InPyFuncDef)
	{
		UClass* SuperClass = NewClass->GetSuperClass();

		// Validate the function definition makes sense
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Override))
		{
			if (EnumHasAnyFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Static | EPyUFunctionDefFlags::Getter | EPyUFunctionDefFlags::Setter))
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' specified as 'override' cannot also specify 'static', 'getter', or 'setter'"), *InFieldName));
				return false;
			}
			if (InPyFuncDef->FuncRetType != Py_None || InPyFuncDef->FuncParamTypes != Py_None)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' specified as 'override' cannot also specify 'ret' or 'params'"), *InFieldName));
				return false;
			}
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Static) && EnumHasAnyFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Getter | EPyUFunctionDefFlags::Setter))
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' specified as 'static' cannot also specify 'getter' or 'setter'"), *InFieldName));
			return false;
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Getter))
		{
			if (EnumHasAnyFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Setter))
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' specified as 'getter' cannot also specify 'setter'"), *InFieldName));
				return false;
			}
			if (EnumHasAnyFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Impure))
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' specified as 'getter' must also specify 'pure=True'"), *InFieldName));
				return false;
			}
		}

		// Resolve the function name to match any previously exported functions from the parent type
		const FName FuncName = FPyWrapperObjectMetaData::ResolveFunctionName(PyType->tp_base, *InFieldName);
		const UFunction* SuperFunc = SuperClass->FindFunctionByName(FuncName);
		if (SuperFunc && !EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Override))
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' cannot override a method from the base type (did you forget to specify 'override=True'?)"), *InFieldName));
			return false;
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Override))
		{
			if (!SuperFunc)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' was set to 'override', but no method was found to override"), *InFieldName));
				return false;
			}
			if (!SuperFunc->HasAnyFunctionFlags(FUNC_BlueprintEvent))
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Method '%s' was set to 'override', but the method found to override was not a blueprint event"), *InFieldName));
				return false;
			}
		}

		// Inspect the argument names and defaults from the Python function
		TArray<FString> FuncArgNames;
		TArray<FPyObjectPtr> FuncArgDefaults;
		if (!PyUtil::InspectFunctionArgs(InPyFuncDef->Func, FuncArgNames, &FuncArgDefaults))
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to inspect the arguments for '%s'"), *InFieldName));
			return false;
		}

		// Create the function, either from the definition, or from the super-function found to override
		NewClass->AddNativeFunction(*FuncName.ToString(), &UPythonGeneratedClass::CallPythonFunction); // Need to do this before the call to DuplicateObject in the case that the super-function already has FUNC_Native
		UFunction* Func = nullptr;
		if (SuperFunc)
		{
			FObjectDuplicationParameters FuncDuplicationParams(const_cast<UFunction*>(SuperFunc), NewClass);
			FuncDuplicationParams.DestName = FuncName;
			FuncDuplicationParams.InternalFlagMask &= ~EInternalObjectFlags::Native;
			Func = CastChecked<UFunction>(StaticDuplicateObjectEx(FuncDuplicationParams));
		}
		else
		{
			Func = NewObject<UFunction>(NewClass, FuncName);
		}
		if (!SuperFunc)
		{
			Func->FunctionFlags |= FUNC_Public;
			// The function is not in the linked list of class fields, insert it so that field iterators & funcs work
			Func->Next = NewClass->Children;
			NewClass->Children = Func;
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Static))
		{
			Func->FunctionFlags |= FUNC_Static;
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Pure))
		{
			Func->FunctionFlags |= FUNC_BlueprintPure;
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Impure))
		{
			Func->FunctionFlags &= ~FUNC_BlueprintPure;
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Getter))
		{
			Func->SetMetaData(PyGenUtil::BlueprintGetterMetaDataKey, TEXT(""));
		}
		if (EnumHasAllFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Setter))
		{
			Func->SetMetaData(PyGenUtil::BlueprintSetterMetaDataKey, TEXT(""));
		}
		Func->FunctionFlags |= (FUNC_Native | FUNC_Event | FUNC_BlueprintEvent | FUNC_BlueprintCallable);
		FPyUFunctionDef::ApplyMetaData(InPyFuncDef, Func);
		NewClass->AddFunctionToFunctionMap(Func, Func->GetFName());
		if (!Func->HasAnyFunctionFlags(FUNC_Static))
		{
			// Check for a malformed function rather than assert in the remove
			if (FuncArgNames.Num() > 0 && FuncArgDefaults.Num() > 0)
			{
				// Strip the zero'th 'self' argument when processing a non-static function
				FuncArgNames.RemoveAt(0, EAllowShrinking::No);
				FuncArgDefaults.RemoveAt(0, EAllowShrinking::No);
			}
			else
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Incorrect number of arguments specified for '%s' (missing self?)"), *InFieldName));
				return false;
			}
		}
		// Build the arguments struct if not overriding a function
		if (!SuperFunc)
		{
			// Make sure the number of function arguments matches the number of argument types specified
			const int32 NumArgTypes = (InPyFuncDef->FuncParamTypes && InPyFuncDef->FuncParamTypes != Py_None) ? PySequence_Size(InPyFuncDef->FuncParamTypes) : 0;
			if (NumArgTypes != FuncArgNames.Num())
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Incorrect number of arguments specified for '%s' (expected %d, got %d)"), *InFieldName, NumArgTypes, FuncArgNames.Num()));
				return false;
			}

			// Adding properties to a function inserts them into a linked list, so we add the return and output values first so that they appear at the end
			if (InPyFuncDef->FuncRetType && InPyFuncDef->FuncRetType != Py_None)
			{
				// If we have a tuple, then we actually want to return a bool but add every type within the tuple as output parameters
				const bool bIsPackedReturn = PyTuple_Check(InPyFuncDef->FuncRetType) && !PyUtil::IsOptionalPropertyDef(InPyFuncDef->FuncRetType);

				PyObject* RetType = bIsPackedReturn ? (PyObject*)&PyBool_Type : InPyFuncDef->FuncRetType;
				FProperty* RetProp = PyUtil::CreateProperty(RetType, 1, Func, TEXT("ReturnValue"));
				if (!RetProp)
				{
					PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create return property (%s) for function '%s'"), *PyUtil::GetFriendlyTypename(RetType), *InFieldName));
					return false;
				}
				RetProp->PropertyFlags |= (CPF_Parm | CPF_OutParm | CPF_ReturnParm);
				Func->AddCppProperty(RetProp);

				if (bIsPackedReturn)
				{
					// Adding properties to a function inserts them into a linked list, so we loop backwards to get the order right
					const int32 NumOutArgs = PyTuple_Size(InPyFuncDef->FuncRetType);
					for (int32 ArgIndex = NumOutArgs - 1; ArgIndex >= 0; --ArgIndex)
					{
						PyObject* ArgTypeObj = PySequence_GetItem(InPyFuncDef->FuncRetType, ArgIndex);
						FProperty* ArgProp = PyUtil::CreateProperty(ArgTypeObj, 1, Func, *FString::Printf(TEXT("OutValue%d"), ArgIndex));
						if (!ArgProp)
						{
							PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create output property (%s) for function '%s' at index %d"), *PyUtil::GetFriendlyTypename(ArgTypeObj), *InFieldName, ArgIndex));
							return false;
						}
						ArgProp->PropertyFlags |= (CPF_Parm | CPF_OutParm);
						Func->AddCppProperty(ArgProp);
						Func->FunctionFlags |= FUNC_HasOutParms;
					}
				}
			}

			// Adding properties to a function inserts them into a linked list, so we loop backwards to get the order right
			for (int32 ArgIndex = FuncArgNames.Num() - 1; ArgIndex >= 0; --ArgIndex)
			{
				PyObject* ArgTypeObj = PySequence_GetItem(InPyFuncDef->FuncParamTypes, ArgIndex);
				FProperty* ArgProp = PyUtil::CreateProperty(ArgTypeObj, 1, Func, *FuncArgNames[ArgIndex]);
				if (!ArgProp)
				{
					PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create property (%s) for function '%s' argument '%s'"), *PyUtil::GetFriendlyTypename(ArgTypeObj), *InFieldName, *FuncArgNames[ArgIndex]));
					return false;
				}
				ArgProp->PropertyFlags |= CPF_Parm;
				Func->AddCppProperty(ArgProp);
			}
		}
		// Apply the defaults to the function arguments and build the Python method params
		PyGenUtil::FGeneratedWrappedFunction GeneratedWrappedFunction;
		GeneratedWrappedFunction.SetFunction(Func);
		// SetFunction doesn't always use the correct names or defaults for generated classes
		for (int32 InputArgIndex = 0; InputArgIndex < GeneratedWrappedFunction.InputParams.Num(); ++InputArgIndex)
		{
			PyGenUtil::FGeneratedWrappedMethodParameter& GeneratedWrappedMethodParam = GeneratedWrappedFunction.InputParams[InputArgIndex];
			const FProperty* Param = GeneratedWrappedMethodParam.ParamProp;

			const FName DefaultValueMetaDataKey = *FString::Printf(TEXT("CPP_Default_%s"), *Param->GetName());

			TOptional<FString> ResolvedDefaultValue;
			if (FuncArgDefaults.IsValidIndex(InputArgIndex) && FuncArgDefaults[InputArgIndex])
			{
				// Convert the default value to the given property...
				PyUtil::FPropValueOnScope DefaultValue(PyUtil::FConstPropOnScope::ExternalReference(Param));
				if (!DefaultValue.IsValid() || !DefaultValue.SetValue(FuncArgDefaults[InputArgIndex], *PyUtil::GetErrorContext(PyType)))
				{
					PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to convert default value for function '%s' argument '%s' (%s)"), *InFieldName, *FuncArgNames[InputArgIndex], *Param->GetClass()->GetName()));
					return false;
				}

				// ... and export it as meta-data
				FString ExportedDefaultValue;
				if (!DefaultValue.GetProp()->ExportText_Direct(ExportedDefaultValue, DefaultValue.GetValue(), DefaultValue.GetValue(), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to export default value for function '%s' argument '%s' (%s)"), *InFieldName, *FuncArgNames[InputArgIndex], *Param->GetClass()->GetName()));
					return false;
				}

				ResolvedDefaultValue = ExportedDefaultValue;
			}
			if (!ResolvedDefaultValue.IsSet() && SuperFunc && SuperFunc->HasAnyFunctionFlags(FUNC_HasDefaults))
			{
				if (SuperFunc->HasMetaData(DefaultValueMetaDataKey))
				{
					ResolvedDefaultValue = SuperFunc->GetMetaData(DefaultValueMetaDataKey);
				}
			}
			if (ResolvedDefaultValue.IsSet())
			{
				Func->SetMetaData(DefaultValueMetaDataKey, *ResolvedDefaultValue.GetValue());
				Func->FunctionFlags |= FUNC_HasDefaults;
			}

			GeneratedWrappedMethodParam.ParamName = PyGenUtil::TCHARToUTF8Buffer(FuncArgNames.IsValidIndex(InputArgIndex) ? *FuncArgNames[InputArgIndex] : *Param->GetName());
			GeneratedWrappedMethodParam.ParamDefaultValue = MoveTemp(ResolvedDefaultValue);
		}
		Func->Bind();
		Func->StaticLink(true);

		if (GeneratedWrappedFunction.InputParams.Num() != FuncArgNames.Num())
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Incorrect number of arguments specified for '%s' (expected %d, got %d)"), *InFieldName, GeneratedWrappedFunction.InputParams.Num(), FuncArgNames.Num()));
			return false;
		}

		// Apply the doc string as the function tooltip
		{
			static const FName ToolTipKey = TEXT("ToolTip");

			const FString DocString = PyUtil::GetDocString(InPyFuncDef->Func);
			if (!DocString.IsEmpty())
			{
				Func->SetMetaData(ToolTipKey, *DocString);
			}
		}

		// Build the definition data for the new method
		PyGenUtil::FFunctionDef& FuncDef = *NewClass->FunctionDefs.Add_GetRef(MakeShared<PyGenUtil::FFunctionDef>());
		FuncDef.GeneratedWrappedMethod.MethodName = PyGenUtil::TCHARToUTF8Buffer(*InFieldName);
		FuncDef.GeneratedWrappedMethod.MethodDoc = PyGenUtil::TCHARToUTF8Buffer(*PyGenUtil::GetFieldTooltip(Func));
		FuncDef.GeneratedWrappedMethod.MethodFunc = MoveTemp(GeneratedWrappedFunction);
		FuncDef.GeneratedWrappedMethod.MethodFlags = FuncArgNames.Num() > 0 ? METH_VARARGS | METH_KEYWORDS : METH_NOARGS;
		if (Func->HasAnyFunctionFlags(FUNC_Static))
		{
			FuncDef.GeneratedWrappedMethod.MethodFlags |= METH_CLASS;
			FuncDef.GeneratedWrappedMethod.MethodCallback = FuncArgNames.Num() > 0 ? PyCFunctionWithClosureCast(&FPyWrapperObject::CallClassMethodWithArgs_Impl) : PyCFunctionWithClosureCast(&FPyWrapperObject::CallClassMethodNoArgs_Impl);
		}
		else
		{
			FuncDef.GeneratedWrappedMethod.MethodCallback = FuncArgNames.Num() > 0 ? PyCFunctionWithClosureCast(&FPyWrapperObject::CallMethodWithArgs_Impl) : PyCFunctionWithClosureCast(&FPyWrapperObject::CallMethodNoArgs_Impl);
		}
		FuncDef.GeneratedWrappedMethod.ToPython(FuncDef.PyMethod);
		FuncDef.PyFunction = FPyObjectPtr::NewReference(InPyFuncDef->Func);
		FuncDef.bIsHidden = EnumHasAnyFlags(InPyFuncDef->FuncFlags, EPyUFunctionDefFlags::Getter | EPyUFunctionDefFlags::Setter);

		return true;
	}

	bool CopyPropertiesFromOldClass()
	{
		check(OldClass);

		NewClass->PropertyDefs.Reserve(OldClass->PropertyDefs.Num());
		for (const TSharedPtr<PyGenUtil::FPropertyDef>& OldPropDef : OldClass->PropertyDefs)
		{
			const FProperty* OldProp = OldPropDef->GeneratedWrappedGetSet.Prop.Prop;
			const UFunction* OldGetter = OldPropDef->GeneratedWrappedGetSet.GetFunc.Func;
			const UFunction* OldSetter = OldPropDef->GeneratedWrappedGetSet.SetFunc.Func;

			FProperty* Prop = CastFieldChecked<FProperty>(FField::Duplicate(OldProp, NewClass, OldProp->GetFName()));
			if (!Prop)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to duplicate property for '%s'"), UTF8_TO_TCHAR(OldPropDef->PyGetSet.name)));
				return false;
			}

			FField::CopyMetaData(OldProp, Prop);
			NewClass->AddCppProperty(Prop);

			PyGenUtil::FPropertyDef& PropDef = *NewClass->PropertyDefs.Add_GetRef(MakeShared<PyGenUtil::FPropertyDef>());
			PropDef.GeneratedWrappedGetSet = OldPropDef->GeneratedWrappedGetSet;
			PropDef.GeneratedWrappedGetSet.Prop.SetProperty(Prop);
			if (OldGetter)
			{
				PropDef.GeneratedWrappedGetSet.GetFunc.SetFunction(NewClass->FindFunctionByName(OldGetter->GetFName()));
			}
			if (OldSetter)
			{
				PropDef.GeneratedWrappedGetSet.SetFunc.SetFunction(NewClass->FindFunctionByName(OldSetter->GetFName()));
			}
			PropDef.GeneratedWrappedGetSet.ToPython(PropDef.PyGetSet);
		}

		return true;
	}

	bool CopyFunctionsFromOldClass()
	{
		check(OldClass);

		NewClass->FunctionDefs.Reserve(OldClass->FunctionDefs.Num());
		for (const TSharedPtr<PyGenUtil::FFunctionDef>& OldFuncDef : OldClass->FunctionDefs)
		{
			const UFunction* OldFunc = OldFuncDef->GeneratedWrappedMethod.MethodFunc.Func;

			NewClass->AddNativeFunction(*OldFunc->GetName(), &UPythonGeneratedClass::CallPythonFunction);
			UFunction* Func = DuplicateObject<UFunction>(OldFunc, NewClass, OldFunc->GetFName());
			if (!Func)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to duplicate function for '%s'"), UTF8_TO_TCHAR(OldFuncDef->PyMethod.MethodName)));
				return false;
			}

			FMetaData::CopyMetadata((UFunction*)OldFunc, Func);
			NewClass->AddFunctionToFunctionMap(Func, Func->GetFName());

			Func->Bind();
			Func->StaticLink(true);

			PyGenUtil::FFunctionDef& FuncDef = *NewClass->FunctionDefs.Add_GetRef(MakeShared<PyGenUtil::FFunctionDef>());
			FuncDef.GeneratedWrappedMethod = OldFuncDef->GeneratedWrappedMethod;
			FuncDef.GeneratedWrappedMethod.MethodFunc.SetFunction(Func);
			FuncDef.PyFunction = OldFuncDef->PyFunction;
			FuncDef.bIsHidden = OldFuncDef->bIsHidden;
			FuncDef.GeneratedWrappedMethod.ToPython(FuncDef.PyMethod);
		}

		return true;
	}

	void ReparentPythonType(PyTypeObject* InNewBasePyType)
	{
		auto UpdateTuple = [](PyObject* InTuple, PyTypeObject* InOldType, PyTypeObject* InNewType)
		{
			if (InTuple)
			{
				const int32 TupleSize = PyTuple_Size(InTuple);
				for (int32 TupleIndex = 0; TupleIndex < TupleSize; ++TupleIndex)
				{
					if (PyTuple_GetItem(InTuple, TupleIndex) == (PyObject*)InOldType)
					{
						FPyTypeObjectPtr NewType = FPyTypeObjectPtr::NewReference(InNewType);
						PyTuple_SetItem(InTuple, TupleIndex, (PyObject*)NewType.Release()); // PyTuple_SetItem steals the reference
					}
				}
			}
		};

		UpdateTuple(PyType->tp_bases, PyType->tp_base, InNewBasePyType);
		UpdateTuple(PyType->tp_mro, PyType->tp_base, InNewBasePyType);
		PyType->tp_base = InNewBasePyType;
	}

private:
	bool RegisterDescriptors()
	{
		for (const TSharedPtr<PyGenUtil::FPropertyDef>& PropDef : NewClass->PropertyDefs)
		{
			FPyObjectPtr GetSetDesc = FPyObjectPtr::StealReference(PyDescr_NewGetSet(PyType, &PropDef->PyGetSet));
			if (!GetSetDesc)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create descriptor for '%s'"), UTF8_TO_TCHAR(PropDef->PyGetSet.name)));
				return false;
			}
			if (PyDict_SetItemString(PyType->tp_dict, PropDef->PyGetSet.name, GetSetDesc) != 0)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to assign descriptor for '%s'"), UTF8_TO_TCHAR(PropDef->PyGetSet.name)));
				return false;
			}
		}

		for (const TSharedPtr<PyGenUtil::FFunctionDef>& FuncDef : NewClass->FunctionDefs)
		{
			if (FuncDef->bIsHidden)
			{
				PyDict_DelItemString(PyType->tp_dict, FuncDef->PyMethod.MethodName);
			}
			else
			{
				FPyObjectPtr MethodDesc = FPyObjectPtr::StealReference(FPyMethodWithClosureDef::NewMethodDescriptor(PyType, &FuncDef->PyMethod));
				if (!MethodDesc)
				{
					PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create descriptor for '%s'"), UTF8_TO_TCHAR(FuncDef->PyMethod.MethodName)));
					return false;
				}
				if (PyDict_SetItemString(PyType->tp_dict, FuncDef->PyMethod.MethodName, MethodDesc) != 0)
				{
					PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to assign descriptor for '%s'"), UTF8_TO_TCHAR(FuncDef->PyMethod.MethodName)));
					return false;
				}
			}
		}

		return true;
	}

	void PrepareOldClassForReinstancing()
	{
		check(OldClass);

		const FString OldClassName = MakeUniqueObjectName(OldClass->GetOuter(), OldClass->GetClass(), *FString::Printf(TEXT("%s_REINST"), *OldClass->GetName())).ToString();
		OldClass->ClassFlags |= CLASS_NewerVersionExists;
		OldClass->SetFlags(RF_NewerVersionExists);
		OldClass->ClearFlags(RF_Public | RF_Standalone);
		OldClass->Rename(*OldClassName, nullptr, REN_DontCreateRedirectors);
		OldClass->UnregisterGeneratedType();
	}

	FString ClassName;
	PyTypeObject* PyType;
	UPythonGeneratedClass* OldClass;
	UPythonGeneratedClass* NewClass;
};

void UPythonGeneratedClass::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPythonGeneratedClass* This = CastChecked<UPythonGeneratedClass>(InThis);
	Collector.AddReferencedObject(This->ClassRedirector, This);
	Collector.AddReferencedObject(This->CDORedirector, This);
}

void UPythonGeneratedClass::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	if (PyType)
	{
		FPyWrapperTypeRegistry::Get().UnregisterWrappedClassType(FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(OldOuter->GetFName(), OldName)), PyType, false);
		FPyWrapperTypeRegistry::Get().RegisterWrappedClassType(this, PyType, false);
	}
}

void UPythonGeneratedClass::PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph)
{
	Super::PostInitInstance(InObj, InstanceGraph);

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;

		if (PyPostInitFunction)
		{
			TGuardValue<bool> IsRunningUserScriptGuard(UE::Python::Private::GIsRunningUserScript, true);

			FPyObjectPtr PySelf = FPyObjectPtr::StealReference((PyObject*)FPyWrapperObjectFactory::Get().CreateInstance(InObj));
			if (PySelf && ensureAlwaysMsgf(PySelf->ob_type == PyType, TEXT("Object '%s' (class: %s) had an unexpected PyType when calling PostInitInstance! Self is '%s' but we expected '%s'"), *InObj->GetPathName(), *GetPathName(), *PyUtil::GetFriendlyTypename(PySelf), *PyUtil::GetFriendlyTypename(PyType)))
			{
				FPyObjectPtr PyArgs = FPyObjectPtr::StealReference(PyTuple_New(1));
				PyTuple_SetItem(PyArgs, 0, PySelf.Release()); // SetItem steals the reference

				FPyObjectPtr Result = FPyObjectPtr::StealReference(PyObject_CallObject(PyPostInitFunction, PyArgs));
				if (!Result)
				{
					PyUtil::ReThrowPythonError();
				}
			}
		}
	}
}

void UPythonGeneratedClass::BeginDestroy()
{
	ReleasePythonResources();
	Super::BeginDestroy();
}

void UPythonGeneratedClass::ReleasePythonResources()
{
	// This may be called after Python has already shut down
	if (Py_IsInitialized())
	{
		FPyScopedGIL GIL;
		UnregisterGeneratedType();
		FPyWrapperObjectMetaData::SetMetaData(PyType.Get(), nullptr);
		PyType.Reset();
		PyPostInitFunction.Reset();
		for (const TSharedPtr<PyGenUtil::FFunctionDef>& FunctionDef : FunctionDefs)
		{
			FunctionDef->PyFunction.Reset();
		}
	}
	else
	{
		// Release ownership if Python has been shut down to avoid attempting to delete the objects (which are already dead)
		PyType.Release();
		PyPostInitFunction.Release();
		for (const TSharedPtr<PyGenUtil::FFunctionDef>& FunctionDef : FunctionDefs)
		{
			FunctionDef->PyFunction.Release();
		}
	}

	PropertyDefs.Reset();
	FunctionDefs.Reset();
	PyMetaData = FPyWrapperObjectMetaData();
}

void UPythonGeneratedClass::UnregisterGeneratedType()
{
	if (PyType)
	{
		FPyWrapperTypeRegistry::Get().UnregisterWrappedClassType(this, PyType, false);
	}
}

bool UPythonGeneratedClass::IsFunctionImplementedInScript(FName InFunctionName) const
{
	UFunction* Function = FindFunctionByName(InFunctionName);
	return Function && Function->GetOuter() && Function->GetOuter()->IsA(UPythonGeneratedClass::StaticClass());
}

UPythonGeneratedClass* UPythonGeneratedClass::GenerateClass(PyTypeObject* InPyType)
{
	// Get the correct super class from the parent type in Python
	UClass* SuperClass = FPyWrapperObjectMetaData::GetClass(InPyType->tp_base);
	if (!SuperClass)
	{
		PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("No super class could be found for this Python type"));
		return nullptr;
	}

	// Builder used to generate the class
	FPythonGeneratedClassBuilder PythonClassBuilder(SuperClass, InPyType);

	// Add the functions to this class
	// We have to process these first as properties may reference them as get/set functions
	{
		PyObject* FieldKey = nullptr;
		PyObject* FieldValue = nullptr;
		Py_ssize_t FieldIndex = 0;
		while (PyDict_Next(InPyType->tp_dict, &FieldIndex, &FieldKey, &FieldValue))
		{
			const FString FieldName = PyUtil::PyObjectToUEString(FieldKey);

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyUValueDefType) == 1)
			{
				// Values are not supported on classes
				PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("Classes do not support values"));
				return nullptr;
			}

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyUFunctionDefType) == 1)
			{
				FPyUFunctionDef* PyFuncDef = (FPyUFunctionDef*)FieldValue;
				if (!PythonClassBuilder.CreateFunctionFromDefinition(FieldName, PyFuncDef))
				{
					return nullptr;
				}
			}
		}
	}

	// Add the properties to this class
	{
		PyObject* FieldKey = nullptr;
		PyObject* FieldValue = nullptr;
		Py_ssize_t FieldIndex = 0;
		while (PyDict_Next(InPyType->tp_dict, &FieldIndex, &FieldKey, &FieldValue))
		{
			const FString FieldName = PyUtil::PyObjectToUEString(FieldKey);

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyFPropertyDefType) == 1)
			{
				FPyFPropertyDef* PyPropDef = (FPyFPropertyDef*)FieldValue;
				if (!PythonClassBuilder.CreatePropertyFromDefinition(FieldName, PyPropDef))
				{
					return nullptr;
				}
			}
		}
	}

	// Finalize the class with its post-init function
	return PythonClassBuilder.Finalize(FPyObjectPtr::StealReference(PyGenUtil::GetPostInitFunc(InPyType)));
}

bool UPythonGeneratedClass::ReparentDerivedClasses(UPythonGeneratedClass* InOldParent, UPythonGeneratedClass* InNewParent)
{
	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(InOldParent, DerivedClasses, /*bRecursive*/false);

	bool bSuccess = true;

	for (UClass* DerivedClass : DerivedClasses)
	{
		if (DerivedClass->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			continue;
		}

		// todo: Blueprint classes?

		if (UPythonGeneratedClass* PyDerivedClass = Cast<UPythonGeneratedClass>(DerivedClass))
		{
			bSuccess &= !!ReparentClass(PyDerivedClass, InNewParent);
		}
	}

	return bSuccess;
}

UPythonGeneratedClass* UPythonGeneratedClass::ReparentClass(UPythonGeneratedClass* InOldClass, UPythonGeneratedClass* InNewParent)
{
	// Builder used to generate the class
	FPythonGeneratedClassBuilder PythonClassBuilder(InOldClass, InNewParent);

	// Copy the data from the old class
	if (!PythonClassBuilder.CopyFunctionsFromOldClass())
	{
		return nullptr;
	}
	if (!PythonClassBuilder.CopyPropertiesFromOldClass())
	{
		return nullptr;
	}

	UPythonGeneratedClass* NewClass = PythonClassBuilder.Finalize(InOldClass->PyPostInitFunction);
	if (NewClass)
	{
		// Update the base of the Python type
		PythonClassBuilder.ReparentPythonType(InNewParent->PyType);
	}
	return NewClass;
}

DEFINE_FUNCTION(UPythonGeneratedClass::CallPythonFunction)
{
	// Note: This function *must not* return until InvokePythonCallableFromUnrealFunctionThunk has been called, as we need to step over the correct amount of data from the bytecode stack!

	const UFunction* Func = Stack.CurrentNativeFunction;

	// Find the Python function to call
	TSharedPtr<PyGenUtil::FFunctionDef> FuncDef;
	{
		// Get the correct class from the UFunction so that we can perform static dispatch to the correct type
		const UPythonGeneratedClass* This = CastChecked<UPythonGeneratedClass>(Func->GetOwnerClass());

		const TSharedPtr<PyGenUtil::FFunctionDef>* FuncDefPtr = This->FunctionDefs.FindByPredicate([Func](const TSharedPtr<PyGenUtil::FFunctionDef>& InFuncDef)
		{
			return InFuncDef->GeneratedWrappedMethod.MethodFunc.Func == Func;
		});
		FuncDef = FuncDefPtr ? *FuncDefPtr : nullptr;

		if (!FuncDef.IsValid())
		{
			UE_LOGF(LogPython, Error, "Failed to find Python function for '%ls' on '%ls'", *Func->GetName(), *This->GetName());
		}
	}

	// Find the Python object to call the function on
	FPyObjectPtr PySelf;
	bool bSelfError = false;
	if (!Func->HasAnyFunctionFlags(FUNC_Static))
	{
		FPyScopedGIL GIL;
		PySelf = FPyObjectPtr::StealReference((PyObject*)FPyWrapperObjectFactory::Get().CreateInstance(P_THIS_OBJECT));
		if (!PySelf)
		{
			UE_LOGF(LogPython, Error, "Failed to create a Python wrapper for '%ls'", *P_THIS_OBJECT->GetName());
			bSelfError = true;
		}
	}

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		if (!PyGenUtil::InvokePythonCallableFromUnrealFunctionThunk(PySelf, FuncDef ? FuncDef->PyFunction.GetPtr() : nullptr, Func, Context, Stack, RESULT_PARAM) || bSelfError)
		{
			PyUtil::ReThrowPythonError();
		}
		PySelf.Reset(); // Need to reset this while still under the GIL
	}
}

#endif	// WITH_PYTHON

