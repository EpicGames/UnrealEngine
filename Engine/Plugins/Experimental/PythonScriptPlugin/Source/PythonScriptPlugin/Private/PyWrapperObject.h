// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/PropertyAccessUtil.h"
#include "PyWrapperObject.generated.h"

class UObjectRedirector;

#if WITH_PYTHON

/** Python type for FPyWrapperObject */
extern PyTypeObject PyWrapperObjectType;

/** Initialize the PyWrapperObject types and add them to the given Python module */
void InitializePyWrapperObject(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed object instances */
struct FPyWrapperObject : public FPyWrapperBase
{
	/** Wrapped object instance */
	TObjectPtr<UObject> ObjectInstance;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperObject* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperObject* InSelf);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperObject* InSelf, UObject* InValue);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperObject* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperObject* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperObject* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperObject* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Get a property value from this instance (called via generated code) */
	static PyObject* GetPropertyValue(FPyWrapperObject* InSelf, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName);

	/** Set a property value on this instance (called via generated code) */
	static int SetPropertyValue(FPyWrapperObject* InSelf, PyObject* InValue, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName, const EPropertyAccessChangeNotifyMode InNotifyMode = EPropertyAccessChangeNotifyMode::Never, const uint64 InReadOnlyFlags = PropertyAccessUtil::RuntimeReadOnlyFlags);

	/** Call a named getter function on this class using the given instance (called via generated code) */
	static PyObject* CallGetterFunction(FPyWrapperObject* InSelf, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef);

	/** Call a named setter function on this class using the given instance (called via generated code) */
	static int CallSetterFunction(FPyWrapperObject* InSelf, PyObject* InValue, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef);

	/** Call a function on this class (called via generated code) */
	static PyObject* CallFunction(PyTypeObject* InType, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName);

	/** Call a function on this class (called via generated code) */
	static PyObject* CallFunction(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName);

	/** Call a function on this class using the given instance (called via generated code) */
	static PyObject* CallFunction(FPyWrapperObject* InSelf, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName);

	/** Call a function on this class using the given instance (called via generated code) */
	static PyObject* CallFunction(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName);

	/** Call a function on this instance (CallFunction internal use only) */
	static PyObject* CallFunction_Impl(UObject* InObj, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName, const TCHAR* InErrorCtxt);

	/** Call a function on this instance (CallFunction internal use only) */
	static PyObject* CallFunction_Impl(UObject* InObj, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const char* InPythonFuncName, const TCHAR* InErrorCtxt);

	/** Implementation of the "call" logic for a Python class method with no arguments (internal Python bindings use only) */
	static PyObject* CallClassMethodNoArgs_Impl(PyTypeObject* InType, void* InClosure);

	/** Implementation of the "call" logic for a Python class method with arguments (internal Python bindings use only) */
	static PyObject* CallClassMethodWithArgs_Impl(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds, void* InClosure);

	/** Implementation of the "call" logic for a Python method with no arguments (internal Python bindings use only) */
	static PyObject* CallMethodNoArgs_Impl(FPyWrapperObject* InSelf, void* InClosure);

	/** Implementation of the "call" logic for a Python method with arguments (internal Python bindings use only) */
	static PyObject* CallMethodWithArgs_Impl(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, void* InClosure);

	/** Call a dynamic function on this instance (CallFunction internal use only) */
	static PyObject* CallDynamicFunction_Impl(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const PyGenUtil::FGeneratedWrappedMethodParameter& InSelfParam, const char* InPythonFuncName);

	/** Implementation of the "call" logic for a dynamic Python method with no arguments (internal Python bindings use only) */
	static PyObject* CallDynamicMethodNoArgs_Impl(FPyWrapperObject* InSelf, void* InClosure);

	/** Implementation of the "call" logic for a dynamic Python method with arguments (internal Python bindings use only) */
	static PyObject* CallDynamicMethodWithArgs_Impl(FPyWrapperObject* InSelf, PyObject* InArgs, PyObject* InKwds, void* InClosure);

	/** Implementation of the "getter" logic for a Python descriptor reading from an object property (internal Python bindings use only) */
	static PyObject* Getter_Impl(FPyWrapperObject* InSelf, void* InClosure);

	/** Implementation of the "setter" logic for a Python descriptor writing to an object property (internal Python bindings use only) */
	static int Setter_Impl(FPyWrapperObject* InSelf, PyObject* InValue, void* InClosure);
};

/** Meta-data for all Unreal exposed object types */
struct FPyWrapperObjectMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperObjectMetaData, FGuid(0x89FC2465, 0xA83F4F31, 0xBBCC1E86, 0xE9D76551))

	FPyWrapperObjectMetaData()
		: Class(nullptr)
	{
	}

	/** Get the UClass from the given type */
	static UClass* GetClass(PyTypeObject* PyType);

	/** Get the UClass from the type of the given instance */
	static UClass* GetClass(FPyWrapperObject* Instance);

	/** Resolve the original property name of a Python property from the given type */
	static FName ResolvePropertyName(PyTypeObject* PyType, const FName InPythonPropertyName);

	/** Resolve the original property name of a Python property of the given instance */
	static FName ResolvePropertyName(FPyWrapperObject* Instance, const FName InPythonPropertyName);

	/** Check to see if the given Python property is deprecated, and optionally return its deprecation message */
	static bool IsPropertyDeprecated(PyTypeObject* PyType, const FName InPythonPropertyName, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the given Python property is deprecated, and optionally return its deprecation message */
	static bool IsPropertyDeprecated(FPyWrapperObject* Instance, const FName InPythonPropertyName, FString* OutDeprecationMessage = nullptr);

	/** Resolve the original function name of a Python method from the given type */
	static FName ResolveFunctionName(PyTypeObject* PyType, const FName InPythonMethodName);

	/** Resolve the original function name of a Python method of the given instance */
	static FName ResolveFunctionName(FPyWrapperObject* Instance, const FName InPythonMethodName);

	/** Check to see if the given Python method is deprecated, and optionally return its deprecation message */
	static bool IsFunctionDeprecated(PyTypeObject* PyType, const FName InPythonMethodName, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the given Python method is deprecated, and optionally return its deprecation message */
	static bool IsFunctionDeprecated(FPyWrapperObject* Instance, const FName InPythonMethodName, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the class is deprecated, and optionally return its deprecation message */
	static bool IsClassDeprecated(PyTypeObject* PyType, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the class is deprecated, and optionally return its deprecation message */
	static bool IsClassDeprecated(FPyWrapperObject* Instance, FString* OutDeprecationMessage = nullptr);

	/** Add object references from this type meta-data to the given collector */
	virtual void AddTypeReferencedObjects(FReferenceCollector& Collector) override;

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;

	/** Get the reflection meta data type object associated with this wrapper type if there is one or nullptr if not. */
	virtual const UField* GetMetaType() const override
	{
		return Class;
	}

	/** Unreal class */
	TObjectPtr<UClass> Class;

	/** Map of properties that were exposed to Python mapped to their original name */
	TMap<FName, FName> PythonProperties;

	/** Map of properties that were exposed to Python mapped to their deprecation message (if deprecated) */
	TMap<FName, FString> PythonDeprecatedProperties;

	/** Map of methods that were exposed to Python mapped to their original name */
	TMap<FName, FName> PythonMethods;

	/** Map of methods that were exposed to Python mapped to their deprecation message (if deprecated) */
	TMap<FName, FString> PythonDeprecatedMethods;

	/** Set if this class is deprecated and using it should emit a deprecation warning */
	TOptional<FString> DeprecationMessage;
};

typedef TPyPtr<FPyWrapperObject> FPyWrapperObjectPtr;

#endif	// WITH_PYTHON

/** An Unreal class that was generated from a Python type */
UCLASS(Transient)
class UPythonGeneratedClass final : public UClass, public IPythonResourceOwner
{
	GENERATED_BODY()

#if WITH_PYTHON

public:
	//~ UObject interface
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void BeginDestroy() override;
	virtual bool IsAsset() const override
	{
		return false;
	}

	//~ UClass interface
	virtual void PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph) override;

	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override;

	/** Unregister this type from FPyWrapperTypeRegistry */
	void UnregisterGeneratedType();

	virtual bool IsFunctionImplementedInScript(FName InFunctionName) const override;

	/** Generate an Unreal class from the given Python type */
	static UPythonGeneratedClass* GenerateClass(PyTypeObject* InPyType);

	/** Generate an Unreal class for all child classes of the old parent using the new parent class as their base (also update the Python types) */
	static bool ReparentDerivedClasses(UPythonGeneratedClass* InOldParent, UPythonGeneratedClass* InNewParent);

	/** Generate an Unreal class based on the given class, but using the given parent class (also update the Python type) */
	static UPythonGeneratedClass* ReparentClass(UPythonGeneratedClass* InOldClass, UPythonGeneratedClass* InNewParent);

private:
	/** Native function used to call the Python functions from C code */
	DECLARE_FUNCTION(CallPythonFunction);

	/** Python type this class was generated from */
	FPyTypeObjectPtr PyType;

	/** PostInit function for this class */
	FPyObjectPtr PyPostInitFunction;

	/** Array of properties generated for this class */
	TArray<TSharedPtr<PyGenUtil::FPropertyDef>> PropertyDefs;

	/** Array of functions generated for this class */
	TArray<TSharedPtr<PyGenUtil::FFunctionDef>> FunctionDefs;

	/** Meta-data for this generated class that is applied to the Python type */
	FPyWrapperObjectMetaData PyMetaData;

	TObjectPtr<UObjectRedirector> ClassRedirector;
	TObjectPtr<UObjectRedirector> CDORedirector;

	friend class FPythonGeneratedClassBuilder;

#else	// WITH_PYTHON

public:
	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override {}

#endif	// WITH_PYTHON
};
