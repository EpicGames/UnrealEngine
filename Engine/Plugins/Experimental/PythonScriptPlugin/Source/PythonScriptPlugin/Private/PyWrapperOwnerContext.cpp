// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperOwnerContext.h"
#include "PyWrapperObject.h"
#include "PyWrapperStruct.h"

#if WITH_PYTHON

FPyWrapperOwnerContext::FPyWrapperOwnerContext()
	: OwnerObject()
	, OwnerProperty(nullptr)
{
}

FPyWrapperOwnerContext::FPyWrapperOwnerContext(PyObject* InOwner, const FProperty* InProp)
	: OwnerObject(FPyObjectPtr::NewReference(InOwner))
	, OwnerProperty(InProp)
{
	checkf(!OwnerProperty || OwnerObject.IsValid(), TEXT("Owner context cannot have an owner property without an owner object"));
}

FPyWrapperOwnerContext::FPyWrapperOwnerContext(const FPyObjectPtr& InOwner, const FProperty* InProp)
	: OwnerObject(InOwner)
	, OwnerProperty(InProp)
{
	checkf(!OwnerProperty || OwnerObject.IsValid(), TEXT("Owner context cannot have an owner property without an owner object"));
}

void FPyWrapperOwnerContext::Reset()
{
	OwnerObject.Reset();
	OwnerProperty = nullptr;
}

bool FPyWrapperOwnerContext::HasOwner() const
{
	return OwnerObject.IsValid();
}

PyObject* FPyWrapperOwnerContext::GetOwnerObject() const
{
	return (PyObject*)OwnerObject.GetPtr();
}

const FProperty* FPyWrapperOwnerContext::GetOwnerProperty() const
{
	return OwnerProperty;
}

void FPyWrapperOwnerContext::AssertValidConversionMethod(const EPyConversionMethod InMethod) const
{
	::AssertValidPyConversionOwner(GetOwnerObject(), InMethod);
}

TUniquePtr<FPropertyAccessChangeNotify> FPyWrapperOwnerContext::BuildChangeNotify(const EPropertyAccessChangeNotifyMode InNotifyMode) const
{
#if WITH_EDITOR
	if (InNotifyMode != EPropertyAccessChangeNotifyMode::Never)
	{
		TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = MakeUnique<FPropertyAccessChangeNotify>();
		ChangeNotify->NotifyMode = InNotifyMode;

		auto AppendOwnerPropertyToChain = [&ChangeNotify](const FPyWrapperOwnerContext& InOwnerContext) -> bool
		{
			const FProperty* LeafProp = nullptr;
			if (PyObject_IsInstance(InOwnerContext.GetOwnerObject(), (PyObject*)&PyWrapperObjectType) == 1 || PyObject_IsInstance(InOwnerContext.GetOwnerObject(), (PyObject*)&PyWrapperStructType) == 1)
			{
				LeafProp = InOwnerContext.GetOwnerProperty();
			}

			if (LeafProp)
			{
				ChangeNotify->ChangedPropertyChain.AddHead(const_cast<FProperty*>(LeafProp));
				return true;
			}

			return false;
		};

		FPyWrapperOwnerContext OwnerContext = *this;
		while (OwnerContext.HasOwner() && AppendOwnerPropertyToChain(OwnerContext))
		{
			PyObject* PyObj = OwnerContext.GetOwnerObject();

			if (PyObj == GetOwnerObject())
			{
				ChangeNotify->ChangedPropertyChain.SetActivePropertyNode(ChangeNotify->ChangedPropertyChain.GetHead()->GetValue());
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperObjectType) == 1)
			{
				// Found an object, this is the end of the chain
				ChangeNotify->ChangedObject = ((FPyWrapperObject*)PyObj)->ObjectInstance;
				ChangeNotify->ChangedPropertyChain.SetActiveMemberPropertyNode(ChangeNotify->ChangedPropertyChain.GetHead()->GetValue());
				break;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperStructType) == 1)
			{
				// Found a struct, recurse up the chain
				OwnerContext = ((FPyWrapperStruct*)PyObj)->OwnerContext;
				continue;
			}

			// Unknown object type - just bail
			break;
		}

		// If we didn't find an object in the chain then we can't emit notifications
		if (!ChangeNotify->ChangedObject)
		{
			ChangeNotify.Reset();
		}

		return ChangeNotify;
	}
#endif
	return nullptr;
}

UObject* FPyWrapperOwnerContext::FindChangeNotifyObject() const
{
	FPyWrapperOwnerContext OwnerContext = *this;
	while (OwnerContext.HasOwner())
	{
		PyObject* PyObj = OwnerContext.GetOwnerObject();

		if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperObjectType) == 1)
		{
			// Found an object, this is the end of the chain
			return ((FPyWrapperObject*)PyObj)->ObjectInstance;
		}

		if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperStructType) == 1)
		{
			// Found a struct, recurse up the chain
			OwnerContext = ((FPyWrapperStruct*)PyObj)->OwnerContext;
			continue;
		}

		// Unknown object type - just bail
		break;
	}

	return nullptr;
}

#endif	// WITH_PYTHON
