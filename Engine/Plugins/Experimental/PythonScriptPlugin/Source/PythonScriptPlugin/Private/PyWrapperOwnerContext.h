// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "PyPtr.h"
#include "PyConversionMethod.h"
#include "UObject/PropertyAccessUtil.h"

#if WITH_PYTHON

/** Owner context information for wrapped types */
class FPyWrapperOwnerContext
{
public:
	/** Default constructor */
	FPyWrapperOwnerContext();

	/** Construct this context from the given Python object and optional property (will create a new reference to the given object) */
	explicit FPyWrapperOwnerContext(PyObject* InOwner, const FProperty* InProp = nullptr);

	/** Construct this context from the given Python object and optional property */
	explicit FPyWrapperOwnerContext(const FPyObjectPtr& InOwner, const FProperty* InProp = nullptr);

	/** Reset this context back to its default state */
	void Reset();

	/** Check to see if this context has an owner set */
	bool HasOwner() const;

	/** Get the Python object that owns the instance being wrapped (if any, borrowed reference) */
	PyObject* GetOwnerObject() const;

	/** Get the property on the owner object that owns the instance being wrapped (if known) */
	const FProperty* GetOwnerProperty() const;

	/** Assert that the given conversion method is valid for this owner context */
	void AssertValidConversionMethod(const EPyConversionMethod InMethod) const;

	/** Build the property change notify that corresponds to this owner context, or null if this owner context shouldn't emit change notifications */
	TUniquePtr<FPropertyAccessChangeNotify> BuildChangeNotify(const EPropertyAccessChangeNotifyMode InNotifyMode) const;

	/** Walk the owner context chain to find a UObject owner instance that should receive change notifications (if any) */
	UObject* FindChangeNotifyObject() const;

private:
	/** The Python object that owns the instance being wrapped (if any) */
	FPyObjectPtr OwnerObject;

	/** The property on the owner object that owns the instance being wrapped (if known) */
	const FProperty* OwnerProperty;
};

#endif	// WITH_PYTHON
