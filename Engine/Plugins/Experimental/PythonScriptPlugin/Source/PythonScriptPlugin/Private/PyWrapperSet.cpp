// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperSet.h"
#include "PyWrapperTypeRegistry.h"
#include "PyConversion.h"
#include "PyReferenceCollector.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"

#if WITH_PYTHON

/** Python type for FPyWrapperSetIterator */
extern PyTypeObject PyWrapperSetIteratorType;

/** Iterator used with sets */
struct FPyWrapperSetIterator
{
	/** Common Python Object */
	PyObject_HEAD

	/** Instance being iterated over */
	FPyWrapperSet* IterInstance;

	/** Current iteration index */
	int32 IterIndex;

	/** New this iterator instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperSetIterator* New(PyTypeObject* InType)
	{
		FPyWrapperSetIterator* Self = (FPyWrapperSetIterator*)InType->tp_alloc(InType, 0);
		if (Self)
		{
			Self->IterInstance = nullptr;
			Self->IterIndex = 0;
		}
		return Self;
	}

	/** Free this iterator instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperSetIterator* InSelf)
	{
		Deinit(InSelf);
		Py_TYPE(InSelf)->tp_free((PyObject*)InSelf);
	}

	/** Initialize this iterator instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperSetIterator* InSelf, FPyWrapperSet* InInstance)
	{
		Deinit(InSelf);

		Py_INCREF(InInstance);
		InSelf->IterInstance = InInstance;
		InSelf->IterIndex = GetElementIndex(InSelf, 0);

		return 0;
	}

	/** Deinitialize this iterator instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperSetIterator* InSelf)
	{
		Py_XDECREF(InSelf->IterInstance);
		InSelf->IterInstance = nullptr;
		InSelf->IterIndex = 0;
	}

	/** Called to validate the internal state of this iterator instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperSetIterator* InSelf)
	{
		if (!InSelf->IterInstance)
		{
			PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - IterInstance is null!"));
			return false;
		}

		return true;
	}

	/** Given a sparse index, get the first element index from this point in the set (including the given index) */
	static int32 GetElementIndex(FPyWrapperSetIterator* InSelf, int32 InSparseIndex)
	{
		FScriptSetHelper ScriptSetHelper(InSelf->IterInstance->SetProp, InSelf->IterInstance->SetInstance);
		const int32 SparseCount = ScriptSetHelper.GetMaxIndex();

		int32 ElementIndex = InSparseIndex;
		for (; ElementIndex < SparseCount; ++ElementIndex)
		{
			if (ScriptSetHelper.IsValidIndex(ElementIndex))
			{
				break;
			}
		}

		return ElementIndex;
	}

	/** Get the iterator */
	static FPyWrapperSetIterator* GetIter(FPyWrapperSetIterator* InSelf)
	{
		Py_INCREF(InSelf);
		return InSelf;
	}

	/** Return the current value (if any) and advance the iterator */
	static PyObject* IterNext(FPyWrapperSetIterator* InSelf)
	{
		if (!ValidateInternalState(InSelf))
		{
			return nullptr;
		}

		FScriptSetHelper ScriptSetHelper(InSelf->IterInstance->SetProp, InSelf->IterInstance->SetInstance);
		const int32 SparseCount = ScriptSetHelper.GetMaxIndex();

		if (InSelf->IterIndex < SparseCount)
		{
			const int32 ElementIndex = InSelf->IterIndex;
			InSelf->IterIndex = GetElementIndex(InSelf, InSelf->IterIndex + 1);

			if (!ScriptSetHelper.IsValidIndex(ElementIndex))
			{
				PyUtil::SetPythonError(PyExc_IndexError, InSelf, TEXT("Iterator was on an invalid element index! Was the set changed while iterating?"));
				return nullptr;
			}

			PyObject* PyItemObj = nullptr;
			if (!PyConversion::PythonizeProperty(ScriptSetHelper.GetElementProperty(), ScriptSetHelper.GetElementPtr(ElementIndex), PyItemObj))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert element property '%s' (%s)"), *ScriptSetHelper.GetElementProperty()->GetName(), *ScriptSetHelper.GetElementProperty()->GetClass()->GetName()));
				return nullptr;
			}
			return PyItemObj;
		}

		PyErr_SetObject(PyExc_StopIteration, Py_None);
		return nullptr;
	}
};

void InitializePyWrapperSet(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperSetType) == 0)
	{
		static FPyWrapperSetMetaData MetaData;
		FPyWrapperSetMetaData::SetMetaData(&PyWrapperSetType, &MetaData);
		ModuleInfo.AddType(&PyWrapperSetType);
	}

	PyType_Ready(&PyWrapperSetIteratorType);
}

FPyWrapperSet* FPyWrapperSet::New(PyTypeObject* InType)
{
	FPyWrapperSet* Self = (FPyWrapperSet*)FPyWrapperBase::New(InType);
	if (Self)
	{
		new(&Self->OwnerContext) FPyWrapperOwnerContext();
		new(&Self->SetProp) PyUtil::FConstSetPropOnScope();
		Self->SetInstance = nullptr;
	}
	return Self;
}

void FPyWrapperSet::Free(FPyWrapperSet* InSelf)
{
	Deinit(InSelf);

	InSelf->OwnerContext.~FPyWrapperOwnerContext();
	InSelf->SetProp.~TPropOnScope();
	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperSet::Init(FPyWrapperSet* InSelf, const PyUtil::FPropertyDef& InElementDef)
{
	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	PyUtil::FPropOnScope SetElementProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InElementDef, 1));
	if (!SetElementProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Set element property was null during init"));
		return -1;
	}

	PyUtil::FSetPropOnScope SetProp = PyUtil::FSetPropOnScope::OwnedReference(new FSetProperty(FFieldVariant(), PyUtil::DefaultPythonPropertyName));
	SetProp->ElementProp = SetElementProp.Release();

	// Need to manually call Link to fix-up some data (such as the C++ property flags and the set layout) that are only set during Link
	{
		FArchive Ar;
		SetProp->LinkWithoutChangingOffset(Ar);
	}

	void* SetValue = FMemory::Malloc(SetProp->GetSize(), SetProp->GetMinAlignment());
	SetProp->InitializeValue(SetValue);

	InSelf->SetProp = MoveTemp(SetProp);
	InSelf->SetInstance = SetValue;

	if (!InSelf->SetProp->ElementProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), *FString::Printf(TEXT("Set element type must be hashable: %s (%s)"), *InSelf->SetProp->ElementProp->GetName(), *InSelf->SetProp->ElementProp->GetClass()->GetName()));
		return -1;
	}

	FPyWrapperSetFactory::Get().MapInstance(InSelf->SetInstance, InSelf);
	return 0;
}

int FPyWrapperSet::Init(FPyWrapperSet* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FSetProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod)
{
	InOwnerContext.AssertValidConversionMethod(InConversionMethod);

	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	check(InProp && InValue);

	PyUtil::FConstSetPropOnScope PropToUse;
	void* SetInstanceToUse = nullptr;
	switch (InConversionMethod)
	{
	case EPyConversionMethod::Copy:
	case EPyConversionMethod::Steal:
		{
			PyUtil::FPropOnScope SetElementProp = PyUtil::FPropOnScope::OwnedReference(PyUtil::CreateProperty(InProp->ElementProp));
			if (!SetElementProp)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to create element property from '%s' (%s)"), *InProp->ElementProp->GetName(), *InProp->ElementProp->GetClass()->GetName()));
				return -1;
			}

			PyUtil::FSetPropOnScope SetProp = PyUtil::FSetPropOnScope::OwnedReference(new FSetProperty(FFieldVariant(), PyUtil::DefaultPythonPropertyName));
			SetProp->ElementProp = SetElementProp.Release();
			
			// Need to manually call Link to fix-up some data (such as the C++ property flags and the set layout) that are only set during Link
			{
				FArchive Ar;
				SetProp->LinkWithoutChangingOffset(Ar);
			}

			PropToUse = MoveTemp(SetProp);

			SetInstanceToUse = FMemory::Malloc(PropToUse->GetSize(), PropToUse->GetMinAlignment());
			PropToUse->InitializeValue(SetInstanceToUse);
			if (InConversionMethod == EPyConversionMethod::Steal)
			{
				FScriptSetHelper SelfScriptSetHelper(PropToUse, SetInstanceToUse);
				SelfScriptSetHelper.MoveAssign(InValue);
			}
			else
			{
				PropToUse->CopyCompleteValue(SetInstanceToUse, InValue);
			}
		}
		break;

	case EPyConversionMethod::Reference:
		{
			PropToUse = PyUtil::FConstSetPropOnScope::ExternalReference(InProp);
			SetInstanceToUse = InValue;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EPyConversionMethod"));
		break;
	}

	check(PropToUse && SetInstanceToUse);

	InSelf->OwnerContext = InOwnerContext;
	InSelf->SetProp = MoveTemp(PropToUse);
	InSelf->SetInstance = SetInstanceToUse;

	if (!InSelf->SetProp->ElementProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), *FString::Printf(TEXT("Set element type must be hashable: %s (%s)"), *InSelf->SetProp->ElementProp->GetName(), *InSelf->SetProp->ElementProp->GetClass()->GetName()));
		return -1;
	}

	FPyWrapperSetFactory::Get().MapInstance(InSelf->SetInstance, InSelf);
	return 0;
}

void FPyWrapperSet::Deinit(FPyWrapperSet* InSelf)
{
	if (InSelf->SetInstance)
	{
		FPyWrapperSetFactory::Get().UnmapInstance(InSelf->SetInstance, Py_TYPE(InSelf));
	}

	if (InSelf->OwnerContext.HasOwner())
	{
		InSelf->OwnerContext.Reset();
	}
	else if (InSelf->SetInstance)
	{
		if (InSelf->SetProp)
		{
			InSelf->SetProp->DestroyValue(InSelf->SetInstance);
		}
		FMemory::Free(InSelf->SetInstance);
	}
	InSelf->SetProp.Reset();
	InSelf->SetInstance = nullptr;
}

bool FPyWrapperSet::ValidateInternalState(FPyWrapperSet* InSelf)
{
	if (!InSelf->SetProp)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - SetProp is null!"));
		return false;
	}

	if (!InSelf->SetInstance)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - SetInstance is null!"));
		return false;
	}

	return true;
}

FPyWrapperSet* FPyWrapperSet::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperSetType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperSet*)InPyObject;
	}

	return nullptr;
}

FPyWrapperSet* FPyWrapperSet::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InElementDef, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperSetType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperSetType) == 1))
	{
		FPyWrapperSet* Self = (FPyWrapperSet*)InPyObject;

		if (!ValidateInternalState(Self))
		{
			return nullptr;
		}

		const PyUtil::FPropertyDef SelfElementPropDef = Self->SetProp->ElementProp;
		if (SelfElementPropDef == InElementDef)
		{
			SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

			Py_INCREF(Self);
			return Self;
		}

		FPyWrapperSetPtr NewSet = FPyWrapperSetPtr::StealReference(FPyWrapperSet::New(InType));
		if (FPyWrapperSet::Init(NewSet, InElementDef) != 0)
		{
			return nullptr;
		}

		// Attempt to convert the entries in the set to the native format of the new set
		{
			FScriptSetHelper SelfScriptSetHelper(Self->SetProp, Self->SetInstance);
			FScriptSetHelper NewScriptSetHelper(NewSet->SetProp, NewSet->SetInstance);

			FString ExportedEntry;
			for (FScriptSetHelper::FIterator It(SelfScriptSetHelper); It; ++It)
			{
				ExportedEntry.Reset();
				if (!SelfScriptSetHelper.GetElementProperty()->ExportText_Direct(ExportedEntry, SelfScriptSetHelper.GetElementPtr(It), SelfScriptSetHelper.GetElementPtr(It), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to export text for element property '%s' (%s) at index %d"), *SelfScriptSetHelper.GetElementProperty()->GetName(), *SelfScriptSetHelper.GetElementProperty()->GetClass()->GetName(), It.GetLogicalIndex()));
					return nullptr;
				}

				const int32 NewElementIndex = NewScriptSetHelper.AddDefaultValue_Invalid_NeedsRehash();
				if (!NewScriptSetHelper.GetElementProperty()->ImportText_Direct(*ExportedEntry, NewScriptSetHelper.GetElementPtr(NewElementIndex), nullptr, PPF_None))
				{
					PyUtil::SetPythonError(PyExc_Exception, Self, *FString::Printf(TEXT("Failed to import text '%s' element for property '%s' (%s) at index %d"), *ExportedEntry, *NewScriptSetHelper.GetElementProperty()->GetName(), *NewScriptSetHelper.GetElementProperty()->GetClass()->GetName(), NewElementIndex));
					return nullptr;
				}
			}

			NewScriptSetHelper.Rehash();
		}

		SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
		return NewSet.Release();
	}

	// Attempt conversion from any iterable sequence type that has a defined length
	if (!PyUtil::IsMappingType(InPyObject))
	{
		const Py_ssize_t SequenceLen = PyObject_Length(InPyObject);
		if (SequenceLen != -1)
		{
			FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(InPyObject));
			if (PyObjIter)
			{
				FPyWrapperSetPtr NewSet = FPyWrapperSetPtr::StealReference(FPyWrapperSet::New(InType));
				if (FPyWrapperSet::Init(NewSet, InElementDef) != 0)
				{
					return nullptr;
				}

				FScriptSetHelper NewScriptSetHelper(NewSet->SetProp, NewSet->SetInstance);
				for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
				{
					FPyObjectPtr SequenceItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
					if (!SequenceItem)
					{
						return nullptr;
					}

					const int32 NewElementIndex = NewScriptSetHelper.AddDefaultValue_Invalid_NeedsRehash();
					if (!PyConversion::NativizeProperty(SequenceItem, NewScriptSetHelper.GetElementProperty(), NewScriptSetHelper.GetElementPtr(NewElementIndex)))
					{
						return nullptr;
					}
				}

				NewScriptSetHelper.Rehash();

				SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
				return NewSet.Release();
			}
		}
	}

	return nullptr;
}

Py_ssize_t FPyWrapperSet::Len(FPyWrapperSet* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	return SelfScriptSetHelper.Num();
}

int FPyWrapperSet::Contains(FPyWrapperSet* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FSetElementOnScope ContainerEntryValue(InSelf->SetProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	return SelfScriptSetHelper.FindElementIndexFromHash(ContainerEntryValue.GetValue()) != INDEX_NONE;
}

int FPyWrapperSet::Add(FPyWrapperSet* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FSetElementOnScope ContainerEntryValue(InSelf->SetProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	SelfScriptSetHelper.AddElement(ContainerEntryValue.GetValue());

	return 0;
}

int FPyWrapperSet::Discard(FPyWrapperSet* InSelf, PyObject* InValue)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	PyUtil::FSetElementOnScope ContainerEntryValue(InSelf->SetProp);
	if (!ContainerEntryValue.IsValid())
	{
		return -1;
	}

	if (!ContainerEntryValue.SetValue(InValue, *PyUtil::GetErrorContext(InSelf)))
	{
		return -1;
	}

	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	const bool bWasRemoved = SelfScriptSetHelper.RemoveElement(ContainerEntryValue.GetValue());

	return bWasRemoved ? 1 : 0;
}

int FPyWrapperSet::Remove(FPyWrapperSet* InSelf, PyObject* InValue)
{
	const int DiscardResult = Discard(InSelf, InValue);
	if (DiscardResult == -1)
	{
		return -1;
	}

	if (DiscardResult == 0)
	{
		PyUtil::SetPythonError(PyExc_KeyError, InSelf, TEXT("The given value was not found in the set"));
		return -1;
	}

	return 0;
}

PyObject* FPyWrapperSet::Pop(FPyWrapperSet* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	const FScriptSetHelper::FIterator It(SelfScriptSetHelper);
	if (It)
	{
		PyObject* PyReturnValue = nullptr;
		if (!PyConversion::PythonizeProperty(SelfScriptSetHelper.GetElementProperty(), SelfScriptSetHelper.GetElementPtr(It), PyReturnValue))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert element property '%s' (%s) at index 0"), *SelfScriptSetHelper.GetElementProperty()->GetName(), *SelfScriptSetHelper.GetElementProperty()->GetClass()->GetName()));
			return nullptr;
		}

		SelfScriptSetHelper.RemoveAt(It.GetInternalIndex());

		return PyReturnValue;
	}

	PyUtil::SetPythonError(PyExc_KeyError, InSelf, TEXT("Cannot pop from an empty set"));
	return nullptr;
}

int FPyWrapperSet::Clear(FPyWrapperSet* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	SelfScriptSetHelper.EmptyElements();

	return 0;
}

FPyWrapperSet* FPyWrapperSet::Difference(FPyWrapperSet* InSelf, PyObject* InOthers)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperSetPtr NewSet = FPyWrapperSetPtr::StealReference(FPyWrapperSetFactory::Get().CreateInstance(InSelf->SetInstance, InSelf->SetProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy));
	if (DifferenceUpdate(NewSet, InOthers) != 0)
	{
		return nullptr;
	}

	return NewSet.Release();
}

int FPyWrapperSet::DifferenceUpdate(FPyWrapperSet* InSelf, PyObject* InOthers)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);

	auto ProcessOtherObject = [&InSelf, &SelfScriptSetHelper, &SelfElementDef](PyObject* InOtherObj, const int32 InObjIndex)
	{
		FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOtherObj, &PyWrapperSetType, SelfElementDef));
		if (!Other)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument %d (%s) to '%s'"), InObjIndex, *PyUtil::GetFriendlyTypename(InOtherObj), *PyUtil::GetFriendlyTypename(InSelf)));
			return false;
		}

		const FScriptSetHelper OtherScriptSetHelper(Other->SetProp, Other->SetInstance);
		for (FScriptSetHelper::FIterator It(OtherScriptSetHelper); It; ++It)
		{
			const void* OtherElementPtr = OtherScriptSetHelper.GetElementPtr(It);
			SelfScriptSetHelper.RemoveElement(OtherElementPtr);
		}

		return true;
	};

	if (PyTuple_Check(InOthers))
	{
		// We need to go through each argument we were passed and remove any of its values that are present in NewSet
		const Py_ssize_t TupleLen = PyTuple_Size(InOthers);
		for (Py_ssize_t TupleIndex = 0; TupleIndex < TupleLen; ++TupleIndex)
		{
			PyObject* OtherObj = PyTuple_GetItem(InOthers, TupleIndex);
			if (!ProcessOtherObject(OtherObj, TupleIndex))
			{
				return -1;
			}
		}
	}
	else
	{
		// Assume a single value
		if (!ProcessOtherObject(InOthers, 0))
		{
			return -1;
		}
	}

	return 0;
}

FPyWrapperSet* FPyWrapperSet::Intersection(FPyWrapperSet* InSelf, PyObject* InOthers)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperSetPtr NewSet = FPyWrapperSetPtr::StealReference(FPyWrapperSetFactory::Get().CreateInstance(InSelf->SetInstance, InSelf->SetProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy));
	if (IntersectionUpdate(NewSet, InOthers) != 0)
	{
		return nullptr;
	}

	return NewSet.Release();
}

int FPyWrapperSet::IntersectionUpdate(FPyWrapperSet* InSelf, PyObject* InOthers)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);

	auto ProcessOtherObject = [&InSelf, &SelfScriptSetHelper, &SelfElementDef](PyObject* InOtherObj, const int32 InObjIndex)
	{
		FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOtherObj, &PyWrapperSetType, SelfElementDef));
		if (!Other)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument %d (%s) to '%s'"), InObjIndex, *PyUtil::GetFriendlyTypename(InOtherObj), *PyUtil::GetFriendlyTypename(InSelf)));
			return false;
		}

		FScriptSetHelper OtherScriptSetHelper(Other->SetProp, Other->SetInstance);

		// We iterate in reverse, as when TSet is TCompactSet (see UE_USE_COMPACT_SET_AS_DEFAULT), 
		// it will perform a RemoveSwap, vs TSparseSet which will leave holes until compaction
		// Reverse iteration works correctly for both implementations, whereas forward iteration 
		// fails for TCompactSet, as it will skip over the element swapped from the end of the set
		const int32 SelfSparseCount = SelfScriptSetHelper.GetMaxIndex();
		for (int32 SelfSparseIndex = SelfSparseCount - 1; SelfSparseIndex >= 0; --SelfSparseIndex)
		{
			if (!SelfScriptSetHelper.IsValidIndex(SelfSparseIndex))
			{
				continue;
			}

			const void* SelfElementPtr = SelfScriptSetHelper.GetElementPtr(SelfSparseIndex);
			if (OtherScriptSetHelper.FindElementIndexFromHash(SelfElementPtr) == INDEX_NONE)
			{
				SelfScriptSetHelper.RemoveAt(SelfSparseIndex);
			}
		}

		return true;
	};

	if (PyTuple_Check(InOthers))
	{
		// We need to go through each argument we were passed and remove any values from NewSet that aren't present in the others
		const Py_ssize_t TupleLen = PyTuple_Size(InOthers);
		for (Py_ssize_t TupleIndex = 0; TupleIndex < TupleLen; ++TupleIndex)
		{
			PyObject* OtherObj = PyTuple_GetItem(InOthers, TupleIndex);
			if (!ProcessOtherObject(OtherObj, TupleIndex))
			{
				return -1;
			}
		}
	}
	else
	{
		// Assume a single value
		if (!ProcessOtherObject(InOthers, 0))
		{
			return -1;
		}
	}

	return 0;
}

FPyWrapperSet* FPyWrapperSet::SymmetricDifference(FPyWrapperSet* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperSetPtr NewSet = FPyWrapperSetPtr::StealReference(FPyWrapperSetFactory::Get().CreateInstance(InSelf->SetInstance, InSelf->SetProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy));
	if (SymmetricDifferenceUpdate(NewSet, InOther) != 0)
	{
		return nullptr;
	}

	return NewSet.Release();
}

int FPyWrapperSet::SymmetricDifferenceUpdate(FPyWrapperSet* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);

	FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOther, &PyWrapperSetType, SelfElementDef));
	if (!Other)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument (%s) to '%s'"), *PyUtil::GetFriendlyTypename(InOther), *PyUtil::GetFriendlyTypename(InSelf)));
		return -1;
	}

	const FScriptSetHelper OtherScriptSetHelper(Other->SetProp, Other->SetInstance);

	// We need to go through the other set and remove any values from Self that are present in Other, and add any values from Other that aren't present in Self
	for (FScriptSetHelper::FIterator It(OtherScriptSetHelper); It; ++It)
	{
		const void* OtherElementPtr = OtherScriptSetHelper.GetElementPtr(It);
		if (SelfScriptSetHelper.FindElementIndexFromHash(OtherElementPtr) == INDEX_NONE)
		{
			SelfScriptSetHelper.AddElement(OtherElementPtr);
		}
		else
		{
			SelfScriptSetHelper.RemoveElement(OtherElementPtr);
		}
	}

	return 0;
}

FPyWrapperSet* FPyWrapperSet::Union(FPyWrapperSet* InSelf, PyObject* InOthers)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperSetPtr NewSet = FPyWrapperSetPtr::StealReference(FPyWrapperSetFactory::Get().CreateInstance(InSelf->SetInstance, InSelf->SetProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy));
	if (Update(NewSet, InOthers) != 0)
	{
		return nullptr;
	}

	return NewSet.Release();
}

int FPyWrapperSet::Update(FPyWrapperSet* InSelf, PyObject* InOthers)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
	FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);

	auto ProcessOtherObject = [&InSelf, &SelfScriptSetHelper, &SelfElementDef](PyObject* InOtherObj, const int32 InObjIndex)
	{
		FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOtherObj, &PyWrapperSetType, SelfElementDef));
		if (!Other)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument %d (%s) to '%s'"), InObjIndex, *PyUtil::GetFriendlyTypename(InOtherObj), *PyUtil::GetFriendlyTypename(InSelf)));
			return false;
		}

		const FScriptSetHelper OtherScriptSetHelper(Other->SetProp, Other->SetInstance);
		for (FScriptSetHelper::FIterator It(OtherScriptSetHelper); It; ++It)
		{
			const void* OtherElementPtr = OtherScriptSetHelper.GetElementPtr(It);
			SelfScriptSetHelper.AddElement(OtherElementPtr);
		}

		return true;
	};

	if (PyTuple_Check(InOthers))
	{
		// We need to go through each argument we were passed and remove any values from NewSet that aren't present in the others
		const Py_ssize_t TupleLen = PyTuple_Size(InOthers);
		for (Py_ssize_t TupleIndex = 0; TupleIndex < TupleLen; ++TupleIndex)
		{
			PyObject* OtherObj = PyTuple_GetItem(InOthers, TupleIndex);
			if (!ProcessOtherObject(OtherObj, TupleIndex))
			{
				return -1;
			}
		}
	}
	else
	{
		// Assume a single value
		if (!ProcessOtherObject(InOthers, 0))
		{
			return -1;
		}
	}

	return 0;
}

int FPyWrapperSet::IsDisjoint(FPyWrapperSet* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FPyWrapperSetPtr IntersectionSet = FPyWrapperSetPtr::StealReference(FPyWrapperSet::Intersection(InSelf, InOther));
	if (!IntersectionSet)
	{
		return -1;
	}

	FScriptSetHelper IntersectionScriptSetHelper(IntersectionSet->SetProp, IntersectionSet->SetInstance);
	if (IntersectionScriptSetHelper.Num() == 0)
	{
		return 1;
	}

	return 0;
}

int FPyWrapperSet::IsSubset(FPyWrapperSet* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
	FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOther, &PyWrapperSetType, SelfElementDef));
	if (!Other)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument (%s) to '%s'"), *PyUtil::GetFriendlyTypename(InOther), *PyUtil::GetFriendlyTypename(InSelf)));
		return -1;
	}

	const FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	const FScriptSetHelper OtherScriptSetHelper(Other->SetProp, Other->SetInstance);

	for (FScriptSetHelper::FIterator It(SelfScriptSetHelper); It; ++It)
	{
		const void* SelfElementPtr = SelfScriptSetHelper.GetElementPtr(It);
		if (OtherScriptSetHelper.FindElementIndexFromHash(SelfElementPtr) == INDEX_NONE)
		{
			return 0;
		}
	}

	return 1;
}

int FPyWrapperSet::IsSuperset(FPyWrapperSet* InSelf, PyObject* InOther)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
	FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOther, &PyWrapperSetType, SelfElementDef));
	if (!Other)
	{
		PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert argument (%s) to '%s'"), *PyUtil::GetFriendlyTypename(InOther), *PyUtil::GetFriendlyTypename(InSelf)));
		return -1;
	}

	const FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);
	const FScriptSetHelper OtherScriptSetHelper(Other->SetProp, Other->SetInstance);

	for (FScriptSetHelper::FIterator It(OtherScriptSetHelper); It; ++It)
	{
		const void* OtherElementPtr = OtherScriptSetHelper.GetElementPtr(It);
		if (SelfScriptSetHelper.FindElementIndexFromHash(OtherElementPtr) == INDEX_NONE)
		{
			return 0;
		}
	}

	return 1;
}

PyTypeObject InitializePyWrapperSetType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperSet::New(InType);
		}

		static void Dealloc(FPyWrapperSet* InSelf)
		{
			FPyWrapperSet::Free(InSelf);
		}

		static int Init(FPyWrapperSet* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;

			static const char *ArgsKwdList[] = { "type", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:call", (char**)ArgsKwdList, &PyTypeObj))
			{
				return -1;
			}

			PyUtil::FPropertyDef SetElementDef;
			const int ValidateTypeResult = PyUtil::ValidateContainerTypeParam(PyTypeObj, SetElementDef, "type", *PyUtil::GetErrorContext(Py_TYPE(InSelf)));
			if (ValidateTypeResult != 0)
			{
				return ValidateTypeResult;
			}

			return FPyWrapperSet::Init(InSelf, SetElementDef);
		}

		static PyObject* Str(FPyWrapperSet* InSelf)
		{
			if (!FPyWrapperSet::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const FScriptSetHelper SelfScriptSetHelper(InSelf->SetProp, InSelf->SetInstance);

			FString ExportedSet;
			for (FScriptSetHelper::FIterator It(SelfScriptSetHelper); It; ++It)
			{
				if (It.GetLogicalIndex() > 0)
				{
					ExportedSet += TEXT(", ");
				}
				ExportedSet += PyUtil::GetFriendlyPropertyValue(SelfScriptSetHelper.GetElementProperty(), SelfScriptSetHelper.GetElementPtr(It), PPF_Delimited | PPF_IncludeTransient);
			}
			return PyUnicode_FromFormat("set([%s])", TCHAR_TO_UTF8(*ExportedSet));
		}

		static PyObject* RichCmp(FPyWrapperSet* InSelf, PyObject* InOther, int InOp)
		{
			if (!FPyWrapperSet::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const PyUtil::FPropertyDef SelfElementDef = InSelf->SetProp->ElementProp;
			FPyWrapperSetPtr Other = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InOther, &PyWrapperSetType, SelfElementDef));
			if (!Other)
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			if (InOp != Py_EQ && InOp != Py_NE)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Only == and != comparison is supported"));
				return nullptr;
			}

			const bool bIsIdentical = InSelf->SetProp->Identical(InSelf->SetInstance, Other->SetInstance, PPF_None);
			return PyBool_FromLong(InOp == Py_EQ ? bIsIdentical : !bIsIdentical);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperSet* InSelf)
		{
			if (!FPyWrapperSet::ValidateInternalState(InSelf))
			{
				return -1;
			}

			if (InSelf->SetProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
			{
				const uint32 SetHash = InSelf->SetProp->GetValueTypeHash(InSelf->SetInstance);
				return (PyUtil::FPyHashType)(SetHash != -1 ? SetHash : 0);
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Type cannot be hashed"));
			return -1;
		}

		static PyObject* GetIter(FPyWrapperSet* InSelf)
		{
			typedef TPyPtr<FPyWrapperSetIterator> FPyWrapperSetIteratorPtr;

			if (!FPyWrapperSet::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FPyWrapperSetIteratorPtr NewIter = FPyWrapperSetIteratorPtr::StealReference(FPyWrapperSetIterator::New(&PyWrapperSetIteratorType));
			if (FPyWrapperSetIterator::Init(NewIter, InSelf) != 0)
			{
				return nullptr;
			}

			return (PyObject*)NewIter.Release();
		}
	};

	struct FNumberFuncs
	{
		static PyObject* Sub(PyObject* InLHS, PyObject* InRHS)
		{
			FPyWrapperSetPtr LHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InLHS));
			FPyWrapperSetPtr RHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InRHS));
			if (!LHS && !RHS)
			{
				PyUtil::SetPythonError(PyExc_TypeError, *PyUtil::GetErrorContext(&PyWrapperSetType), *FString::Printf(TEXT("One of LHS or RHS must be a '%s'"), *PyUtil::GetFriendlyTypename(&PyWrapperSetType)));
				return nullptr;
			}

			return (PyObject*)(LHS ? FPyWrapperSet::Difference(LHS, InRHS) : FPyWrapperSet::Difference(RHS, InLHS));
		}

		static PyObject* SubInplace(FPyWrapperSet* InSelf, PyObject* InOther)
		{
			if (FPyWrapperSet::DifferenceUpdate(InSelf, InOther) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* And(PyObject* InLHS, PyObject* InRHS)
		{
			FPyWrapperSetPtr LHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InLHS));
			FPyWrapperSetPtr RHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InRHS));
			if (!LHS && !RHS)
			{
				PyUtil::SetPythonError(PyExc_TypeError, *PyUtil::GetErrorContext(&PyWrapperSetType), *FString::Printf(TEXT("One of LHS or RHS must be a '%s'"), *PyUtil::GetFriendlyTypename(&PyWrapperSetType)));
				return nullptr;
			}

			return (PyObject*)(LHS ? FPyWrapperSet::Intersection(LHS, InRHS) : FPyWrapperSet::Intersection(RHS, InLHS));
		}

		static PyObject* AndInplace(FPyWrapperSet* InSelf, PyObject* InOther)
		{
			if (FPyWrapperSet::IntersectionUpdate(InSelf, InOther) != 0)
			{
				return nullptr;
			}
			
			Py_RETURN_NONE;
		}

		static PyObject* Xor(PyObject* InLHS, PyObject* InRHS)
		{
			FPyWrapperSetPtr LHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InLHS));
			FPyWrapperSetPtr RHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InRHS));
			if (!LHS && !RHS)
			{
				PyUtil::SetPythonError(PyExc_TypeError, *PyUtil::GetErrorContext(&PyWrapperSetType), *FString::Printf(TEXT("One of LHS or RHS must be a '%s'"), *PyUtil::GetFriendlyTypename(&PyWrapperSetType)));
				return nullptr;
			}

			return (PyObject*)(LHS ? FPyWrapperSet::SymmetricDifference(LHS, InRHS) : FPyWrapperSet::SymmetricDifference(RHS, InLHS));
		}

		static PyObject* XorInplace(FPyWrapperSet* InSelf, PyObject* InOther)
		{
			if (FPyWrapperSet::SymmetricDifferenceUpdate(InSelf, InOther) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Or(PyObject* InLHS, PyObject* InRHS)
		{
			FPyWrapperSetPtr LHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InLHS));
			FPyWrapperSetPtr RHS = FPyWrapperSetPtr::StealReference(FPyWrapperSet::CastPyObject(InRHS));
			if (!LHS && !RHS)
			{
				PyUtil::SetPythonError(PyExc_TypeError, *PyUtil::GetErrorContext(&PyWrapperSetType), *FString::Printf(TEXT("One of LHS or RHS must be a '%s'"), *PyUtil::GetFriendlyTypename(&PyWrapperSetType)));
				return nullptr;
			}

			return (PyObject*)(LHS ? FPyWrapperSet::Union(LHS, InRHS) : FPyWrapperSet::Union(RHS, InLHS));
		}

		static PyObject* OrInplace(FPyWrapperSet* InSelf, PyObject* InOther)
		{
			if (FPyWrapperSet::Update(InSelf, InOther) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}
	};

	struct FSequenceFuncs
	{
		static Py_ssize_t Len(FPyWrapperSet* InSelf)
		{
			return FPyWrapperSet::Len(InSelf);
		}

		static int Contains(FPyWrapperSet* InSelf, PyObject* InValue)
		{
			return FPyWrapperSet::Contains(InSelf, InValue);
		}
	};

	struct FMethods
	{
		// This is for type hinting, to make the class appears as a generic (https://peps.python.org/pep-0560/) and support hinting like 's: unreal.Set[int] = unreal.Set(int)'
		static PyObject* GetClassItem(PyObject* Type, PyObject* Item)
		{
			Py_INCREF(Type);
			return Type;
		}

		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyTypeObj = nullptr;
			PyObject* PyObj = nullptr;

			static const char *ArgsKwdList[] = { "type", "obj", nullptr };
			if (PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO:call", (char**)ArgsKwdList, &PyTypeObj, &PyObj))
			{
				PyUtil::FPropertyDef SetElementDef;
				if (PyUtil::ValidateContainerTypeParam(PyTypeObj, SetElementDef, "type", *PyUtil::GetErrorContext(InType)) != 0)
				{
					return nullptr;
				}

				PyObject* PyCastResult = (PyObject*)FPyWrapperSet::CastPyObject(PyObj, InType, SetElementDef);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* Copy(FPyWrapperSet* InSelf)
		{
			if (!FPyWrapperSet::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperSetFactory::Get().CreateInstance(InSelf->SetInstance, InSelf->SetProp, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}

		static PyObject* Add(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:add", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperSet::Add(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Discard(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:discard", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperSet::Discard(InSelf, PyObj) == -1)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Remove(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:remove", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperSet::Remove(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Pop(FPyWrapperSet* InSelf)
		{
			return FPyWrapperSet::Pop(InSelf);
		}

		static PyObject* Clear(FPyWrapperSet* InSelf)
		{
			if (FPyWrapperSet::Clear(InSelf) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Difference(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			check(PyTuple_Check(InArgs));
			return (PyObject*)FPyWrapperSet::Difference(InSelf, InArgs);
		}

		static PyObject* DifferenceUpdate(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			check(PyTuple_Check(InArgs));
			
			if (FPyWrapperSet::DifferenceUpdate(InSelf, InArgs) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Intersection(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			check(PyTuple_Check(InArgs));
			return (PyObject*)FPyWrapperSet::Intersection(InSelf, InArgs);
		}

		static PyObject* IntersectionUpdate(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			check(PyTuple_Check(InArgs));

			if (FPyWrapperSet::IntersectionUpdate(InSelf, InArgs) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* SymmetricDifference(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:symmetric_difference", &PyObj))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperSet::SymmetricDifference(InSelf, PyObj);
		}

		static PyObject* SymmetricDifferenceUpdate(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:symmetric_difference_update", &PyObj))
			{
				return nullptr;
			}

			if (FPyWrapperSet::SymmetricDifferenceUpdate(InSelf, PyObj) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* Union(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			check(PyTuple_Check(InArgs));
			return (PyObject*)FPyWrapperSet::Union(InSelf, InArgs);
		}

		static PyObject* Update(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			check(PyTuple_Check(InArgs));

			if (FPyWrapperSet::Update(InSelf, InArgs) != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* IsDisjoint(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:isdisjoint", &PyObj))
			{
				return nullptr;
			}

			const int IsDisjointResult = FPyWrapperSet::IsDisjoint(InSelf, PyObj);
			if (IsDisjointResult == -1)
			{
				return nullptr;
			}

			if (IsDisjointResult == 1)
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsSubset(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:issubset", &PyObj))
			{
				return nullptr;
			}

			const int IsSubsetResult = FPyWrapperSet::IsSubset(InSelf, PyObj);
			if (IsSubsetResult == -1)
			{
				return nullptr;
			}

			if (IsSubsetResult == 1)
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsSuperset(FPyWrapperSet* InSelf, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:issuperset", &PyObj))
			{
				return nullptr;
			}

			const int IsSupersetResult = FPyWrapperSet::IsSuperset(InSelf, PyObj);
			if (IsSupersetResult == -1)
			{
				return nullptr;
			}

			if (IsSupersetResult == 1)
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}
	};

	// NOTE: _ElemType = typing.TypeVar('_ElemType') and Type/Iterable are defined by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "__class_getitem__", PyCFunctionCast(FMethods::GetClassItem), METH_O|METH_CLASS, "__class_getitem__(cls, item: _ElemType) -- implemented for type hinting purpose only" },
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_KEYWORDS | METH_CLASS, "cast(cls, type: Type[_ElemType], obj: object) -> Set[_ElemType] -- cast the given object to this Unreal set type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> Set[_ElemType] -- copy this Unreal set" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> Set[_ElemType] -- copy this Unreal set" },
		{ "add", PyCFunctionCast(&FMethods::Add), METH_VARARGS, "add(self, value: _ElemType) -> None -- add the given value to this Unreal set if not already present" },
		{ "discard", PyCFunctionCast(&FMethods::Discard), METH_VARARGS, "discard(self, value: _ElemType) -> None -- remove the given value from this Unreal set, or do nothing if not present" },
		{ "remove", PyCFunctionCast(&FMethods::Remove), METH_VARARGS, "remove(self, value: _ElemType) -> None -- remove the given value from this Unreal set, or raise KeyError if not present" },
		{ "pop", PyCFunctionCast(&FMethods::Pop), METH_NOARGS, "pop(self) -> _ElemType -- remove and return an arbitrary value from this Unreal set, or raise KeyError if the set is empty" },
		{ "clear", PyCFunctionCast(&FMethods::Clear), METH_NOARGS, "clear(self) -> None -- remove all values from this Unreal set" },
		{ "difference", PyCFunctionCast(&FMethods::Difference), METH_VARARGS, "difference(self, *iterable: Iterable[_ElemType]) -> Set[_ElemType] -- return the difference between this Unreal set and the other iterables passed for comparison (ie, all values that are in this set but not the others)" },
		{ "difference_update", PyCFunctionCast(&FMethods::DifferenceUpdate), METH_VARARGS, "difference_update(self, *iterables: Iterable[_ElemType]) -> None -- make this set the difference between this Unreal set and the other iterables passed for comparison (ie, all values that are in this set but not the others)" },
		{ "intersection", PyCFunctionCast(&FMethods::Intersection), METH_VARARGS, "intersection(self, *iterables: Iterable[_ElemType]) -> Set[_ElemType] -- return the intersection between this Unreal set and the other iterables passed for comparison (ie, values that are common to all of the sets)" },
		{ "intersection_update", PyCFunctionCast(&FMethods::IntersectionUpdate), METH_VARARGS, "intersection_update(self, *iterables: Iterable[_ElemType]) -> None -- make this set the intersection between this Unreal set and the other iterables passed for comparison (ie, values that are common to all of the sets)" },
		{ "symmetric_difference", PyCFunctionCast(&FMethods::SymmetricDifference), METH_VARARGS, "symmetric_difference(self, other: Iterable[_ElemType]) -> Set[_ElemType] -- return the symmetric difference between this Unreal set and another (ie, values that are in exactly one of the sets)" },
		{ "symmetric_difference_update", PyCFunctionCast(&FMethods::SymmetricDifferenceUpdate), METH_VARARGS, "symmetric_difference_update(self, other: Iterable[_ElemType]) -> None -- make this set the symmetric difference between this Unreal set and another (ie, values that are in exactly one of the sets)" },
		{ "union", PyCFunctionCast(&FMethods::Union), METH_VARARGS, "union(self, *iterables: Iterable[_ElemType]) -> Set[_ElemType] -- return the union between this Unreal set and the other iterables passed for comparison (ie, values that are in any set)" },
		{ "update", PyCFunctionCast(&FMethods::Update), METH_VARARGS, "update(self, *iterables: Iterable[_ElemType]) -> None -- make this set the union between this Unreal set and the other iterables passed for comparison (ie, values that are in any set)" },
		{ "isdisjoint", PyCFunctionCast(&FMethods::IsDisjoint), METH_VARARGS, "isdisjoint(self, other: Iterable[_ElemType]) -> bool -- return True if there is a null intersection between this Unreal set and another" },
		{ "issubset", PyCFunctionCast(&FMethods::IsSubset), METH_VARARGS, "issubset(self, other: Iterable[_ElemType]) -> bool -- return True if another set contains this Unreal set" },
		{ "issuperset", PyCFunctionCast(&FMethods::IsSuperset), METH_VARARGS, "issuperset(self, other: Iterable[_ElemType]) -> bool -- return True if this Unreal set contains another" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"Set", /* tp_name */
		sizeof(FPyWrapperSet), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed set instances";

	static PyNumberMethods PyNumber;
	PyNumber.nb_subtract = (binaryfunc)&FNumberFuncs::Sub;
	PyNumber.nb_inplace_subtract = (binaryfunc)&FNumberFuncs::SubInplace;
	PyNumber.nb_and = (binaryfunc)&FNumberFuncs::And;
	PyNumber.nb_inplace_and = (binaryfunc)&FNumberFuncs::AndInplace;
	PyNumber.nb_xor = (binaryfunc)&FNumberFuncs::Xor;
	PyNumber.nb_inplace_xor = (binaryfunc)&FNumberFuncs::XorInplace;
	PyNumber.nb_or = (binaryfunc)&FNumberFuncs::Or;
	PyNumber.nb_inplace_or = (binaryfunc)&FNumberFuncs::OrInplace;

	static PySequenceMethods PySequence;
	PySequence.sq_length = (lenfunc)&FSequenceFuncs::Len;
	PySequence.sq_contains = (objobjproc)&FSequenceFuncs::Contains;

	PyType.tp_as_number = &PyNumber;
	PyType.tp_as_sequence = &PySequence;

	return PyType;
}

PyTypeObject InitializePyWrapperSetIteratorType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperSetIterator::New(InType);
		}

		static void Dealloc(FPyWrapperSetIterator* InSelf)
		{
			FPyWrapperSetIterator::Free(InSelf);
		}

		static int Init(FPyWrapperSetIterator* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				return -1;
			}

			if (PyObject_IsInstance(PyObj, (PyObject*)&PyWrapperSetType) != 1)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot initialize '%s' with an instance of '%s'"), *PyUtil::GetFriendlyTypename(InSelf), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return FPyWrapperSetIterator::Init(InSelf, (FPyWrapperSet*)PyObj);
		}

		static FPyWrapperSetIterator* GetIter(FPyWrapperSetIterator* InSelf)
		{
			return FPyWrapperSetIterator::GetIter(InSelf);
		}

		static PyObject* IterNext(FPyWrapperSetIterator* InSelf)
		{
			return FPyWrapperSetIterator::IterNext(InSelf);
		}
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"SetIterator", /* tp_name */
		sizeof(FPyWrapperSetIterator), /* tp_basicsize */
	};

	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_iter = (getiterfunc)&FFuncs::GetIter;
	PyType.tp_iternext = (iternextfunc)&FFuncs::IterNext;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT;
	PyType.tp_doc = "Type for all Unreal exposed set iterators";

	return PyType;
}

PyTypeObject PyWrapperSetType = InitializePyWrapperSetType();
PyTypeObject PyWrapperSetIteratorType = InitializePyWrapperSetIteratorType();

void FPyWrapperSetMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperSet* Self = static_cast<FPyWrapperSet*>(Instance);
	if (Self->SetProp && Self->SetInstance && !Self->OwnerContext.HasOwner())
	{
		Self->SetProp.AddReferencedObjects(Collector);
		FPyReferenceCollector::AddReferencedObjectsFromProperty(Collector, Self->SetProp, Self->SetInstance);
	}
}

#endif	// WITH_PYTHON
