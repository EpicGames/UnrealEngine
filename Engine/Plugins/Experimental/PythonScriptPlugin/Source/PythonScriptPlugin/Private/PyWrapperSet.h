// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "PyWrapperOwnerContext.h"
#include "PyUtil.h"

#if WITH_PYTHON

/** Python type for FPyWrapperSet */
extern PyTypeObject PyWrapperSetType;

/** Initialize the PyWrapperSet types and add them to the given Python module */
void InitializePyWrapperSet(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed set instances */
struct FPyWrapperSet : public FPyWrapperBase
{
	/** The owner of the wrapped set instance (if any) */
	FPyWrapperOwnerContext OwnerContext;

	/** Property describing the set */
	PyUtil::FConstSetPropOnScope SetProp;

	/** Wrapped set instance */
	void* SetInstance;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperSet* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperSet* InSelf);

	/** Initialize this wrapper (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperSet* InSelf, const PyUtil::FPropertyDef& InElementDef);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperSet* InSelf, const FPyWrapperOwnerContext& InOwnerContext, const FSetProperty* InProp, void* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperSet* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperSet* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperSet* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperSet* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, const PyUtil::FPropertyDef& InElementDef, FPyConversionResult* OutCastResult = nullptr);

	/** Get the length of this container (equivalent to 'len(x)' in Python) */
	static Py_ssize_t Len(FPyWrapperSet* InSelf);

	/** Does this container have an entry with the given value? (equivalent to 'v in x' in Python) */
	static int Contains(FPyWrapperSet* InSelf, PyObject* InValue);

	/** Add the given value to this container (equivalent to 'x.add(v)' in Python) */
	static int Add(FPyWrapperSet* InSelf, PyObject* InValue);

	/** Remove the given value from this container, doing nothing if it's not present (equivalent to 'x.discard(v)' in Python) */
	static int Discard(FPyWrapperSet* InSelf, PyObject* InValue);

	/** Remove the given value from this container (equivalent to 'x.remove(v)' in Python) */
	static int Remove(FPyWrapperSet* InSelf, PyObject* InValue);

	/** Remove and return an arbitrary value from this container (equivalent to 'x.pop()' in Python, returns new reference) */
	static PyObject* Pop(FPyWrapperSet* InSelf);

	/** Remove all values from this container (equivalent to 'x.clear()' in Python) */
	static int Clear(FPyWrapperSet* InSelf);

	/** Return the difference between this container and the other iterables passed for comparison (equivalent to 'x.difference(...)' in Python, returns new reference) */
	static FPyWrapperSet* Difference(FPyWrapperSet* InSelf, PyObject* InOthers);

	/** Update this container to be the difference between itself and the other iterables passed for comparison (equivalent to 'x.difference_update(...)' in Python) */
	static int DifferenceUpdate(FPyWrapperSet* InSelf, PyObject* InOthers);

	/** Return the intersection between this container and the other iterables passed for comparison (equivalent to 'x.intersection(...)' in Python, returns new reference) */
	static FPyWrapperSet* Intersection(FPyWrapperSet* InSelf, PyObject* InOthers);

	/** Update this container to be the intersection between this container and the other iterables passed for comparison (equivalent to 'x.intersection_update(...)' in Python) */
	static int IntersectionUpdate(FPyWrapperSet* InSelf, PyObject* InOthers);

	/** Return the symmetric difference between this container and another (equivalent to 'x.symmetric_difference(other)' in Python, returns new reference) */
	static FPyWrapperSet* SymmetricDifference(FPyWrapperSet* InSelf, PyObject* InOther);

	/** Update this container to be the symmetric difference between this container and another (equivalent to 'x.symmetric_difference_update(other)' in Python) */
	static int SymmetricDifferenceUpdate(FPyWrapperSet* InSelf, PyObject* InOther);

	/** Return the union between this container and the other iterables passed for comparison (equivalent to 'x.union(...)' in Python, returns new reference) */
	static FPyWrapperSet* Union(FPyWrapperSet* InSelf, PyObject* InOthers);

	/** Update this container to be the union between this container and the other iterables passed for comparison (equivalent to 'x.update(...)' in Python) */
	static int Update(FPyWrapperSet* InSelf, PyObject* InOthers);

	/** Return true if there is a null intersection between this container and another (equivalent to 'x.isdisjoint(other)' in Python) */
	static int IsDisjoint(FPyWrapperSet* InSelf, PyObject* InOther);

	/** Return true if there is another container contains this (equivalent to 'x.issubset(other)' in Python) */
	static int IsSubset(FPyWrapperSet* InSelf, PyObject* InOther);

	/** Return true if there this container contains another (equivalent to 'x.issuperset(other)' in Python) */
	static int IsSuperset(FPyWrapperSet* InSelf, PyObject* InOther);
};

/** Meta-data for all Unreal exposed set types */
struct FPyWrapperSetMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperSetMetaData, FGuid(0xB31DDC32, 0xAEA548BE, 0xB6FB5AB2, 0x02E5CA83))

	FPyWrapperSetMetaData()
	{
	}

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;
};

typedef TPyPtr<FPyWrapperSet> FPyWrapperSetPtr;

#endif	// WITH_PYTHON
