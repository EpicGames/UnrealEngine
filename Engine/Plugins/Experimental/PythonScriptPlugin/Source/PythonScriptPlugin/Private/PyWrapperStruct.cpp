// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperStruct.h"
#include "PyWrapperTypeRegistry.h"
#include "PyGIL.h"
#include "PyCore.h"
#include "PyReferenceCollector.h"
#include "UObject/Package.h"
#include "UObject/Class.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/ScopeExit.h"
#include "Templates/Casts.h"
#include "StructUtils/UserDefinedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PyWrapperStruct)

#if WITH_PYTHON

void InitializePyWrapperStruct(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperStructType) == 0)
	{
		static FPyWrapperStructMetaData MetaData;
		FPyWrapperStructMetaData::SetMetaData(&PyWrapperStructType, &MetaData);
		ModuleInfo.AddType(&PyWrapperStructType);
	}
}

const IPyWrapperStructAllocationPolicy* GetPyWrapperStructAllocationPolicy(UScriptStruct* InStruct)
{
	class FPyWrapperStructAllocationPolicy_Heap : public IPyWrapperStructAllocationPolicy
	{
		virtual void* AllocateStruct(const FPyWrapperStruct* InSelf, UScriptStruct* InStruct) const override
		{
			return FMemory::Malloc(FMath::Max(InStruct->GetStructureSize(), 1));
		}

		virtual void FreeStruct(const FPyWrapperStruct* InSelf, void* InAlloc) const override
		{
			FMemory::Free(InAlloc);
		}
	};

	if (const IPyWrapperInlineStructFactory* InlineStructFactory = FPyWrapperTypeRegistry::Get().GetInlineStructFactory(InStruct->GetStructPathName()))
	{
		return InlineStructFactory->GetPythonObjectAllocationPolicy();
	}

	static const FPyWrapperStructAllocationPolicy_Heap HeapAllocPolicy = FPyWrapperStructAllocationPolicy_Heap();
	return &HeapAllocPolicy;
}

FPyWrapperStruct* FPyWrapperStruct::New(PyTypeObject* InType)
{
	FPyWrapperStruct* Self = (FPyWrapperStruct*)FPyWrapperBase::New(InType);
	if (Self)
	{
		new(&Self->OwnerContext) FPyWrapperOwnerContext();
		Self->ScriptStruct = nullptr;
		Self->StructInstance = nullptr;
	}
	return Self;
}

void FPyWrapperStruct::Free(FPyWrapperStruct* InSelf)
{
	Deinit(InSelf);

	InSelf->OwnerContext.~FPyWrapperOwnerContext();
	FPyWrapperBase::Free(InSelf);
}

int FPyWrapperStruct::Init(FPyWrapperStruct* InSelf)
{
	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	UScriptStruct* Struct = FPyWrapperStructMetaData::GetStruct(InSelf);
	if (!Struct)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Struct is null"));
		return -1;
	}

	const IPyWrapperStructAllocationPolicy* AllocPolicy = GetPyWrapperStructAllocationPolicy(Struct);
	if (!AllocPolicy)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("AllocPolicy is null"));
		return -1;
	}

	// Deprecated structs emit a warning
	{
		FString DeprecationMessage;
		if (FPyWrapperStructMetaData::IsStructDeprecated(InSelf, &DeprecationMessage) &&
			PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf, *FString::Printf(TEXT("Struct '%s' is deprecated: %s"), UTF8_TO_TCHAR(Py_TYPE(InSelf)->tp_name), *DeprecationMessage)) == -1
			)
		{
			// -1 from SetPythonWarning means the warning should be an exception
			return -1;
		}
	}

	void* StructInstance = AllocPolicy->AllocateStruct(InSelf, Struct);
	Struct->InitializeStruct(StructInstance);

	InSelf->ScriptStruct = Struct;
	InSelf->StructInstance = StructInstance;

	FPyWrapperStructFactory::Get().MapInstance(InSelf->StructInstance, InSelf);
	return 0;
}

int FPyWrapperStruct::Init(FPyWrapperStruct* InSelf, const FPyWrapperOwnerContext& InOwnerContext, UScriptStruct* InStruct, void* InValue, const EPyConversionMethod InConversionMethod)
{
	InOwnerContext.AssertValidConversionMethod(InConversionMethod);

	Deinit(InSelf);

	const int BaseInit = FPyWrapperBase::Init(InSelf);
	if (BaseInit != 0)
	{
		return BaseInit;
	}

	check(InValue);

	const IPyWrapperStructAllocationPolicy* AllocPolicy = GetPyWrapperStructAllocationPolicy(InStruct);
	if (!AllocPolicy)
	{
		PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("AllocPolicy is null"));
		return -1;
	}

	void* StructInstanceToUse = nullptr;
	switch (InConversionMethod)
	{
	case EPyConversionMethod::Copy:
	case EPyConversionMethod::Steal:
		{
			StructInstanceToUse = AllocPolicy->AllocateStruct(InSelf, InStruct);
			InStruct->InitializeStruct(StructInstanceToUse);
			InStruct->CopyScriptStruct(StructInstanceToUse, InValue);
		}
		break;

	case EPyConversionMethod::Reference:
		{
			StructInstanceToUse = InValue;
		}
		break;

	default:
		checkf(false, TEXT("Unknown EPyConversionMethod"));
		break;
	}

	check(StructInstanceToUse);

	InSelf->OwnerContext = InOwnerContext;
	InSelf->ScriptStruct = InStruct;
	InSelf->StructInstance = StructInstanceToUse;

	FPyWrapperStructFactory::Get().MapInstance(InSelf->StructInstance, InSelf);
	return 0;
}

void FPyWrapperStruct::Deinit(FPyWrapperStruct* InSelf)
{
	if (InSelf->StructInstance)
	{
		FPyWrapperStructFactory::Get().UnmapInstance(InSelf->StructInstance, Py_TYPE(InSelf));
	}

	if (InSelf->OwnerContext.HasOwner())
	{
		InSelf->OwnerContext.Reset();
	}
	else if (InSelf->StructInstance)
	{
		if (InSelf->ScriptStruct)
		{
			InSelf->ScriptStruct->DestroyStruct(InSelf->StructInstance);
		}

		const IPyWrapperStructAllocationPolicy* AllocPolicy = GetPyWrapperStructAllocationPolicy(InSelf->ScriptStruct);
		if (AllocPolicy)
		{
			AllocPolicy->FreeStruct(InSelf, InSelf->StructInstance);
		}
	}
	InSelf->ScriptStruct = nullptr;
	InSelf->StructInstance = nullptr;
}

bool FPyWrapperStruct::ValidateInternalState(FPyWrapperStruct* InSelf)
{
	if (!InSelf->ScriptStruct)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - ScriptStruct is null!"));
		return false;
	}

	if (!InSelf->StructInstance)
	{
		PyUtil::SetPythonError(PyExc_Exception, Py_TYPE(InSelf), TEXT("Internal Error - StructInstance is null!"));
		return false;
	}

	return true;
}

FPyWrapperStruct* FPyWrapperStruct::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperStructType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperStruct*)InPyObject;
	}

	return nullptr;
}

FPyWrapperStruct* FPyWrapperStruct::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperStructType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperStructType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperStruct*)InPyObject;
	}

	if (PyUtil::HasLength(InPyObject) && !PyUtil::IsMappingType(InPyObject))
	{
		FPyWrapperStructPtr NewStruct = FPyWrapperStructPtr::StealReference(FPyWrapperStruct::New(InType));
		if (FPyWrapperStruct::Init(NewStruct) != 0)
		{
			return nullptr;
		}

		FPyWrapperStructMetaData* StructMetaData = FPyWrapperStructMetaData::GetMetaData(NewStruct);
		if (!StructMetaData)
		{
			return nullptr;
		}

		// Don't allow conversion from sequences with more items than we have InitParams
		const int32 SequenceLen = PyObject_Length(InPyObject);
		if (SequenceLen > StructMetaData->InitParams.Num())
		{
			PyUtil::SetPythonError(PyExc_Exception, NewStruct.Get(), *FString::Printf(TEXT("Struct has %d initialization parameters, but the given sequence had %d elements"), StructMetaData->InitParams.Num(), SequenceLen));
			return nullptr;
		}

		// Attempt to convert each entry in the sequence to the corresponding struct entry
		FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(InPyObject));
		if (PyObjIter)
		{
			for (const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam : StructMetaData->InitParams)
			{
				FPyObjectPtr SequenceItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
				if (!SequenceItem)
				{
					if (PyErr_Occurred())
					{
						return nullptr;
					}
					break;
				}

				const TArray<void*> NoArchetypeInsts;
				const int Result = PyUtil::SetPropertyValue(NewStruct->ScriptStruct, NewStruct->StructInstance, SequenceItem, InitParam.ParamProp, InitParam.ParamName.GetData(), nullptr, 0, false, *PyUtil::GetErrorContext(NewStruct.Get()), NoArchetypeInsts);
				if (Result != 0)
				{
					return nullptr;
				}
			}
		}

		SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
		return NewStruct.Release();
	}

	if (PyUtil::IsMappingType(InPyObject))
	{
		FPyWrapperStructPtr NewStruct = FPyWrapperStructPtr::StealReference(FPyWrapperStruct::New(InType));
		if (FPyWrapperStruct::Init(NewStruct) != 0)
		{
			return nullptr;
		}

		FPyWrapperStructMetaData* StructMetaData = FPyWrapperStructMetaData::GetMetaData(NewStruct);
		if (!StructMetaData)
		{
			return nullptr;
		}

		// Don't allow conversion from dicts with more items than we have InitParams
		const int32 DictLen = PyObject_Length(InPyObject);
		if (DictLen > StructMetaData->InitParams.Num())
		{
			PyUtil::SetPythonError(PyExc_Exception, NewStruct.Get(), *FString::Printf(TEXT("Struct has %d initialization parameters, but the given dict had %d elements"), StructMetaData->InitParams.Num(), DictLen));
			return nullptr;
		}

		// Attempt to convert each matching entry in the dict to the corresponding struct entry
		for (const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam : StructMetaData->InitParams)
		{
			PyObject* MappingItem = PyMapping_GetItemString(InPyObject, (char*)InitParam.ParamName.GetData());
			if (MappingItem)
			{
				TArray<void*> NoArchetypeInsts;
				const int Result = PyUtil::SetPropertyValue(NewStruct->ScriptStruct, NewStruct->StructInstance, MappingItem, InitParam.ParamProp, InitParam.ParamName.GetData(), nullptr, 0, false, *PyUtil::GetErrorContext(NewStruct.Get()), NoArchetypeInsts);
				if (Result != 0)
				{
					return nullptr;
				}
			}
			else
			{
				// Clear the look-up error
				PyErr_Clear();
			}
		}

		SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
		return NewStruct.Release();
	}

	return nullptr;
}

int FPyWrapperStruct::MakeStruct(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	FPyWrapperStructMetaData* StructMetaData = FPyWrapperStructMetaData::GetMetaData(InSelf);
	if (!StructMetaData)
	{
		return -1;
	}

	// If this struct has a custom make function, use that rather than the generic version
	if (StructMetaData->MakeFunc.Func)
	{
		return CallMakeFunction_Impl(InSelf, InArgs, InKwds, StructMetaData->MakeFunc);
	}

	// We can early out if we have no data to apply
	if (PyTuple_Size(InArgs) == 0 && (!InKwds || PyDict_Size(InKwds) == 0))
	{
		return 0;
	}

	// Generic implementation just tries to assign each property
	TArray<PyObject*> Params;
	if (!PyGenUtil::ParseMethodParameters(InArgs, InKwds, StructMetaData->InitParams, "call", Params))
	{
		return -1;
	}

	for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ++ParamIndex)
	{
		PyObject* PyValue = Params[ParamIndex];
		if (PyValue)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam = StructMetaData->InitParams[ParamIndex];
			if (!PyConversion::NativizeProperty_InContainer(PyValue, InitParam.ParamProp, InSelf->StructInstance, 0))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert type '%s' to property '%s' (%s) for attribute '%s' on '%s'"), *PyUtil::GetFriendlyTypename(PyValue), *InitParam.ParamProp->GetName(), *InitParam.ParamProp->GetClass()->GetName(), UTF8_TO_TCHAR(InitParam.ParamName.GetData()), *InSelf->ScriptStruct->GetName()));
				return -1;
			}
		}
	}

	return 0;
}

PyObject* FPyWrapperStruct::BreakStruct(FPyWrapperStruct* InSelf)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	FPyWrapperStructMetaData* StructMetaData = FPyWrapperStructMetaData::GetMetaData(InSelf);
	if (!StructMetaData)
	{
		return nullptr;
	}

	// If this struct has a custom break function, use that rather than use the generic version
	if (StructMetaData->BreakFunc.Func)
	{
		return CallBreakFunction_Impl(InSelf, StructMetaData->BreakFunc);
	}

	// Generic implementation just creates a dict with each property
	FPyObjectPtr PyPropDict = FPyObjectPtr::StealReference(PyDict_New());
	for (int32 ParamIndex = 0; ParamIndex < StructMetaData->InitParams.Num(); ++ParamIndex)
	{
		const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam = StructMetaData->InitParams[ParamIndex];
		PyObject* PyValue = nullptr;
		if (!PyConversion::PythonizeProperty_InContainer(InitParam.ParamProp, InSelf->StructInstance, 0, PyValue))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert property '%s' (%s) for attribute '%s' on '%s'"), *InitParam.ParamProp->GetName(), *InitParam.ParamProp->GetClass()->GetName(), UTF8_TO_TCHAR(InitParam.ParamName.GetData()), *InSelf->ScriptStruct->GetName()));
			return nullptr;
		}
		FPyObjectPtr Value = FPyObjectPtr::StealReference(PyValue);
		FPyObjectPtr Key = FPyObjectPtr::StealReference(PyUnicode_FromString(InitParam.ParamName.GetData()));
		PyDict_SetItem(PyPropDict, Key, Value);
	}

	return PyPropDict.Release();
}

PyObject* FPyWrapperStruct::GetPropertyValue(FPyWrapperStruct* InSelf, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	return PyGenUtil::GetPropertyValue(InSelf->ScriptStruct, InSelf->StructInstance, InPropDef, InPythonAttrName, (PyObject*)InSelf, *PyUtil::GetErrorContext(InSelf));
}

int FPyWrapperStruct::SetPropertyValue(FPyWrapperStruct* InSelf, PyObject* InValue, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName, const EPropertyAccessChangeNotifyMode InNotifyMode, const uint64 InReadOnlyFlags)
{
	if (!ValidateInternalState(InSelf))
	{
		return -1;
	}

	// Structs are not a template by default (for standalone structs)
	bool OwnerIsTemplate = false;
	if (const UObject* OwnerObject = PyUtil::GetOwnerObject((PyObject*)InSelf))
	{
		OwnerIsTemplate = PropertyAccessUtil::IsObjectTemplate(OwnerObject);
	}

	const TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = FPyWrapperOwnerContext((PyObject*)InSelf, InPropDef.Prop).BuildChangeNotify(InNotifyMode);
	const TArray<void*> NoArchetypeInsts;
	return PyGenUtil::SetPropertyValue(InSelf->ScriptStruct, InSelf->StructInstance, InValue, InPropDef, InPythonAttrName, ChangeNotify.Get(), InReadOnlyFlags, OwnerIsTemplate, *PyUtil::GetErrorContext(InSelf), NoArchetypeInsts);
}

int FPyWrapperStruct::CallMakeFunction_Impl(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef)
{
	TArray<PyObject*> Params;
	if (!PyGenUtil::ParseMethodParameters(InArgs, InKwds, InFuncDef.InputParams, "call", Params))
	{
		return -1;
	}

	if (ensureAlways(InFuncDef.Func))
	{
		UClass* Class = InFuncDef.Func->GetOwnerClass();
		UObject* Obj = Class->GetDefaultObject();

		PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
		PyGenUtil::ApplyParamDefaults(FuncParams.GetMemory(), InFuncDef.InputParams);
		for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ++ParamIndex)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ParamDef = InFuncDef.InputParams[ParamIndex];

			PyObject* PyValue = Params[ParamIndex];
			if (PyValue)
			{
				if (!PyConversion::NativizeProperty_InContainer(PyValue, ParamDef.ParamProp, FuncParams.GetMemory(), 0))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert parameter '%s' when calling function '%s.%s' on '%s'"), UTF8_TO_TCHAR(ParamDef.ParamName.GetData()), *Class->GetName(), *InFuncDef.Func->GetName(), *Obj->GetName()));
					return -1;
				}
			}
		}
		if (!PyUtil::InvokeFunctionCall(Obj, InFuncDef.Func, FuncParams.GetMemory(), *PyUtil::GetErrorContext(InSelf)))
		{
			return -1;
		}
		if (ensureAlways(InFuncDef.OutputParams.Num() == 1 && CastField<FStructProperty>(InFuncDef.OutputParams[0].ParamProp) && CastFieldChecked<const FStructProperty>(InFuncDef.OutputParams[0].ParamProp)->Struct->IsChildOf(InSelf->ScriptStruct)))
		{
			// Copy the result back onto ourself
			const PyGenUtil::FGeneratedWrappedMethodParameter& ReturnParam = InFuncDef.OutputParams[0];
			const void* ReturnArgInstance = ReturnParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
			InSelf->ScriptStruct->CopyScriptStruct(InSelf->StructInstance, ReturnArgInstance);
		}
	}

	return 0;
}

PyObject* FPyWrapperStruct::CallBreakFunction_Impl(FPyWrapperStruct* InSelf, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef)
{
	if (ensureAlways(InFuncDef.Func))
	{
		UClass* Class = InFuncDef.Func->GetOwnerClass();
		UObject* Obj = Class->GetDefaultObject();

		PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
		if (ensureAlways(InFuncDef.InputParams.Num() == 1 && CastField<FStructProperty>(InFuncDef.InputParams[0].ParamProp) && InSelf->ScriptStruct->IsChildOf(CastFieldChecked<const FStructProperty>(InFuncDef.InputParams[0].ParamProp)->Struct)))
		{
			// Copy us as the 'self' argument
			const PyGenUtil::FGeneratedWrappedMethodParameter& SelfParam = InFuncDef.InputParams[0];
			void* SelfArgInstance = SelfParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
			CastFieldChecked<const FStructProperty>(SelfParam.ParamProp)->Struct->CopyScriptStruct(SelfArgInstance, InSelf->StructInstance);
		}
		if (!PyUtil::InvokeFunctionCall(Obj, InFuncDef.Func, FuncParams.GetMemory(), *PyUtil::GetErrorContext(InSelf)))
		{
			return nullptr;
		}
		FPyObjectPtr PyPropDict = FPyObjectPtr::StealReference(PyDict_New());
		for (int32 ParamIndex = 0; ParamIndex < InFuncDef.OutputParams.Num(); ++ParamIndex)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ParamDef = InFuncDef.OutputParams[ParamIndex];

			PyObject* PyValue = nullptr;
			if (!PyConversion::PythonizeProperty_InContainer(ParamDef.ParamProp, FuncParams.GetMemory(), 0, PyValue, EPyConversionMethod::Steal))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert return property '%s' when calling function '%s.%s' on '%s'"), UTF8_TO_TCHAR(ParamDef.ParamName.GetData()), *Class->GetName(), *InFuncDef.Func->GetName(), *Obj->GetName()));
				return nullptr;
			}
			FPyObjectPtr Key = FPyObjectPtr::StealReference(PyUnicode_FromString(ParamDef.ParamName.GetData()));
			FPyObjectPtr Value = FPyObjectPtr::StealReference(PyValue);
			PyDict_SetItem(PyPropDict, Key, Value);
		}
		return PyPropDict.Release();
	}

	Py_RETURN_NONE;
}

PyObject* FPyWrapperStruct::CallDynamicFunction_Impl(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const PyGenUtil::FGeneratedWrappedMethodParameter& InSelfParam, const PyGenUtil::FGeneratedWrappedMethodParameter& InSelfReturn, const char* InPythonFuncName)
{
	TArray<PyObject*> Params;
	if ((InArgs || InKwds) && !PyGenUtil::ParseMethodParameters(InArgs, InKwds, InFuncDef.InputParams, InPythonFuncName, Params))
	{
		return nullptr;
	}

	if (ensureAlways(InFuncDef.Func))
	{
		UClass* Class = InFuncDef.Func->GetOwnerClass();
		UObject* Obj = Class->GetDefaultObject();

		// Deprecated functions emit a warning
		if (InFuncDef.DeprecationMessage.IsSet())
		{
			if (PyUtil::SetPythonWarning(PyExc_DeprecationWarning, InSelf, *FString::Printf(TEXT("Function '%s' on '%s' is deprecated: %s"), UTF8_TO_TCHAR(InPythonFuncName), *Class->GetName(), *InFuncDef.DeprecationMessage.GetValue())) == -1)
			{
				// -1 from SetPythonWarning means the warning should be an exception
				return nullptr;
			}
		}

		PY_UFUNCTION_STACK(FuncParams, InFuncDef.Func);
		PyGenUtil::ApplyParamDefaults(FuncParams.GetMemory(), InFuncDef.InputParams);
		if (ensureAlways(CastField<FStructProperty>(InSelfParam.ParamProp) && InSelf->ScriptStruct->IsChildOf(CastFieldChecked<const FStructProperty>(InSelfParam.ParamProp)->Struct)))
		{
			void* SelfArgInstance = InSelfParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
			CastFieldChecked<const FStructProperty>(InSelfParam.ParamProp)->Struct->CopyScriptStruct(SelfArgInstance, InSelf->StructInstance);
		}
		for (int32 ParamIndex = 0; ParamIndex < Params.Num(); ++ParamIndex)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ParamDef = InFuncDef.InputParams[ParamIndex];

			PyObject* PyValue = Params[ParamIndex];
			if (PyValue)
			{
				if (!PyConversion::NativizeProperty_InContainer(PyValue, ParamDef.ParamProp, FuncParams.GetMemory(), 0))
				{
					PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert parameter '%s' when calling function '%s.%s' on '%s'"), UTF8_TO_TCHAR(ParamDef.ParamName.GetData()), *Class->GetName(), *InFuncDef.Func->GetName(), *Obj->GetName()));
					return nullptr;
				}
			}
		}
		const FString ErrorCtxt = PyUtil::GetErrorContext(InSelf);
		if (!PyUtil::InvokeFunctionCall(Obj, InFuncDef.Func, FuncParams.GetMemory(), *ErrorCtxt))
		{
			return nullptr;
		}
		if (InSelfReturn.ParamProp && ensureAlways(CastField<FStructProperty>(InSelfReturn.ParamProp) && CastFieldChecked<const FStructProperty>(InSelfReturn.ParamProp)->Struct->IsChildOf(InSelf->ScriptStruct)))
		{
			// Copy the 'self' return value back onto ourself
			const void* SelfReturnInstance = InSelfReturn.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
			InSelf->ScriptStruct->CopyScriptStruct(InSelf->StructInstance, SelfReturnInstance);
		}
		return PyGenUtil::PackReturnValues(FuncParams.GetMemory(), InFuncDef.OutputParams, *ErrorCtxt, *FString::Printf(TEXT("function '%s.%s' on '%s'"), *Class->GetName(), *InFuncDef.Func->GetName(), *Obj->GetName()));
	}

	Py_RETURN_NONE;
}

PyObject* FPyWrapperStruct::CallDynamicMethodNoArgs_Impl(FPyWrapperStruct* InSelf, void* InClosure)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	const PyGenUtil::FGeneratedWrappedDynamicMethod* Closure = (PyGenUtil::FGeneratedWrappedDynamicMethod*)InClosure;
	return CallDynamicFunction_Impl(InSelf, nullptr, nullptr, Closure->MethodFunc, Closure->SelfParam, Closure->SelfReturn, Closure->MethodName.GetData());
}

PyObject* FPyWrapperStruct::CallDynamicMethodWithArgs_Impl(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds, void* InClosure)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	const PyGenUtil::FGeneratedWrappedDynamicMethod* Closure = (PyGenUtil::FGeneratedWrappedDynamicMethod*)InClosure;
	return CallDynamicFunction_Impl(InSelf, InArgs, InKwds, Closure->MethodFunc, Closure->SelfParam, Closure->SelfReturn, Closure->MethodName.GetData());
}

PyObject* FPyWrapperStruct::CallOperatorFunction_Impl(FPyWrapperStruct* InSelf, PyObject* InRHS, const PyGenUtil::FGeneratedWrappedOperatorFunction& InOpFunc, const TOptional<EPyConversionResultState> InRequiredConversionResult, FPyConversionResult* OutRHSConversionResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutRHSConversionResult);

	if (ensureAlways(InOpFunc.Func))
	{
		UClass* Class = InOpFunc.Func->GetOwnerClass();
		UObject* Obj = Class->GetDefaultObject();

		// Build the input arguments (failures here aren't fatal as we may have multiple functions to evaluate on the stack, only one of which may accept the RHS parameter)
		PY_UFUNCTION_STACK(FuncParams, InOpFunc.Func);
		PyGenUtil::ApplyParamDefaults(FuncParams.GetMemory(), InOpFunc.AdditionalParams);
		if (InOpFunc.OtherParam.ParamProp)
		{
			TArray<void*> NoArchetypeInsts;
			const FPyConversionResult RHSResult = PyConversion::NativizeProperty_InContainer(InRHS, InOpFunc.OtherParam.ParamProp, FuncParams.GetMemory(), 0, NoArchetypeInsts, nullptr, PyConversion::ESetErrorState::No);
			SetOptionalPyConversionResult(RHSResult, OutRHSConversionResult);

			if (!RHSResult)
			{
				return nullptr;
			}

			if (InRequiredConversionResult.IsSet() && RHSResult.GetState() != InRequiredConversionResult.GetValue())
			{
				return nullptr;
			}
		}
		if (ensureAlways(CastField<FStructProperty>(InOpFunc.SelfParam.ParamProp) && InSelf->ScriptStruct->IsChildOf(CastFieldChecked<const FStructProperty>(InOpFunc.SelfParam.ParamProp)->Struct)))
		{
			void* StructArgInstance = InOpFunc.SelfParam.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
			CastFieldChecked<const FStructProperty>(InOpFunc.SelfParam.ParamProp)->Struct->CopyScriptStruct(StructArgInstance, InSelf->StructInstance);
		}
		if (!PyUtil::InvokeFunctionCall(Obj, InOpFunc.Func, FuncParams.GetMemory(), *PyUtil::GetErrorContext(InSelf)))
		{
			return nullptr;
		}

		PyObject* ReturnPyObj = nullptr;
		if (InOpFunc.SelfReturn.ParamProp)
		{
			if (ensureAlways(CastField<FStructProperty>(InOpFunc.SelfReturn.ParamProp) && CastFieldChecked<const FStructProperty>(InOpFunc.SelfReturn.ParamProp)->Struct->IsChildOf(InSelf->ScriptStruct)))
			{
				// Copy the 'self' return value back onto ourself
				const void* SelfReturnInstance = InOpFunc.SelfReturn.ParamProp->ContainerPtrToValuePtr<void>(FuncParams.GetMemory());
				InSelf->ScriptStruct->CopyScriptStruct(InSelf->StructInstance, SelfReturnInstance);
			}

			Py_INCREF(InSelf);
			ReturnPyObj = (PyObject*)InSelf;
		}
		else if (InOpFunc.ReturnParam.ParamProp)
		{
			if (!PyConversion::PythonizeProperty_InContainer(InOpFunc.ReturnParam.ParamProp, FuncParams.GetMemory(), 0, ReturnPyObj, EPyConversionMethod::Steal))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert return property '%s' (%s) when calling function '%s' on '%s'"), *InOpFunc.ReturnParam.ParamProp->GetName(), *InOpFunc.ReturnParam.ParamProp->GetClass()->GetName(), *InOpFunc.Func->GetName(), *Obj->GetName()));
				return nullptr;
			}
		}
		else
		{
			Py_INCREF(Py_None);
			ReturnPyObj = Py_None;
		}

		return ReturnPyObj;
	}

	return nullptr;
}

PyObject* FPyWrapperStruct::CallOperator_Impl(FPyWrapperStruct* InSelf, PyObject* InRHS, const PyGenUtil::EGeneratedWrappedOperatorType InOpType)
{
	if (!ValidateInternalState(InSelf))
	{
		return nullptr;
	}

	// Walk up the inheritance chain to find the correct op functions to use
	// We take the first one with any functions set, so that overrides on a derived type hide those from the base type
	TArray<PyGenUtil::FGeneratedWrappedOperatorFunction>* OpFuncsPtr = nullptr;
	{
		PyTypeObject* PyType = Py_TYPE(InSelf);
		do
		{
			PyTypeObject* NextPyType = nullptr;
			if (FPyWrapperStructMetaData* PyWrapperMetaData = FPyWrapperStructMetaData::GetMetaData(PyType))
			{
				if (PyWrapperMetaData->OpStacks[(int32)InOpType].Funcs.Num() > 0)
				{
					OpFuncsPtr = &PyWrapperMetaData->OpStacks[(int32)InOpType].Funcs;
					break;
				}

				if (const UScriptStruct* SuperStruct = PyWrapperMetaData->Struct ? Cast<UScriptStruct>(PyWrapperMetaData->Struct->GetSuperStruct()) : nullptr)
				{
					NextPyType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(SuperStruct);
				}
			}
			PyType = NextPyType;
		}
		while (PyType);
	}

	if (OpFuncsPtr)
	{
		// We process the operator stack in two passes:
		//	- The first pass looks for a signature that exactly matches the given argument
		//	- The second pass allows type coercion to occur when calling the signature
		// We use the first pass to find a function that may be called for the second pass
		const PyGenUtil::FGeneratedWrappedOperatorFunction* CoercedOpFunc = nullptr;
		for (const PyGenUtil::FGeneratedWrappedOperatorFunction& OpFunc : *OpFuncsPtr)
		{
			FPyConversionResult RHSConversionResult = FPyConversionResult::Failure();
			PyObject* PyResult = CallOperatorFunction_Impl(InSelf, InRHS, OpFunc, EPyConversionResultState::Success, &RHSConversionResult);
			if (PyResult)
			{
				return PyResult;
			}
			else if (!CoercedOpFunc && RHSConversionResult.GetState() == EPyConversionResultState::SuccessWithCoercion)
			{
				CoercedOpFunc = &OpFunc;
			}
		}
		if (CoercedOpFunc)
		{
			PyObject* PyResult = CallOperatorFunction_Impl(InSelf, InRHS, *CoercedOpFunc);
			if (PyResult)
			{
				return PyResult;
			}
		}
	}

	Py_INCREF(Py_NotImplemented);
	return Py_NotImplemented;
}

PyObject* FPyWrapperStruct::Getter_Impl(FPyWrapperStruct* InSelf, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedGetSet* Closure = (PyGenUtil::FGeneratedWrappedGetSet*)InClosure;
	return GetPropertyValue(InSelf, Closure->Prop, Closure->GetSetName.GetData());
}

int FPyWrapperStruct::Setter_Impl(FPyWrapperStruct* InSelf, PyObject* InValue, void* InClosure)
{
	const PyGenUtil::FGeneratedWrappedGetSet* Closure = (PyGenUtil::FGeneratedWrappedGetSet*)InClosure;
	return SetPropertyValue(InSelf, InValue, Closure->Prop, Closure->GetSetName.GetData());
}

PyTypeObject InitializePyWrapperStructType()
{
	struct FFuncs
	{
		static PyObject* New(PyTypeObject* InType, PyObject* InArgs, PyObject* InKwds)
		{
			return (PyObject*)FPyWrapperStruct::New(InType);
		}

		static void Dealloc(FPyWrapperStruct* InSelf)
		{
			FPyWrapperStruct::Free(InSelf);
		}

		static int Init(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			return FPyWrapperStruct::Init(InSelf);
		}

		static PyObject* Str(FPyWrapperStruct* InSelf)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			const FString ExportedStruct = PyUtil::GetFriendlyStructValue(InSelf->ScriptStruct, InSelf->StructInstance, PPF_IncludeTransient);
			return PyUnicode_FromFormat("<Struct '%s' (%p) %s>", TCHAR_TO_UTF8(*InSelf->ScriptStruct->GetName()), InSelf->StructInstance, TCHAR_TO_UTF8(*ExportedStruct));
		}

		static PyObject* RichCmp(FPyWrapperStruct* InSelf, PyObject* InOther, int InOp)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			auto PythonCmpOpToWrapperOp = [InOp]() -> PyGenUtil::EGeneratedWrappedOperatorType
			{
				switch (InOp)
				{
				case Py_EQ:
					return PyGenUtil::EGeneratedWrappedOperatorType::Equal;
				case Py_NE:
					return PyGenUtil::EGeneratedWrappedOperatorType::NotEqual;
				case Py_LT:
					return PyGenUtil::EGeneratedWrappedOperatorType::Less;
				case Py_LE:
					return PyGenUtil::EGeneratedWrappedOperatorType::LessEqual;
				case Py_GT:
					return PyGenUtil::EGeneratedWrappedOperatorType::Greater;
				case Py_GE:
					return PyGenUtil::EGeneratedWrappedOperatorType::GreaterEqual;
				default:
					checkf(false, TEXT("Unknown Python comparison type!"));
					break;
				}
				return PyGenUtil::EGeneratedWrappedOperatorType::Equal;
			};

			return FPyWrapperStruct::CallOperator_Impl(InSelf, InOther, PythonCmpOpToWrapperOp());
		}

		static PyUtil::FPyHashType Hash(FPyWrapperStruct* InSelf)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return -1;
			}

			// UserDefinedStruct overrides GetStructTypeHash to work without valid CppStructOps
			if (InSelf->ScriptStruct->IsA<UUserDefinedStruct>() || (InSelf->ScriptStruct->GetCppStructOps() && InSelf->ScriptStruct->GetCppStructOps()->HasGetTypeHash()))
			{
				const PyUtil::FPyHashType PyHash = (PyUtil::FPyHashType)InSelf->ScriptStruct->GetStructTypeHash(InSelf->StructInstance);
				return PyHash != -1 ? PyHash : 0;
			}

			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Type cannot be hashed"));
			return -1;
		}
	};

#define DEFINE_INQUIRY_OPERATOR_FUNC(OP, NOT_IMPLEMENTED_VALUE)																	\
	static int OP(FPyWrapperStruct* InLHS)																						\
	{																															\
		PyObject* PyResult = FPyWrapperStruct::CallOperator_Impl(InLHS, nullptr, PyGenUtil::EGeneratedWrappedOperatorType::OP);	\
		const int Result = PyObjectResultToInt(PyResult, (NOT_IMPLEMENTED_VALUE));												\
		Py_XDECREF(PyResult);																									\
		return Result;																											\
	}
#define DEFINE_UNARY_OPERATOR_FUNC(OP)																							\
	static PyObject* OP(FPyWrapperStruct* InLHS)																				\
	{																															\
		return FPyWrapperStruct::CallOperator_Impl(InLHS, nullptr, PyGenUtil::EGeneratedWrappedOperatorType::OP);				\
	}
#define DEFINE_BINARY_OPERATOR_FUNC(OP)																							\
	static PyObject* OP(FPyWrapperStruct* InLHS, PyObject* InRHS)																\
	{																															\
		return FPyWrapperStruct::CallOperator_Impl(InLHS, InRHS, PyGenUtil::EGeneratedWrappedOperatorType::OP);					\
	}
	struct FNumberFuncs
	{
		static int PyObjectResultToInt(PyObject* PyResult, int NotImplementedValue)
		{
			int Result = -1;
			if (PyResult)
			{
				if (PyResult == Py_NotImplemented)
				{
					Result = NotImplementedValue;
				}
				else if (PyBool_Check(PyResult))
				{
					Result = (PyResult == Py_True) ? 1 : 0;
				}
				else
				{
					PyConversion::Nativize(PyResult, Result);
				}
			}
			return Result;
		}

		DEFINE_INQUIRY_OPERATOR_FUNC(Bool, 1)
		DEFINE_BINARY_OPERATOR_FUNC(Add)
		DEFINE_BINARY_OPERATOR_FUNC(InlineAdd)
		DEFINE_BINARY_OPERATOR_FUNC(Subtract)
		DEFINE_BINARY_OPERATOR_FUNC(InlineSubtract)
		DEFINE_BINARY_OPERATOR_FUNC(Multiply)
		DEFINE_BINARY_OPERATOR_FUNC(InlineMultiply)
		DEFINE_BINARY_OPERATOR_FUNC(Divide)
		DEFINE_BINARY_OPERATOR_FUNC(InlineDivide)
		DEFINE_BINARY_OPERATOR_FUNC(Modulus)
		DEFINE_BINARY_OPERATOR_FUNC(InlineModulus)
		DEFINE_BINARY_OPERATOR_FUNC(And)
		DEFINE_BINARY_OPERATOR_FUNC(InlineAnd)
		DEFINE_BINARY_OPERATOR_FUNC(Or)
		DEFINE_BINARY_OPERATOR_FUNC(InlineOr)
		DEFINE_BINARY_OPERATOR_FUNC(Xor)
		DEFINE_BINARY_OPERATOR_FUNC(InlineXor)
		DEFINE_BINARY_OPERATOR_FUNC(RightShift)
		DEFINE_BINARY_OPERATOR_FUNC(InlineRightShift)
		DEFINE_BINARY_OPERATOR_FUNC(LeftShift)
		DEFINE_BINARY_OPERATOR_FUNC(InlineLeftShift)
		DEFINE_UNARY_OPERATOR_FUNC(Negated)
	};
#undef DEFINE_INQUIRY_OPERATOR_FUNC
#undef DEFINE_UNARY_OPERATOR_FUNC
#undef DEFINE_BINARY_OPERATOR_FUNC

	struct FMethods
	{
		static PyObject* PostInit(FPyWrapperStruct* InSelf)
		{
			Py_RETURN_NONE;
		}

		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:cast", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperStruct::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* StaticStruct(PyTypeObject* InType)
		{
			UScriptStruct* Struct = FPyWrapperStructMetaData::GetStruct(InType);
			return PyConversion::Pythonize(Struct);
		}

		static PyObject* Copy(FPyWrapperStruct* InSelf)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			return (PyObject*)FPyWrapperStructFactory::Get().CreateInstance(InSelf->ScriptStruct, InSelf->StructInstance, FPyWrapperOwnerContext(), EPyConversionMethod::Copy);
		}

		static PyObject* Assign(FPyWrapperStruct* InSelf, PyObject* InArgs)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:assign", &PyObj))
			{
				return nullptr;
			}
			check(PyObj);

			FPyWrapperStructPtr PyStruct = FPyWrapperStructPtr::StealReference(FPyWrapperStruct::CastPyObject(PyObj, Py_TYPE(InSelf)));
			if (!PyStruct)
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InSelf)));
				return nullptr;
			}

			if (PyStruct && ensureAlways(PyStruct->ScriptStruct->IsChildOf(InSelf->ScriptStruct)))
			{
				InSelf->ScriptStruct->CopyScriptStruct(InSelf->StructInstance, PyStruct->StructInstance);
			}

			Py_RETURN_NONE;
		}

		static PyObject* ToTuple(FPyWrapperStruct* InSelf)
		{
			FPyObjectPtr PyDict = FPyObjectPtr::StealReference(FPyWrapperStruct::BreakStruct(InSelf));
			if (!PyDict || !PyDict_Check(PyDict))
			{
				PyErr_SetString(PyExc_TypeError, "Expected a dictionary object");
				return nullptr;
			}

			// Get the "values" from the object
			FPyObjectPtr Values = FPyObjectPtr::StealReference(PyDict_Values(PyDict));
			if (!Values)
			{
				return nullptr; // Exception set internally
			}

			// Convert the values (which are iterable) into a new tuple
			return PyList_AsTuple(Values);
		}

		static PyObject* ToDict(FPyWrapperStruct* InSelf)
		{
			return FPyWrapperStruct::BreakStruct(InSelf);
		}

		static PyGenUtil::FGeneratedWrappedProperty FindEditorPropertyImpl(FPyWrapperStruct* InSelf, const FName InPythonPropName)
		{
			PyGenUtil::FGeneratedWrappedProperty WrappedPropDef;

			const FName ResolvedName = FPyWrapperStructMetaData::ResolvePropertyName(InSelf, InPythonPropName);
			const FProperty* ResolvedProp = InSelf->ScriptStruct->FindPropertyByName(ResolvedName);
			if (!ResolvedProp)
			{
				ResolvedProp = InSelf->ScriptStruct->CustomFindProperty(ResolvedName);
			}
			if (!ResolvedProp)
			{
				PyUtil::SetPythonError(PyExc_Exception, InSelf, *FString::Printf(TEXT("Failed to find property '%s' for attribute '%s' on '%s'"), *ResolvedName.ToString(), *InPythonPropName.ToString(), *InSelf->ScriptStruct->GetName()));
				return WrappedPropDef;
			}

			TOptional<FString> PropDeprecationMessage;
			{
				FString PropDeprecationMessageStr;
				if (FPyWrapperStructMetaData::IsPropertyDeprecated(InSelf, InPythonPropName, &PropDeprecationMessageStr))
				{
					PropDeprecationMessage = MoveTemp(PropDeprecationMessageStr);
				}
			}

			if (PropDeprecationMessage.IsSet())
			{
				WrappedPropDef.SetProperty(ResolvedProp, PyGenUtil::FGeneratedWrappedProperty::SPF_None);
				WrappedPropDef.DeprecationMessage = MoveTemp(PropDeprecationMessage);
			}
			else
			{
				WrappedPropDef.SetProperty(ResolvedProp);
			}

			return WrappedPropDef;
		}

		static PyObject* GetEditorProperty(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyNameObj = nullptr;

			static const char *ArgsKwdList[] = { "name", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:get_editor_property", (char**)ArgsKwdList, &PyNameObj))
			{
				return nullptr;
			}

			FName Name;
			if (!PyConversion::Nativize(PyNameObj, Name))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(InSelf)));
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedProperty WrappedPropDef = FindEditorPropertyImpl(InSelf, Name);
			if (!WrappedPropDef.Prop)
			{
				return nullptr;
			}

			return FPyWrapperStruct::GetPropertyValue(InSelf, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()));
		}

		static PyObject* SetEditorProperty(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyNameObj = nullptr;
			PyObject* PyValueObj = nullptr;
			PyObject* PyNotifyModeObj = nullptr;

			static const char *ArgsKwdList[] = { "name", "value", "notify_mode", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "OO|O:set_editor_property", (char**)ArgsKwdList, &PyNameObj, &PyValueObj, &PyNotifyModeObj))
			{
				return nullptr;
			}

			FName Name;
			if (!PyConversion::Nativize(PyNameObj, Name))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'name' (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(InSelf)));
				return nullptr;
			}

			static const UEnum* PropertyAccessChangeNotifyModeEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/CoreUObject.EPropertyAccessChangeNotifyMode"));

			EPropertyAccessChangeNotifyMode NotifyMode = EPropertyAccessChangeNotifyMode::Default;
			if (PyNotifyModeObj && !PyConversion::NativizeEnumEntry(PyNotifyModeObj, PropertyAccessChangeNotifyModeEnum, NotifyMode))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert 'notify_mode' (%s) to 'PropertyAccessChangeNotifyMode'"), *PyUtil::GetFriendlyTypename(PyNotifyModeObj)));
				return nullptr;
			}

			const PyGenUtil::FGeneratedWrappedProperty WrappedPropDef = FindEditorPropertyImpl(InSelf, Name);
			if (!WrappedPropDef.Prop)
			{
				return nullptr;
			}

			const int Result = FPyWrapperStruct::SetPropertyValue(InSelf, PyValueObj, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), NotifyMode, PropertyAccessUtil::EditorReadOnlyFlags);
			if (Result != 0)
			{
				return nullptr;
			}

			Py_RETURN_NONE;
		}

		static PyObject* SetEditorProperties(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyPropertyInfoDict = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:set_editor_properties", &PyPropertyInfoDict))
			{
				return nullptr;
			}

			if (!PyUtil::IsMappingType(PyPropertyInfoDict))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("'property_info' (%s) is expected to be a mapping type (eg, dict)"), *PyUtil::GetFriendlyTypename(PyPropertyInfoDict)));
				return nullptr;
			}

			// Build up the list of properties and their new values
			typedef TTuple<FName, PyGenUtil::FGeneratedWrappedProperty, FPyObjectPtr> FPropertyInfoPair;
			TArray<FPropertyInfoPair> PropertyInfos;
			{
				FPyObjectPtr PyPropertyInfoDictIter = FPyObjectPtr::StealReference(PyObject_GetIter(PyPropertyInfoDict));
				if (PyPropertyInfoDictIter)
				{
					FPyObjectPtr PyKeyItem;
					while ((PyKeyItem = FPyObjectPtr::StealReference(PyIter_Next(PyPropertyInfoDictIter))))
					{
						FName Name;
						if (!PyConversion::Nativize(PyKeyItem, Name))
						{
							PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert dict key (%s) to 'Name'"), *PyUtil::GetFriendlyTypename(PyKeyItem)));
							return nullptr;
						}

						PyGenUtil::FGeneratedWrappedProperty WrappedPropDef = FindEditorPropertyImpl(InSelf, Name);
						if (!WrappedPropDef.Prop)
						{
							return nullptr;
						}

						FPyObjectPtr PyValueItem = FPyObjectPtr::StealReference(PyObject_GetItem(PyPropertyInfoDict, PyKeyItem));
						if (ensure(PyValueItem))
						{
							PropertyInfos.Add(MakeTuple(Name, MoveTemp(WrappedPropDef), MoveTemp(PyValueItem)));
						}
					}

					// Iteration error?
					if (PyErr_Occurred())
					{
						return nullptr;
					}
				}
			}

			// At this point we know that every property we were asked to set was found, but not that the values are going to be compatible
			// Checking value coercion is tricky without actually doing the conversion, so we'll assume that this point that we need to do the change notifies
			UObject* OwnerObjectInstance = InSelf->OwnerContext.FindChangeNotifyObject();
			if (OwnerObjectInstance)
			{
				OwnerObjectInstance->PreEditChange(nullptr);
			}
			ON_SCOPE_EXIT
			{
				if (OwnerObjectInstance)
				{
					OwnerObjectInstance->PostEditChange();
				}
			};

			// Try and set the value of each property
			for (const FPropertyInfoPair& PropertyInfo : PropertyInfos)
			{
				const FName Name = PropertyInfo.Get<0>();
				const PyGenUtil::FGeneratedWrappedProperty& WrappedPropDef = PropertyInfo.Get<1>();
				PyObject* PyValueObj = PropertyInfo.Get<2>().GetPtr();

				const int Result = FPyWrapperStruct::SetPropertyValue(InSelf, PyValueObj, WrappedPropDef, TCHAR_TO_UTF8(*Name.ToString()), EPropertyAccessChangeNotifyMode::Never, PropertyAccessUtil::EditorReadOnlyFlags);
				if (Result != 0)
				{
					return nullptr;
				}
			}

			Py_RETURN_NONE;
		}

		static PyObject* ExportToText(FPyWrapperStruct* InSelf)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			FString ExportedText;
			InSelf->ScriptStruct->ExportText(ExportedText, InSelf->StructInstance, InSelf->StructInstance, nullptr, PPF_None, nullptr);
			return PyConversion::Pythonize(ExportedText);
		}

		static PyObject* ImportFromText(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			if (!FPyWrapperStruct::ValidateInternalState(InSelf))
			{
				return nullptr;
			}

			PyObject* PyContentObj = nullptr;
			static const char *ArgsKwdList[] = { "content", nullptr };
			if (!PyArg_ParseTupleAndKeywords(InArgs, InKwds, "O:import_text", (char**)ArgsKwdList, &PyContentObj))
			{
				return nullptr;
			}

			FString TextToImport;
			if(PyConversion::Nativize(PyContentObj, TextToImport).Failed())
			{
				return nullptr;
			}

			class FErrorPipe : public FOutputDevice
			{
			public:

				FPyWrapperStruct* Self;
				int32 NumErrors;

				FErrorPipe(FPyWrapperStruct* InSelf)
					: FOutputDevice()
					, Self(InSelf)
					, NumErrors(0)
				{
				}

				virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
				{
					if(Verbosity == ELogVerbosity::Fatal || Verbosity == ELogVerbosity::Error)
					{
						if(++NumErrors == 1)
						{
							PyUtil::SetPythonError(PyExc_RuntimeError, Self, V);
						}
					}
					else if(Verbosity == ELogVerbosity::Warning)
					{
						PyUtil::SetPythonWarning(PyExc_RuntimeWarning, Self, V);
					}
				}
			};

			FErrorPipe ErrorPipe(InSelf);
			InSelf->ScriptStruct->ImportText(*TextToImport, InSelf->StructInstance, nullptr, PPF_None, &ErrorPipe, InSelf->ScriptStruct->GetName());
			return PyConversion::Pythonize(ErrorPipe.NumErrors == 0);
		}
	};

	// NOTE: _T = typing.TypeVar('_T') and Type/Any/Union/Mapping/Sequence/Tuple are defined by the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ PyGenUtil::PostInitFuncName, PyCFunctionCast(&FMethods::PostInit), METH_NOARGS, "_post_init(self) -> None -- called during Unreal struct initialization (equivalent to PostInitProperties in C++)" },
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls: Type[_T], object: Union[object, Mapping[str, object], Iterable[object]]) -> _T -- cast the given object to this Unreal struct type. Can be partial Mapping[fieldName, fiedValue] or a sequence of field values" },
		{ "static_struct", PyCFunctionCast(&FMethods::StaticStruct), METH_NOARGS | METH_CLASS, "static_struct(cls) -> ScriptStruct -- get the Unreal struct of this type" },
		{ "__copy__", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "__copy__(self) -> Any -- copy this Unreal struct" },
		{ "copy", PyCFunctionCast(&FMethods::Copy), METH_NOARGS, "copy(self) -> Any -- copy this Unreal struct" },
		{ "assign", PyCFunctionCast(&FMethods::Assign), METH_VARARGS, "assign(self, other: object) -> None -- assign the value of this Unreal struct to value of the given object" },
		{ "to_tuple", PyCFunctionCast(&FMethods::ToTuple), METH_NOARGS, "to_tuple(self) -> Tuple[object, ...] -- break this Unreal struct into a tuple of its properties" },
		{ "to_dict", PyCFunctionCast(&FMethods::ToDict), METH_NOARGS, "to_dict(self) -> Dict[str, object] -- convert this Unreal struct into a dict of its properties" },
		{ "get_editor_property", PyCFunctionCast(&FMethods::GetEditorProperty), METH_VARARGS | METH_KEYWORDS, "get_editor_property(self, name: Union[Name, str]) -> object -- get the value of any property visible to the editor" },
		{ "set_editor_property", PyCFunctionCast(&FMethods::SetEditorProperty), METH_VARARGS | METH_KEYWORDS, "set_editor_property(self, name: Union[Name, str], value: object, notify_mode: PropertyAccessChangeNotifyMode=PropertyAccessChangeNotifyMode.DEFAULT) -> None -- set the value of any property visible to the editor, ensuring that the pre/post change notifications are called" },
		{ "set_editor_properties", PyCFunctionCast(&FMethods::SetEditorProperties), METH_VARARGS, "set_editor_properties(self, properties: Mapping[str, object]) -> None -- set the value of any properties visible to the editor (from a name->value dict), ensuring that the pre/post change notifications are called" },
		{ "export_text", PyCFunctionCast(&FMethods::ExportToText), METH_NOARGS, "export_text(self) -> str -- exports the content of the Unreal struct of this type" },
		{ "import_text", PyCFunctionCast(&FMethods::ImportFromText), METH_VARARGS | METH_KEYWORDS, "import_text(self, content: str) -> bool -- imports the provided string into the Unreal struct" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = {
		PyVarObject_HEAD_INIT(nullptr, 0)
		"StructBase", /* tp_name */
		sizeof(FPyWrapperStruct), /* tp_basicsize */
	};

	PyType.tp_base = &PyWrapperBaseType;
	PyType.tp_new = (newfunc)&FFuncs::New;
	PyType.tp_dealloc = (destructor)&FFuncs::Dealloc;
	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Str;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;

	PyType.tp_methods = PyMethods;

	PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;
	PyType.tp_doc = "Type for all Unreal exposed struct instances";

	static PyNumberMethods PyNumber;
	PyNumber.nb_bool = (inquiry)&FNumberFuncs::Bool;
	PyNumber.nb_add = (binaryfunc)&FNumberFuncs::Add;
	PyNumber.nb_inplace_add = (binaryfunc)&FNumberFuncs::InlineAdd;
	PyNumber.nb_subtract = (binaryfunc)&FNumberFuncs::Subtract;
	PyNumber.nb_inplace_subtract = (binaryfunc)&FNumberFuncs::InlineSubtract;
	PyNumber.nb_multiply = (binaryfunc)&FNumberFuncs::Multiply;
	PyNumber.nb_inplace_multiply = (binaryfunc)&FNumberFuncs::InlineMultiply;
	PyNumber.nb_true_divide = (binaryfunc)&FNumberFuncs::Divide;
	PyNumber.nb_inplace_true_divide = (binaryfunc)&FNumberFuncs::InlineDivide;
	PyNumber.nb_remainder = (binaryfunc)&FNumberFuncs::Modulus;
	PyNumber.nb_inplace_remainder = (binaryfunc)&FNumberFuncs::InlineModulus;
	PyNumber.nb_and = (binaryfunc)&FNumberFuncs::And;
	PyNumber.nb_inplace_and = (binaryfunc)&FNumberFuncs::InlineAnd;
	PyNumber.nb_or = (binaryfunc)&FNumberFuncs::Or;
	PyNumber.nb_inplace_or = (binaryfunc)&FNumberFuncs::InlineOr;
	PyNumber.nb_xor = (binaryfunc)&FNumberFuncs::Xor;
	PyNumber.nb_inplace_xor = (binaryfunc)&FNumberFuncs::InlineXor;
	PyNumber.nb_rshift = (binaryfunc)&FNumberFuncs::RightShift;
	PyNumber.nb_inplace_rshift = (binaryfunc)&FNumberFuncs::InlineRightShift;
	PyNumber.nb_lshift = (binaryfunc)&FNumberFuncs::LeftShift;
	PyNumber.nb_inplace_lshift = (binaryfunc)&FNumberFuncs::InlineLeftShift;
	PyNumber.nb_negative = (unaryfunc)&FNumberFuncs::Negated;

	PyType.tp_as_number = &PyNumber;

	return PyType;
}

PyTypeObject PyWrapperStructType = InitializePyWrapperStructType();

FPyWrapperStructMetaData::FPyWrapperStructMetaData()
	: Struct(nullptr)
{
}

void FPyWrapperStructMetaData::AddTypeReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Struct);
}

void FPyWrapperStructMetaData::AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector)
{
	FPyWrapperStruct* Self = static_cast<FPyWrapperStruct*>(Instance);
	
	Collector.AddReferencedObject(Self->ScriptStruct);
	if (Self->ScriptStruct && Self->StructInstance && !Self->OwnerContext.HasOwner())
	{
		FPyReferenceCollector::AddReferencedObjectsFromStruct(Collector, Self->ScriptStruct, Self->StructInstance);
	}
}

UScriptStruct* FPyWrapperStructMetaData::GetStruct(PyTypeObject* PyType)
{
	FPyWrapperStructMetaData* PyWrapperMetaData = FPyWrapperStructMetaData::GetMetaData(PyType);
	return PyWrapperMetaData ? PyWrapperMetaData->Struct : nullptr;
}

UScriptStruct* FPyWrapperStructMetaData::GetStruct(FPyWrapperStruct* Instance)
{
	return GetStruct(Py_TYPE(Instance));
}

FName FPyWrapperStructMetaData::ResolvePropertyName(PyTypeObject* PyType, const FName InPythonPropertyName)
{
	if (FPyWrapperStructMetaData* PyWrapperMetaData = FPyWrapperStructMetaData::GetMetaData(PyType))
	{
		if (const FName* MappedPropName = PyWrapperMetaData->PythonProperties.Find(InPythonPropertyName))
		{
			return *MappedPropName;
		}

		if (const UScriptStruct* SuperStruct = PyWrapperMetaData->Struct ? Cast<UScriptStruct>(PyWrapperMetaData->Struct->GetSuperStruct()) : nullptr)
		{
			PyTypeObject* SuperStructPyType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(SuperStruct);
			return ResolvePropertyName(SuperStructPyType, InPythonPropertyName);
		}
	}

	return InPythonPropertyName;
}

FName FPyWrapperStructMetaData::ResolvePropertyName(FPyWrapperStruct* Instance, const FName InPythonPropertyName)
{
	return ResolvePropertyName(Py_TYPE(Instance), InPythonPropertyName);
}

bool FPyWrapperStructMetaData::IsPropertyDeprecated(PyTypeObject* PyType, const FName InPythonPropertyName, FString* OutDeprecationMessage)
{
	if (FPyWrapperStructMetaData* PyWrapperMetaData = FPyWrapperStructMetaData::GetMetaData(PyType))
	{
		if (const FString* DeprecationMessage = PyWrapperMetaData->PythonDeprecatedProperties.Find(InPythonPropertyName))
		{
			if (OutDeprecationMessage)
			{
				*OutDeprecationMessage = *DeprecationMessage;
			}
			return true;
		}

		if (const UScriptStruct* SuperStruct = PyWrapperMetaData->Struct ? Cast<UScriptStruct>(PyWrapperMetaData->Struct->GetSuperStruct()) : nullptr)
		{
			PyTypeObject* SuperStructPyType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(SuperStruct);
			return IsPropertyDeprecated(SuperStructPyType, InPythonPropertyName, OutDeprecationMessage);
		}
	}

	return false;
}

bool FPyWrapperStructMetaData::IsPropertyDeprecated(FPyWrapperStruct* Instance, const FName InPythonPropertyName, FString* OutDeprecationMessage)
{
	return IsPropertyDeprecated(Py_TYPE(Instance), InPythonPropertyName, OutDeprecationMessage);
}

bool FPyWrapperStructMetaData::IsStructDeprecated(PyTypeObject* PyType, FString* OutDeprecationMessage)
{
	if (FPyWrapperStructMetaData* PyWrapperMetaData = FPyWrapperStructMetaData::GetMetaData(PyType))
	{
		if (PyWrapperMetaData->DeprecationMessage.IsSet())
		{
			if (OutDeprecationMessage)
			{
				*OutDeprecationMessage = PyWrapperMetaData->DeprecationMessage.GetValue();
			}
			return true;
		}
	}

	return false;
}

bool FPyWrapperStructMetaData::IsStructDeprecated(FPyWrapperStruct* Instance, FString* OutDeprecationMessage)
{
	return IsStructDeprecated(Py_TYPE(Instance), OutDeprecationMessage);
}

class FPythonGeneratedStructBuilder
{
public:
	FPythonGeneratedStructBuilder(UScriptStruct* InSuperStruct, PyTypeObject* InPyType)
		: StructName()
		, PyType(InPyType)
		, OldStruct(nullptr)
		, NewStruct(nullptr)
	{
		UObject* StructOuter = nullptr;
		PyUtil::GetGeneratedTypeOuterAndName(PyType, StructOuter, StructName);

		// Find any existing struct with the name we want to use
		OldStruct = FindObject<UPythonGeneratedStruct>(StructOuter, *StructName);

		// Create a new struct with a temporary name; we will rename it as part of Finalize
		const FString NewStructName = MakeUniqueObjectName(StructOuter, UPythonGeneratedStruct::StaticClass(), *FString::Printf(TEXT("%s_NEWINST"), *StructName)).ToString();
		NewStruct = NewObject<UPythonGeneratedStruct>(StructOuter, *NewStructName, RF_Public | RF_Transient);
		NewStruct->SetMetaData(TEXT("DisplayName"), *PyUtil::GetGeneratedTypeDisplayName(PyType));
		NewStruct->SetMetaData(TEXT("BlueprintType"), TEXT("true"));
		NewStruct->SetSuperStruct(InSuperStruct);
	}

	~FPythonGeneratedStructBuilder()
	{
		// If NewStruct is still set at this point, if means Finalize wasn't called and we should destroy the partially built struct
		if (NewStruct)
		{
			NewStruct->ClearFlags(RF_Public | RF_Standalone);
			NewStruct = nullptr;

			Py_BEGIN_ALLOW_THREADS
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			Py_END_ALLOW_THREADS
		}
	}

	UPythonGeneratedStruct* Finalize(FPyObjectPtr InPyPostInitFunction)
	{
		// Set the post-init function
		NewStruct->PyPostInitFunction = InPyPostInitFunction;
		if (!NewStruct->PyPostInitFunction)
		{
			return nullptr;
		}

		// Replace the definitions with real descriptors
		if (!RegisterDescriptors())
		{
			return nullptr;
		}

		// Let Python know that we've changed its type
		PyType_Modified(PyType);

		// Build a complete list of init params for this struct
		TArray<PyGenUtil::FGeneratedWrappedMethodParameter> StructInitParams;
		if (const FPyWrapperStructMetaData* SuperMetaData = FPyWrapperStructMetaData::GetMetaData(PyType->tp_base))
		{
			StructInitParams = SuperMetaData->InitParams;
		}
		for (const TSharedPtr<PyGenUtil::FPropertyDef>& PropDef : NewStruct->PropertyDefs)
		{
			if (!PropDef->GeneratedWrappedGetSet.Prop.DeprecationMessage.IsSet())
			{
				PyGenUtil::FGeneratedWrappedMethodParameter& StructInitParam = StructInitParams.AddDefaulted_GetRef();
				StructInitParam.ParamName = PropDef->GeneratedWrappedGetSet.GetSetName;
				StructInitParam.ParamProp = PropDef->GeneratedWrappedGetSet.Prop.Prop;
				StructInitParam.ParamDefaultValue = FString();
			}
		}

		// We can no longer fail, so prepare the old struct for removal and set the correct name on the new struct
		if (OldStruct)
		{
			PrepareOldStructForReinstancing();
		}

		NewStruct->StructRedirector = CreatePythonTypeLegacyRedirector(PyUtil::GetCleanTypename(PyType), FTopLevelAssetPath(NewStruct->GetOuter()->GetFName(), *StructName));
		if (NewStruct->StructRedirector)
		{
			NewStruct->StructRedirector->DestinationObject = NewStruct;
		}
		NewStruct->Rename(*StructName, nullptr, REN_DontCreateRedirectors);

		// Finalize the struct
		NewStruct->Bind();
		NewStruct->StaticLink(true);

		// Add the object meta-data to the type
		NewStruct->PyMetaData.Struct = NewStruct;
		NewStruct->PyMetaData.InitParams = MoveTemp(StructInitParams);
		FPyWrapperStructMetaData::SetMetaData(PyType, &NewStruct->PyMetaData);

		// Map the Unreal struct to the Python type
		NewStruct->PyType = FPyTypeObjectPtr::NewReference(PyType);
		FPyWrapperTypeRegistry::Get().RegisterWrappedStructType(NewStruct, PyType, false);

		// Re-instance the old struct
		if (OldStruct)
		{
			FPyWrapperTypeReinstancer::Get().AddPendingStruct(OldStruct, NewStruct);
		}

		// Null the NewStruct pointer so the destructor doesn't kill it
		UPythonGeneratedStruct* FinalizedStruct = NewStruct;
		NewStruct = nullptr;
		return FinalizedStruct;
	}

	bool CreatePropertyFromDefinition(const FString& InFieldName, FPyFPropertyDef* InPyPropDef)
	{
		UScriptStruct* SuperStruct = Cast<UScriptStruct>(NewStruct->GetSuperStruct());

		// Resolve the property name to match any previously exported properties from the parent type
		const FName PropName = FPyWrapperStructMetaData::ResolvePropertyName(PyType->tp_base, *InFieldName);
		if (SuperStruct && SuperStruct->FindPropertyByName(PropName))
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Property '%s' (%s) cannot override a property from the base type"), *InFieldName, *PyUtil::GetFriendlyTypename(InPyPropDef->PropType)));
			return false;
		}

		// Structs cannot support getter/setter functions (or any functions)
		if (!InPyPropDef->GetterFuncName.IsEmpty() || !InPyPropDef->SetterFuncName.IsEmpty())
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Struct property '%s' (%s) has a getter or setter, which is not supported on structs"), *InFieldName, *PyUtil::GetFriendlyTypename(InPyPropDef->PropType)));
			return false;
		}

		// Create the property from its definition
		FProperty* Prop = PyUtil::CreateProperty(InPyPropDef->PropType, 1, NewStruct, PropName);
		if (!Prop)
		{
			PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create property for '%s' (%s)"), *InFieldName, *PyUtil::GetFriendlyTypename(InPyPropDef->PropType)));
			return false;
		}
		Prop->PropertyFlags |= (CPF_Edit | CPF_BlueprintVisible);
		FPyFPropertyDef::ApplyMetaData(InPyPropDef, Prop);
		NewStruct->AddCppProperty(Prop);

		// Build the definition data for the new property accessor
		PyGenUtil::FPropertyDef& PropDef = *NewStruct->PropertyDefs.Add_GetRef(MakeShared<PyGenUtil::FPropertyDef>());
		PropDef.GeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*InFieldName);
		PropDef.GeneratedWrappedGetSet.GetSetDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("type: %s\n%s"), *PyGenUtil::GetPropertyPythonType(Prop), *PyGenUtil::GetFieldTooltip(Prop)));
		PropDef.GeneratedWrappedGetSet.Prop.SetProperty(Prop);
		PropDef.GeneratedWrappedGetSet.GetCallback = (getter)&FPyWrapperStruct::Getter_Impl;
		PropDef.GeneratedWrappedGetSet.SetCallback = (setter)&FPyWrapperStruct::Setter_Impl;
		PropDef.GeneratedWrappedGetSet.ToPython(PropDef.PyGetSet);

		return true;
	}

private:
	bool RegisterDescriptors()
	{
		for (const TSharedPtr<PyGenUtil::FPropertyDef>& PropDef : NewStruct->PropertyDefs)
		{
			FPyObjectPtr GetSetDesc = FPyObjectPtr::StealReference(PyDescr_NewGetSet(PyType, &PropDef->PyGetSet));
			if (!GetSetDesc)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to create descriptor for '%s'"), UTF8_TO_TCHAR(PropDef->PyGetSet.name)));
				return false;
			}
			if (PyDict_SetItemString(PyType->tp_dict, PropDef->PyGetSet.name, GetSetDesc) != 0)
			{
				PyUtil::SetPythonError(PyExc_Exception, PyType, *FString::Printf(TEXT("Failed to assign descriptor for '%s'"), UTF8_TO_TCHAR(PropDef->PyGetSet.name)));
				return false;
			}
		}

		return true;
	}

	void PrepareOldStructForReinstancing()
	{
		check(OldStruct);

		const FString OldStructName = MakeUniqueObjectName(OldStruct->GetOuter(), OldStruct->GetClass(), *FString::Printf(TEXT("%s_REINST"), *OldStruct->GetName())).ToString();
		OldStruct->SetFlags(RF_NewerVersionExists);
		OldStruct->ClearFlags(RF_Public | RF_Standalone);
		OldStruct->Rename(*OldStructName, nullptr, REN_DontCreateRedirectors);
		OldStruct->UnregisterGeneratedType();
	}

	FString StructName;
	PyTypeObject* PyType;
	UPythonGeneratedStruct* OldStruct;
	UPythonGeneratedStruct* NewStruct;
};

void UPythonGeneratedStruct::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UPythonGeneratedStruct* This = CastChecked<UPythonGeneratedStruct>(InThis);
	Collector.AddReferencedObject(This->StructRedirector, This);
}

void UPythonGeneratedStruct::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	if (PyType)
	{
		FPyWrapperTypeRegistry::Get().UnregisterWrappedStructType(FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(OldOuter->GetFName(), OldName)), PyType, false);
		FPyWrapperTypeRegistry::Get().RegisterWrappedStructType(this, PyType, false);
	}
}

void UPythonGeneratedStruct::InitializeStruct(void* Dest, int32 ArrayDim) const
{
	Super::InitializeStruct(Dest, ArrayDim);

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;

		if (PyPostInitFunction)
		{
			TGuardValue<bool> IsRunningUserScriptGuard(UE::Python::Private::GIsRunningUserScript, true);

			const int32 Stride = GetStructureSize();
			for (int32 ArrIndex = 0; ArrIndex < ArrayDim; ++ArrIndex)
			{
				void* StructInstance = static_cast<uint8*>(Dest) + (ArrIndex * Stride);
				FPyObjectPtr PySelf = FPyObjectPtr::StealReference((PyObject*)FPyWrapperStructFactory::Get().CreateInstance((UPythonGeneratedStruct*)this, StructInstance, FPyWrapperOwnerContext(Py_None), EPyConversionMethod::Reference));
				if (PySelf && ensureAlwaysMsgf(PySelf->ob_type == PyType, TEXT("Struct instance (struct: %s) had an unexpected PyType when calling InitializeStruct! Self is '%s' but we expected '%s'"), *GetPathName(), *PyUtil::GetFriendlyTypename(PySelf), *PyUtil::GetFriendlyTypename(PyType)))
				{
					FPyObjectPtr PyArgs = FPyObjectPtr::StealReference(PyTuple_New(1));
					PyTuple_SetItem(PyArgs, 0, PySelf.Release()); // SetItem steals the reference

					FPyObjectPtr Result = FPyObjectPtr::StealReference(PyObject_CallObject((PyObject*)PyPostInitFunction.GetPtr(), PyArgs));
					if (!Result)
					{
						PyUtil::ReThrowPythonError();
					}
				}
			}
		}
	}
}

void UPythonGeneratedStruct::BeginDestroy()
{
	ReleasePythonResources();
	Super::BeginDestroy();
}

void UPythonGeneratedStruct::ReleasePythonResources()
{
	// This may be called after Python has already shut down
	if (Py_IsInitialized())
	{
		FPyScopedGIL GIL;
		UnregisterGeneratedType();
		FPyWrapperStructMetaData::SetMetaData(PyType.Get(), nullptr);
		PyType.Reset();
		PyPostInitFunction.Reset();
	}
	else
	{
		// Release ownership if Python has been shut down to avoid attempting to delete the objects (which are already dead)
		PyType.Release();
		PyPostInitFunction.Release();
	}

	PropertyDefs.Reset();
	PyMetaData = FPyWrapperStructMetaData();
}

void UPythonGeneratedStruct::UnregisterGeneratedType()
{
	if (PyType)
	{
		FPyWrapperTypeRegistry::Get().UnregisterWrappedStructType(this, PyType, false);
	}
}

UPythonGeneratedStruct* UPythonGeneratedStruct::GenerateStruct(PyTypeObject* InPyType)
{
	// Get the correct super struct from the parent type in Python
	UScriptStruct* SuperStruct = nullptr;
	if (InPyType->tp_base != &PyWrapperStructType)
	{
		SuperStruct = FPyWrapperStructMetaData::GetStruct(InPyType->tp_base);
		if (!SuperStruct)
		{
			PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("No super struct could be found for this Python type"));
			return nullptr;
		}
	}

	// Builder used to generate the struct
	FPythonGeneratedStructBuilder PythonStructBuilder(SuperStruct, InPyType);

	// Add the fields to this struct
	{
		PyObject* FieldKey = nullptr;
		PyObject* FieldValue = nullptr;
		Py_ssize_t FieldIndex = 0;
		while (PyDict_Next(InPyType->tp_dict, &FieldIndex, &FieldKey, &FieldValue))
		{
			const FString FieldName = PyUtil::PyObjectToUEString(FieldKey);

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyUValueDefType) == 1)
			{
				// Values are not supported on structs
				PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("Structs do not support values"));
				return nullptr;
			}

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyFPropertyDefType) == 1)
			{
				FPyFPropertyDef* PyPropDef = (FPyFPropertyDef*)FieldValue;
				if (!PythonStructBuilder.CreatePropertyFromDefinition(FieldName, PyPropDef))
				{
					return nullptr;
				}
			}

			if (PyObject_IsInstance(FieldValue, (PyObject*)&PyUFunctionDefType) == 1)
			{
				// Functions are not supported on structs
				PyUtil::SetPythonError(PyExc_Exception, InPyType, TEXT("Structs do not support methods"));
				return nullptr;
			}
		}
	}

	// Finalize the struct with its post-init function
	return PythonStructBuilder.Finalize(FPyObjectPtr::StealReference(PyGenUtil::GetPostInitFunc(InPyType)));
}

#endif	// WITH_PYTHON

