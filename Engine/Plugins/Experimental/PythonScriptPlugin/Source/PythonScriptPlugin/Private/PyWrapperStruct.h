// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBase.h"
#include "PyWrapperOwnerContext.h"
#include "PyUtil.h"
#include "PyConversion.h"
#include "UObject/Class.h"
#include "UObject/PropertyAccessUtil.h"
#include "Templates/TypeCompatibleBytes.h"
#include "PyWrapperStruct.generated.h"

class UObjectRedirector;

#if WITH_PYTHON

/** Python type for FPyWrapperStruct */
extern PyTypeObject PyWrapperStructType;

/** Initialize the PyWrapperStruct types and add them to the given Python module */
void InitializePyWrapperStruct(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed struct instances */
struct FPyWrapperStruct : public FPyWrapperBase
{
	/** The owner of the wrapped struct instance (if any) */
	FPyWrapperOwnerContext OwnerContext;

	/** Struct type of this instance */
	TObjectPtr<UScriptStruct> ScriptStruct;

	/** Wrapped struct instance */
	void* StructInstance;

	/** New this wrapper instance (called via tp_new for Python, or directly in C++) */
	static FPyWrapperStruct* New(PyTypeObject* InType);

	/** Free this wrapper instance (called via tp_dealloc for Python) */
	static void Free(FPyWrapperStruct* InSelf);

	/** Initialize this wrapper instance (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperStruct* InSelf);

	/** Initialize this wrapper instance to the given value (called via tp_init for Python, or directly in C++) */
	static int Init(FPyWrapperStruct* InSelf, const FPyWrapperOwnerContext& InOwnerContext, UScriptStruct* InStruct, void* InValue, const EPyConversionMethod InConversionMethod);

	/** Deinitialize this wrapper instance (called via Init and Free to restore the instance to its New state) */
	static void Deinit(FPyWrapperStruct* InSelf);

	/** Called to validate the internal state of this wrapper instance prior to operating on it (should be called by all functions that expect to operate on an initialized type; will set an error state on failure) */
	static bool ValidateInternalState(FPyWrapperStruct* InSelf);

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperStruct* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperStruct* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** "Make" this struct from the given arguments, either using a custom make function or by setting the named property values on this instance (called via generated code) */
	static int MakeStruct(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds);

	/** "Break" this struct into a tuple, either using a custom break function or by getting each property value on this instance (called via generated code) */
	static PyObject* BreakStruct(FPyWrapperStruct* InSelf);

	/** Get a property value from this instance (called via generated code) */
	static PyObject* GetPropertyValue(FPyWrapperStruct* InSelf, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName);

	/** Set a property value on this instance (called via generated code) */
	static int SetPropertyValue(FPyWrapperStruct* InSelf, PyObject* InValue, const PyGenUtil::FGeneratedWrappedProperty& InPropDef, const char* InPythonAttrName, const EPropertyAccessChangeNotifyMode InNotifyMode = EPropertyAccessChangeNotifyMode::Never, const uint64 InReadOnlyFlags = PropertyAccessUtil::RuntimeReadOnlyFlags);

	/** Call a make function on this instance (MakeStruct internal use only) */
	static int CallMakeFunction_Impl(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef);

	/** Call a break function on this instance (BreakStruct internal use only) */
	static PyObject* CallBreakFunction_Impl(FPyWrapperStruct* InSelf, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef);

	/** Call a dynamic function on this instance (CallFunction internal use only) */
	static PyObject* CallDynamicFunction_Impl(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds, const PyGenUtil::FGeneratedWrappedFunction& InFuncDef, const PyGenUtil::FGeneratedWrappedMethodParameter& InSelfParam, const PyGenUtil::FGeneratedWrappedMethodParameter& InSelfReturn, const char* InPythonFuncName);

	/** Implementation of the "call" logic for a dynamic Python method with no arguments (internal Python bindings use only) */
	static PyObject* CallDynamicMethodNoArgs_Impl(FPyWrapperStruct* InSelf, void* InClosure);

	/** Implementation of the "call" logic for a dynamic Python method with arguments (internal Python bindings use only) */
	static PyObject* CallDynamicMethodWithArgs_Impl(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds, void* InClosure);

	/** Implementation of the "call" logic for a Python operator function (internal CallOperator use only) */
	static PyObject* CallOperatorFunction_Impl(FPyWrapperStruct* InSelf, PyObject* InRHS, const PyGenUtil::FGeneratedWrappedOperatorFunction& InOpFunc, const TOptional<EPyConversionResultState> InRequiredConversionResult = TOptional<EPyConversionResultState>(), FPyConversionResult* OutRHSConversionResult = nullptr);

	/** Implementation of the "call" logic for a Python operator function (internal Python bindings use only) */
	static PyObject* CallOperator_Impl(FPyWrapperStruct* InSelf, PyObject* InRHS, const PyGenUtil::EGeneratedWrappedOperatorType InOpType);

	/** Implementation of the "getter" logic for a Python descriptor reading from an struct property (internal Python bindings use only) */
	static PyObject* Getter_Impl(FPyWrapperStruct* InSelf, void* InClosure);

	/** Implementation of the "setter" logic for a Python descriptor writing to an struct property (internal Python bindings use only) */
	static int Setter_Impl(FPyWrapperStruct* InSelf, PyObject* InValue, void* InClosure);

	/** Get a pointer to the typed struct instance this wrapper represents */
	template <typename StructType>
	static StructType* GetTypedStructPtr(FPyWrapperStruct* InSelf)
	{
		return (StructType*)InSelf->StructInstance;
	}

	/** Get a reference to the typed struct instance this wrapper represents */
	template <typename StructType>
	static StructType& GetTypedStruct(FPyWrapperStruct* InSelf)
	{
		return *GetTypedStructPtr<StructType>(InSelf);
	}
};

/** Specialized version of FPyWrapperStruct that can store its struct payload inline (requires a known type) */
template <typename InlineType>
struct TPyWrapperInlineStruct : public FPyWrapperStruct
{
	typedef InlineType WrappedType;

	/** Inline struct instance (do not use directly, StructInstance will be set to point to this when appropriate via FPyWrapperStructAllocationPolicy_Inline) */
	TTypeCompatibleBytes<WrappedType> InlineStructInstance;

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static TPyWrapperInlineStruct* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr)
	{
		return (TPyWrapperInlineStruct*)FPyWrapperStruct::CastPyObject(InPyObject, OutCastResult);
	}

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static TPyWrapperInlineStruct* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr)
	{
		return (TPyWrapperInlineStruct*)FPyWrapperStruct::CastPyObject(InPyObject, InType, OutCastResult);
	}

	/** Get a pointer to the typed struct instance this wrapper represents */
	static WrappedType* GetTypedStructPtr(TPyWrapperInlineStruct* InSelf)
	{
		return FPyWrapperStruct::GetTypedStructPtr<WrappedType>(InSelf);
	}

	/** Get a reference to the typed struct instance this wrapper represents */
	static WrappedType& GetTypedStruct(TPyWrapperInlineStruct* InSelf)
	{
		return FPyWrapperStruct::GetTypedStruct<WrappedType>(InSelf);
	}

	/** Getter function for intrinsic field access (for use with PyGetSetDef) */
	template <typename FieldType, FieldType WrappedType::*FieldPtr>
	static PyObject* IntrinsicFieldGetter(TPyWrapperInlineStruct* InSelf, void* InClosure)
	{
		return PyConversion::Pythonize(GetTypedStruct(InSelf).*FieldPtr);
	}

	/** Setter function for intrinsic field access (for use with PyGetSetDef) */
	template <typename FieldType, FieldType WrappedType::*FieldPtr>
	static int IntrinsicFieldSetter(TPyWrapperInlineStruct* InSelf, PyObject* InValue, void* InClosure)
	{
		if (!InValue)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Cannot delete attribute from type"));
			return -1;
		}

		if (!PyConversion::Nativize(InValue, GetTypedStruct(InSelf).*FieldPtr))
		{
			return -1;
		}

		return 0;
	}

	/** Getter function for struct field access (for use with PyGetSetDef) */
	template <typename FieldType, FieldType WrappedType::*FieldPtr>
	static PyObject* StructFieldGetter(TPyWrapperInlineStruct* InSelf, void* InClosure)
	{
		return PyConversion::PythonizeStructInstance(GetTypedStruct(InSelf).*FieldPtr);
	}

	/** Setter function for struct field access (for use with PyGetSetDef) */
	template <typename FieldType, FieldType WrappedType::*FieldPtr>
	static int StructFieldSetter(TPyWrapperInlineStruct* InSelf, PyObject* InValue, void* InClosure)
	{
		if (!InValue)
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, TEXT("Cannot delete attribute from type"));
			return -1;
		}

		if (!PyConversion::NativizeStructInstance(InValue, GetTypedStruct(InSelf).*FieldPtr))
		{
			return -1;
		}

		return 0;
	}

	/** Call a function with no parameters and no return value (for use with PyMethodDef) */
	template <void(WrappedType::*FuncPtr)()>
	static PyObject* CallFunc_NoParams_NoReturn(TPyWrapperInlineStruct* InSelf)
	{
		(GetTypedStruct(InSelf).*FuncPtr)();
		Py_RETURN_NONE;
	}

	/** Call a const function with no parameters and no return value (for use with PyMethodDef) */
	template <void(WrappedType::*FuncPtr)() const>
	static PyObject* CallConstFunc_NoParams_NoReturn(TPyWrapperInlineStruct* InSelf)
	{
		(GetTypedStruct(InSelf).*FuncPtr)();
		Py_RETURN_NONE;
	}

	/** Call a function with no parameters and an intrinsic return value (for use with PyMethodDef) */
	template <typename ReturnType, ReturnType(WrappedType::*FuncPtr)()>
	static PyObject* CallFunc_NoParams_IntrinsicReturn(TPyWrapperInlineStruct* InSelf)
	{
		return PyConversion::Pythonize((GetTypedStruct(InSelf).*FuncPtr)());
	}

	/** Call a const function with no parameters and an intrinsic return value (for use with PyMethodDef) */
	template <typename ReturnType, ReturnType(WrappedType::*FuncPtr)() const>
	static PyObject* CallConstFunc_NoParams_IntrinsicReturn(TPyWrapperInlineStruct* InSelf)
	{
		return PyConversion::Pythonize((GetTypedStruct(InSelf).*FuncPtr)());
	}

	/** Call a function with no parameters and a struct return value (for use with PyMethodDef) */
	template <typename ReturnType, ReturnType(WrappedType::*FuncPtr)()>
	static PyObject* CallFunc_NoParams_StructReturn(TPyWrapperInlineStruct* InSelf)
	{
		return PyConversion::PythonizeStructInstance((GetTypedStruct(InSelf).*FuncPtr)());
	}

	/** Call a const function with no parameters and a struct return value (for use with PyMethodDef) */
	template <typename ReturnType, ReturnType(WrappedType::*FuncPtr)() const>
	static PyObject* CallConstFunc_NoParams_StructReturn(TPyWrapperInlineStruct* InSelf)
	{
		return PyConversion::PythonizeStructInstance((GetTypedStruct(InSelf).*FuncPtr)());
	}
};

/** Struct allocation policy interface */
class IPyWrapperStructAllocationPolicy
{
public:
	/** Allocate memory to hold an instance of the given struct and return the result */
	virtual void* AllocateStruct(const FPyWrapperStruct* InSelf, UScriptStruct* InStruct) const = 0;

	/** Free memory previously allocated with AllocateStruct */
	virtual void FreeStruct(const FPyWrapperStruct* InSelf, void* InAlloc) const = 0;
};

/** Inline struct factory interface */
class IPyWrapperInlineStructFactory
{
public:
	/** Get the name of the Unreal struct this factory is for */
	virtual FTopLevelAssetPath GetStructName() const = 0;

	/** Get the size of the Python object that should be constructed (in bytes) */
	virtual int32 GetPythonObjectSizeBytes() const = 0;

	/** Get the allocation policy of the Python object */
	virtual const IPyWrapperStructAllocationPolicy* GetPythonObjectAllocationPolicy() const = 0;
};

/** Concrete implementation of an inline struct factory for the given type */
template <typename InlineType>
class TPyWrapperInlineStructFactory : public IPyWrapperInlineStructFactory
{
private:
	class FPyWrapperStructAllocationPolicy_Inline : public IPyWrapperStructAllocationPolicy
	{
	public:
		virtual void* AllocateStruct(const FPyWrapperStruct* InSelf, UScriptStruct* InStruct) const override
		{
			return (void*)&static_cast<const TPyWrapperInlineStruct<InlineType>*>(InSelf)->InlineStructInstance;
		}

		virtual void FreeStruct(const FPyWrapperStruct* InSelf, void* InAlloc) const override
		{
		}
	};

public:
	virtual FTopLevelAssetPath GetStructName() const override
	{
		return TBaseStructure<InlineType>::Get()->GetStructPathName();
	}

	virtual int32 GetPythonObjectSizeBytes() const override
	{
		return sizeof(TPyWrapperInlineStruct<InlineType>);
	}

	virtual const IPyWrapperStructAllocationPolicy* GetPythonObjectAllocationPolicy() const override
	{
		static const FPyWrapperStructAllocationPolicy_Inline InlineAllocPolicy = FPyWrapperStructAllocationPolicy_Inline();
		return &InlineAllocPolicy;
	}
};

/** Meta-data for all Unreal exposed struct types */
struct FPyWrapperStructMetaData : public FPyWrapperBaseMetaData
{
	PY_METADATA_METHODS(FPyWrapperStructMetaData, FGuid(0x03C9EA75, 0x2C86448B, 0xB53D1453, 0x94AA6BAE))

	FPyWrapperStructMetaData();

	/** Get the UScriptStruct from the given type */
	static UScriptStruct* GetStruct(PyTypeObject* PyType);

	/** Get the UScriptStruct from the type of the given instance */
	static UScriptStruct* GetStruct(FPyWrapperStruct* Instance);

	/** Resolve the original property name of a Python property from the given type */
	static FName ResolvePropertyName(PyTypeObject* PyType, const FName InPythonPropertyName);

	/** Resolve the original property name of a Python property of the given instance */
	static FName ResolvePropertyName(FPyWrapperStruct* Instance, const FName InPythonPropertyName);

	/** Check to see if the given Python property is deprecated, and optionally return its deprecation message */
	static bool IsPropertyDeprecated(PyTypeObject* PyType, const FName InPythonPropertyName, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the given Python property is deprecated, and optionally return its deprecation message */
	static bool IsPropertyDeprecated(FPyWrapperStruct* Instance, const FName InPythonPropertyName, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the struct is deprecated, and optionally return its deprecation message */
	static bool IsStructDeprecated(PyTypeObject* PyType, FString* OutDeprecationMessage = nullptr);

	/** Check to see if the struct is deprecated, and optionally return its deprecation message */
	static bool IsStructDeprecated(FPyWrapperStruct* Instance, FString* OutDeprecationMessage = nullptr);

	/** Add object references from this type meta-data to the given collector */
	virtual void AddTypeReferencedObjects(FReferenceCollector& Collector) override;

	/** Add object references from the given Python object to the given collector */
	virtual void AddInstanceReferencedObjects(FPyWrapperBase* Instance, FReferenceCollector& Collector) override;

	/** Get the reflection meta data type object associated with this wrapper type if there is one or nullptr if not. */
	virtual const UField* GetMetaType() const override
	{
		return Struct;
	}

	/** Unreal struct */
	TObjectPtr<UScriptStruct> Struct;

	/** Map of properties that were exposed to Python mapped to their original name */
	TMap<FName, FName> PythonProperties;

	/** Map of properties that were exposed to Python mapped to their deprecation message (if deprecated) */
	TMap<FName, FString> PythonDeprecatedProperties;

	/** Make function associated with this struct */
	PyGenUtil::FGeneratedWrappedFunction MakeFunc;

	/** Break function associated with this struct */
	PyGenUtil::FGeneratedWrappedFunction BreakFunc;

	/** Array of properties that were exposed to Python and can be used during init */
	TArray<PyGenUtil::FGeneratedWrappedMethodParameter> InitParams;

	/** The operator stacks for this struct type */
	PyGenUtil::FGeneratedWrappedOperatorStack OpStacks[(int32)PyGenUtil::EGeneratedWrappedOperatorType::Num];

	/** Set if this struct is deprecated and using it should emit a deprecation warning */
	TOptional<FString> DeprecationMessage;
};

typedef TPyPtr<FPyWrapperStruct> FPyWrapperStructPtr;

#endif	// WITH_PYTHON

/** An Unreal struct that was generated from a Python type */
UCLASS()
class UPythonGeneratedStruct final : public UScriptStruct, public IPythonResourceOwner
{
	GENERATED_BODY()

#if WITH_PYTHON

public:
	//~ UObject interface
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void BeginDestroy() override;
	virtual bool IsAsset() const override
	{
		return false;
	}

	//~ UStruct interface
	virtual void InitializeStruct(void* Dest, int32 ArrayDim = 1) const override;

	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override;

	/** Unregister this type from FPyWrapperTypeRegistry */
	void UnregisterGeneratedType();

	/** Generate an Unreal struct from the given Python type */
	static UPythonGeneratedStruct* GenerateStruct(PyTypeObject* InPyType);

private:
	/** Python type this struct was generated from */
	FPyTypeObjectPtr PyType;

	/** PostInit function for this struct */
	FPyObjectPtr PyPostInitFunction;

	/** Array of properties generated for this struct */
	TArray<TSharedPtr<PyGenUtil::FPropertyDef>> PropertyDefs;

	/** Meta-data for this generated struct that is applied to the Python type */
	FPyWrapperStructMetaData PyMetaData;

	TObjectPtr<UObjectRedirector> StructRedirector;

	friend class FPythonGeneratedStructBuilder;

#else	// WITH_PYTHON

public:
	//~ IPythonResourceOwner interface
	virtual void ReleasePythonResources() override {}

#endif	// WITH_PYTHON
};
