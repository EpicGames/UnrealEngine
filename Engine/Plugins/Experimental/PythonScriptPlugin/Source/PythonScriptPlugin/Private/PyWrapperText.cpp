// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperText.h"
#include "PyWrapperTypeRegistry.h"
#include "PyUtil.h"
#include "PyConversion.h"
#include "Internationalization/TextFormatter.h"

#if WITH_PYTHON

namespace PyTextUtil
{

bool ExtractFormatArgumentKey(FPyWrapperText* InSelf, PyObject* InObj, FFormatArgumentData& OutFormatArg)
{
	if (PyConversion::Nativize(InObj, OutFormatArg.ArgumentName, PyConversion::ESetErrorState::No))
	{
		return true;
	}

	int32 ArgumentIndex = 0;
	if (PyConversion::Nativize(InObj, ArgumentIndex, PyConversion::ESetErrorState::No))
	{
		OutFormatArg.ArgumentName = FString::FromInt(ArgumentIndex);
		return true;
	}

	PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert key (%s) to a valid key type (string or int)"), *PyUtil::GetFriendlyTypename(InObj)));
	return false;
}

bool ExtractFormatArgumentValue(FPyWrapperText* InSelf, PyObject* InObj, FFormatArgumentData& OutFormatArg)
{
	if (PyConversion::Nativize(InObj, OutFormatArg.ArgumentValue, PyConversion::ESetErrorState::No))
	{
		OutFormatArg.ArgumentValueType = EFormatArgumentType::Text;
		return true;
	}

	// Don't use PyConversion for numeric types as they would allow coercion from float<->int
	if (PyLong_Check(InObj))
	{
		OutFormatArg.ArgumentValueType = EFormatArgumentType::Int;
		OutFormatArg.ArgumentValueInt = PyLong_AsLongLong(InObj);
		return true;
	}

	if (PyFloat_Check(InObj))
	{
		OutFormatArg.ArgumentValueType = EFormatArgumentType::Float;
		OutFormatArg.ArgumentValueFloat = PyFloat_AsDouble(InObj);
		return true;
	}

	PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert value (%s) to a valid value type (text, int, or float)"), *PyUtil::GetFriendlyTypename(InObj)));
	return false;
}

bool ExtractFormatArguments(FPyWrapperText* InSelf, PyObject* InObj, const int32 InArgIndex, TArray<FFormatArgumentData>& InOutFormatArgs)
{
	// Is this some kind of container, or a single value?
	const bool bIsStringType = static_cast<bool>(PyUnicode_Check(InObj));
	if (!bIsStringType && PyUtil::HasLength(InObj))
	{
		const Py_ssize_t SequenceLen = PyObject_Length(InObj);
		check(SequenceLen != -1);

		FPyObjectPtr PyObjIter = FPyObjectPtr::StealReference(PyObject_GetIter(InObj));
		if (PyObjIter)
		{
			if (PyUtil::IsMappingType(InObj))
			{
				// Conversion from a mapping type
				for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
				{
					FPyObjectPtr KeyItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
					if (!KeyItem)
					{
						return false;
					}

					FPyObjectPtr ValueItem = FPyObjectPtr::StealReference(PyObject_GetItem(InObj, KeyItem));
					if (!ValueItem)
					{
						return false;
					}

					FFormatArgumentData& FormatArg = InOutFormatArgs.AddDefaulted_GetRef();
					if (!ExtractFormatArgumentKey(InSelf, KeyItem, FormatArg))
					{
						PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert format argument %d (%s) at index %zd"), InArgIndex, *PyUtil::GetFriendlyTypename(InObj), SequenceIndex));
						return false;
					}
					if (!ExtractFormatArgumentValue(InSelf, ValueItem, FormatArg))
					{
						PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert format argument %d (%s) with key '%s' at index %zd"), InArgIndex, *PyUtil::GetFriendlyTypename(InObj), *FormatArg.ArgumentName, SequenceIndex));
						return false;
					}
				}
			}
			else
			{
				// Conversion from a sequence
				for (Py_ssize_t SequenceIndex = 0; SequenceIndex < SequenceLen; ++SequenceIndex)
				{
					FPyObjectPtr ValueItem = FPyObjectPtr::StealReference(PyIter_Next(PyObjIter));
					if (!ValueItem)
					{
						return false;
					}

					FFormatArgumentData& FormatArg = InOutFormatArgs.AddDefaulted_GetRef();
					FormatArg.ArgumentName = FString::FromInt(InArgIndex);
					if (!ExtractFormatArgumentValue(InSelf, ValueItem, FormatArg))
					{
						PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert format argument %d (%s) at index %zd"), InArgIndex, *PyUtil::GetFriendlyTypename(InObj), SequenceIndex));
						return false;
					}
				}
			}
		}
	}
	else
	{
		FFormatArgumentData& FormatArg = InOutFormatArgs.AddDefaulted_GetRef();
		FormatArg.ArgumentName = FString::FromInt(InArgIndex);
		if (!ExtractFormatArgumentValue(InSelf, InObj, FormatArg))
		{
			PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Cannot convert format argument %d (%s)"), InArgIndex, *PyUtil::GetFriendlyTypename(InObj)));
			return false;
		}
	}

	return true;
}

} // PyTextUtil

void InitializePyWrapperText(PyGenUtil::FNativePythonModule& ModuleInfo)
{
	if (PyType_Ready(&PyWrapperTextType) == 0)
	{
		ModuleInfo.AddType(&PyWrapperTextType);
	}
}

void FPyWrapperText::InitValue(FPyWrapperText* InSelf, const FText InValue)
{
	Super::InitValue(InSelf, InValue);
	FPyWrapperTextFactory::Get().MapInstance(InSelf->Value, InSelf);
}

void FPyWrapperText::DeinitValue(FPyWrapperText* InSelf)
{
	FPyWrapperTextFactory::Get().UnmapInstance(InSelf->Value, Py_TYPE(InSelf));
	Super::DeinitValue(InSelf);
}

FPyWrapperText* FPyWrapperText::CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperTextType) == 1)
	{
		SetOptionalPyConversionResult(FPyConversionResult::Success(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperText*)InPyObject;
	}

	return nullptr;
}

FPyWrapperText* FPyWrapperText::CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult)
{
	SetOptionalPyConversionResult(FPyConversionResult::Failure(), OutCastResult);

	if (PyObject_IsInstance(InPyObject, (PyObject*)InType) == 1 && (InType == &PyWrapperTextType || PyObject_IsInstance(InPyObject, (PyObject*)&PyWrapperTextType) == 1))
	{
		SetOptionalPyConversionResult(Py_TYPE(InPyObject) == InType ? FPyConversionResult::Success() : FPyConversionResult::SuccessWithCoercion(), OutCastResult);

		Py_INCREF(InPyObject);
		return (FPyWrapperText*)InPyObject;
	}

	{
		FText InitValue;
		if (PyConversion::Nativize(InPyObject, InitValue))
		{
			FPyWrapperTextPtr NewText = FPyWrapperTextPtr::StealReference(FPyWrapperText::New(InType));
			if (NewText)
			{
				if (FPyWrapperText::Init(NewText, InitValue) != 0)
				{
					return nullptr;
				}
			}
			SetOptionalPyConversionResult(FPyConversionResult::SuccessWithCoercion(), OutCastResult);
			return NewText.Release();
		}
	}

	return nullptr;
}

PyTypeObject InitializePyWrapperTextType()
{
	struct FFuncs
	{
		static int Init(FPyWrapperText* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			FText InitValue;

			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "|O:call", &PyObj))
			{
				return -1;
			}
			
			if (PyObj && !PyConversion::Nativize(PyObj, InitValue))
			{
				PyUtil::SetPythonError(PyExc_TypeError, InSelf, *FString::Printf(TEXT("Failed to convert init argument '%s' to 'Text'"), *PyUtil::GetFriendlyTypename(PyObj)));
				return -1;
			}

			return FPyWrapperText::Init(InSelf, InitValue);
		}

		static PyObject* Str(FPyWrapperText* InSelf)
		{
			return PyUnicode_FromString(TCHAR_TO_UTF8(*InSelf->Value.ToString()));
		}

		static PyObject* Repr(FPyWrapperText* InSelf)
		{
			return PyUnicode_FromString(TCHAR_TO_UTF8(*FString::Printf(TEXT("Text(\"%s\")"), *InSelf->Value.ToString())));
		}

		static PyObject* RichCmp(FPyWrapperText* InSelf, PyObject* InOther, int InOp)
		{
			FText Other;
			if (!PyConversion::Nativize(InOther, Other, PyConversion::ESetErrorState::No))
			{
				Py_INCREF(Py_NotImplemented);
				return Py_NotImplemented;
			}

			return PyUtil::PyRichCmp(InSelf->Value.CompareTo(Other), 0, InOp);
		}

		static PyUtil::FPyHashType Hash(FPyWrapperText* InSelf)
		{
			PyUtil::SetPythonError(PyExc_Exception, InSelf, TEXT("Type cannot be hashed"));
			return -1;
		}
	};

	struct FMethods
	{
		static PyObject* Cast(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (PyArg_ParseTuple(InArgs, "O:call", &PyObj))
			{
				PyObject* PyCastResult = (PyObject*)FPyWrapperText::CastPyObject(PyObj, InType);
				if (!PyCastResult)
				{
					PyUtil::SetPythonError(PyExc_TypeError, InType, *FString::Printf(TEXT("Cannot cast type '%s' to '%s'"), *PyUtil::GetFriendlyTypename(PyObj), *PyUtil::GetFriendlyTypename(InType)));
				}
				return PyCastResult;
			}

			return nullptr;
		}

		static PyObject* AsNumber(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:as_number", &PyObj))
			{
				return nullptr;
			}
			check(PyObj);

			FText NumberText;

			if (PyFloat_Check(PyObj))
			{
				double Number = 0.0;
				if (!PyConversion::Nativize(PyObj, Number))
				{
					return nullptr;
				}
				NumberText = FText::AsNumber(Number);
			}
			else
			{
				int64 Number = 0.0;
				if (!PyConversion::Nativize(PyObj, Number))
				{
					return nullptr;
				}
				NumberText = FText::AsNumber(Number);
			}

			return PyConversion::Pythonize(NumberText);
		}

		static PyObject* AsPercent(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyObj = nullptr;
			if (!PyArg_ParseTuple(InArgs, "O:as_percent", &PyObj))
			{
				return nullptr;
			}

			double Percentage = 0.0;
			if (!PyConversion::Nativize(PyObj, Percentage))
			{
				return nullptr;
			}

			const FText PercentageText = FText::AsPercent(Percentage);
			return PyConversion::Pythonize(PercentageText);
		}

		static PyObject* AsCurrency(PyTypeObject* InType, PyObject* InArgs)
		{
			PyObject* PyBaseVal = nullptr;
			PyObject* PyCurrencyCode = nullptr;
			if (!PyArg_ParseTuple(InArgs, "OO:as_currency", &PyBaseVal, &PyCurrencyCode))
			{
				return nullptr;
			}

			int32 BaseVal = 0;
			if (!PyConversion::Nativize(PyBaseVal, BaseVal))
			{
				return nullptr;
			}

			FString CurrencyCode;
			if (!PyConversion::Nativize(PyCurrencyCode, CurrencyCode))
			{
				return nullptr;
			}

			const FText CurrencyText = FText::AsCurrencyBase(BaseVal, CurrencyCode);
			return PyConversion::Pythonize(CurrencyText);
		}

		static PyObject* IsEmpty(FPyWrapperText* InSelf)
		{
			if (InSelf->Value.IsEmpty())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsEmptyOrWhitespace(FPyWrapperText* InSelf)
		{
			if (InSelf->Value.IsEmptyOrWhitespace())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsTransient(FPyWrapperText* InSelf)
		{
			if (InSelf->Value.IsTransient())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsCultureInvariant(FPyWrapperText* InSelf)
		{
			if (InSelf->Value.IsCultureInvariant())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* IsFromStringTable(FPyWrapperText* InSelf)
		{
			if (InSelf->Value.IsFromStringTable())
			{
				Py_RETURN_TRUE;
			}

			Py_RETURN_FALSE;
		}

		static PyObject* ToLower(FPyWrapperText* InSelf)
		{
			const FText LowerText = InSelf->Value.ToLower();
			return PyConversion::Pythonize(LowerText);
		}

		static PyObject* ToUpper(FPyWrapperText* InSelf)
		{
			const FText UpperText = InSelf->Value.ToUpper();
			return PyConversion::Pythonize(UpperText);
		}

		static PyObject* Format(FPyWrapperText* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			TArray<FFormatArgumentData> FormatArgs;

			// Process each sequence argument
			if (InArgs)
			{
				const Py_ssize_t ArgsLen = PyTuple_Size(InArgs);
				for (Py_ssize_t ArgIndex = 0; ArgIndex < ArgsLen; ++ArgIndex)
				{
					PyObject* PyArg = PyTuple_GetItem(InArgs, ArgIndex);
					if (PyArg && !PyTextUtil::ExtractFormatArguments(InSelf, PyArg, ArgIndex, FormatArgs))
					{
						return nullptr;
					}
				}
			}

			// Process each named argument
			if (InKwds && !PyTextUtil::ExtractFormatArguments(InSelf, InKwds, -1, FormatArgs))
			{
				return nullptr;
			}

			const FText FormattedText = FTextFormatter::Format(InSelf->Value, MoveTemp(FormatArgs), false, false);
			return PyConversion::Pythonize(FormattedText);
		}
	};

	// NOTE: Union/Mapping are defines in the Python typing module.
	static PyMethodDef PyMethods[] = {
		{ "cast", PyCFunctionCast(&FMethods::Cast), METH_VARARGS | METH_CLASS, "cast(cls, object: object) -> Text -- cast the given object to this Unreal text type" },
		{ "as_number", PyCFunctionCast(&FMethods::AsNumber), METH_VARARGS | METH_CLASS, "as_number(cls, num: float) -> Text -- convert the given number to a culture correct Unreal text representation" },
		{ "as_percent", PyCFunctionCast(&FMethods::AsPercent), METH_VARARGS | METH_CLASS, "as_percent(cls, num: float) -> Text -- convert the given number to a culture correct Unreal text percentgage representation" },
		{ "as_currency", PyCFunctionCast(&FMethods::AsCurrency), METH_VARARGS | METH_CLASS, "as_currency(cls, val: int, code: str) -> Text -- convert the given number (specified in the smallest unit for the given currency i.e. 650 for $6.50) to a culture correct Unreal text currency representation" },
		// todo: datetime?
		{ "is_empty", PyCFunctionCast(&FMethods::IsEmpty), METH_NOARGS, "is_empty(self) -> bool -- is this Unreal text empty?" },
		{ "is_empty_or_whitespace", PyCFunctionCast(&FMethods::IsEmptyOrWhitespace), METH_NOARGS, "is_empty_or_whitespace(self) -> bool -- is this Unreal text empty or only whitespace?" },
		{ "is_transient", PyCFunctionCast(&FMethods::IsTransient), METH_NOARGS, "is_transient(self) -> bool -- is this Unreal text transient?" },
		{ "is_culture_invariant", PyCFunctionCast(&FMethods::IsEmptyOrWhitespace), METH_NOARGS, "is_culture_invariant(self) -> bool -- is this Unreal text culture invariant?" },
		{ "is_from_string_table", PyCFunctionCast(&FMethods::IsFromStringTable), METH_NOARGS, "is_from_string_table(self) -> bool -- is this Unreal text referencing a string table entry?" },
		{ "to_lower", PyCFunctionCast(&FMethods::ToLower), METH_NOARGS, "to_lower(self) -> Text -- convert this Unreal text to lowercase in a culture correct way" },
		{ "to_upper", PyCFunctionCast(&FMethods::ToUpper), METH_NOARGS, "to_upper(self) -> Text -- convert this Unreal text to uppercase in a culture correct way" },
		{ "format", PyCFunctionCast(&FMethods::Format), METH_VARARGS | METH_KEYWORDS, "format(self, *args: Union[Mapping[str, object], object], **named_args: object) -> Text -- use this Unreal text as a format pattern and generate a new text using the format arguments (may be a mapping[arg_name, arg] or comma separed (optionally named) arguments)" },
		{ nullptr, nullptr, 0, nullptr }
	};

	PyTypeObject PyType = InitializePyWrapperBasicType<FPyWrapperText>("Text", "Type for all Unreal exposed text instances");

	PyType.tp_init = (initproc)&FFuncs::Init;
	PyType.tp_str = (reprfunc)&FFuncs::Str;
	PyType.tp_repr = (reprfunc)&FFuncs::Repr;
	PyType.tp_richcompare = (richcmpfunc)&FFuncs::RichCmp;
	PyType.tp_hash = (hashfunc)&FFuncs::Hash;

	PyType.tp_methods = PyMethods;

	return PyType;
}

PyTypeObject PyWrapperTextType = InitializePyWrapperTextType();

#endif	// WITH_PYTHON
