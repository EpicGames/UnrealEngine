// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PyWrapperBasic.h"

#if WITH_PYTHON

/** Python type for FPyWrapperText */
extern PyTypeObject PyWrapperTextType;

/** Initialize the FPyWrapperText types and add them to the given Python module */
void InitializePyWrapperText(PyGenUtil::FNativePythonModule& ModuleInfo);

/** Type for all Unreal exposed FText instances */
struct FPyWrapperText : public TPyWrapperBasic<FText, FPyWrapperText>
{
	typedef TPyWrapperBasic<FText, FPyWrapperText> Super;

	/** Cast the given Python object to this wrapped type (returns a new reference) */
	static FPyWrapperText* CastPyObject(PyObject* InPyObject, FPyConversionResult* OutCastResult = nullptr);

	/** Cast the given Python object to this wrapped type, or attempt to convert the type into a new wrapped instance (returns a new reference) */
	static FPyWrapperText* CastPyObject(PyObject* InPyObject, PyTypeObject* InType, FPyConversionResult* OutCastResult = nullptr);

	/** Initialize the value of this wrapper instance (internal) */
	static void InitValue(FPyWrapperText* InSelf, const FText InValue);

	/** Deinitialize the value of this wrapper instance (internal) */
	static void DeinitValue(FPyWrapperText* InSelf);
};

typedef TPyPtr<FPyWrapperText> FPyWrapperTextPtr;

#endif	// WITH_PYTHON
