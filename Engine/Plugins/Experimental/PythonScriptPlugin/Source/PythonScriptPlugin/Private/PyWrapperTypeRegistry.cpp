// Copyright Epic Games, Inc. All Rights Reserved.

#include "PyWrapperTypeRegistry.h"
#include "PyWrapperOwnerContext.h"
#include "PyWrapperObject.h"
#include "PyWrapperStruct.h"
#include "PyWrapperDelegate.h"
#include "PyWrapperEnum.h"
#include "PyWrapperName.h"
#include "PyWrapperText.h"
#include "PyWrapperArray.h"
#include "PyWrapperFixedArray.h"
#include "PyWrapperSet.h"
#include "PyWrapperMap.h"
#include "PyWrapperFieldPath.h"
#include "PyConversion.h"
#include "PyGIL.h"
#include "PyCore.h"
#include "PyEditor.h"
#include "PyEngine.h"
#include "PyFileWriter.h"
#include "PythonScriptPluginSettings.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "HAL/IConsoleManager.h"
#include "HAL/ThreadHeartBeat.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringBuilder.h"
#include "SourceCodeNavigation.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectHash.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/EnumProperty.h"
#include "UObject/StructOnScope.h"
#include "UObject/NameTypes.h"
#include "Misc/CoreDelegates.h"
#include "Async/UniqueLock.h"
#include "Async/ParallelFor.h"
#include "Tasks/Task.h"
#include "Sanitizer/RaceDetector.h"
#if WITH_EDITOR
#include "Kismet2/ReloadUtilities.h"
#endif

#define LOCTEXT_NAMESPACE "PyWrapperTypeRegistry"

#if WITH_PYTHON

namespace UE::Python::Private
{

static bool GPythonForceWrapperGenerationOnSingleThread = false;
static FAutoConsoleVariableRef CVarGPythonForceWrapperGenerationOnSingleThread(
	TEXT("Engine.Python.ForceWrapperGenerationOnSingleThread"),
	GPythonForceWrapperGenerationOnSingleThread,
	TEXT("Force all wrapper generation to happen on a single thread (slow)."),
	ECVF_Default
);

#if USING_INSTRUMENTATION

static bool GPythonWrapperGenerationRaceDetector = false;
static FAutoConsoleVariableRef CVarGPythonWrapperGenerationRaceDetector(
	TEXT("Engine.Python.WrapperGenerationRaceDetector"),
	GPythonWrapperGenerationRaceDetector,
	TEXT("Activate race detector for the duration of the python wrapper generation."),
	ECVF_Default
);

#endif

static bool GIsInMultithreadedGeneration = false;

} // namespace UE::Python::Private

namespace UE::Python
{

/**
 * Finds the UFunction corresponding to the name specified by 'HasNativeMake' or 'HasNativeBreak' meta data key.
 * @param The structure to inspect for the 'HasNativeMake' or 'HasNativeBreak' meta data keys.
 * @param InNativeMetaDataKey The native meta data key name. Can only be 'HasNativeMake' or 'HasNativeBreak'.
 * @param InScriptDefaultMetaDataKey The script default meta data key name. Can only be 'ScriptDefaultMake' or 'ScriptDefaultBreak'.
 * @param NotFoundFn Function invoked if the structure specifies as Make or Break function, but the function couldn't be found.
 * @return The function, if the struct has the meta key and if the function was found. Null otherwise.
 */
template<typename NotFoundFuncT>
UFunction* FindMakeBreakFunction(const UScriptStruct* InStruct, const FName& InNativeMetaDataKey, const FName& InScriptDefaultMetaDataKey, const NotFoundFuncT& NotFoundFn)
{
	check(InNativeMetaDataKey == PyGenUtil::HasNativeMakeMetaDataKey || InNativeMetaDataKey == PyGenUtil::HasNativeBreakMetaDataKey);
	check(InScriptDefaultMetaDataKey == PyGenUtil::ScriptDefaultMakeMetaDataKey || InScriptDefaultMetaDataKey == PyGenUtil::ScriptDefaultBreakMetaDataKey);

	UFunction* MakeBreakFunc = nullptr;
	if (!InStruct->HasMetaData(InScriptDefaultMetaDataKey))
	{
		const FString MakeBreakFunctionName = InStruct->GetMetaData(InNativeMetaDataKey);
		if (!MakeBreakFunctionName.IsEmpty())
		{
			// Find the function.
			MakeBreakFunc = FindObject<UFunction>(/*Outer*/nullptr, *MakeBreakFunctionName, EFindObjectFlags::ExactClass);
			if (!MakeBreakFunc)
			{
				NotFoundFn(MakeBreakFunctionName);
			}
		}
	}
	return MakeBreakFunc;
}

void SetMakeFunction(FPyWrapperStructMetaData& MetaData, UFunction* MakeFunc)
{
	check(MetaData.Struct != nullptr && MakeFunc != nullptr);

	MetaData.MakeFunc.SetFunction(MakeFunc);

	const bool bHasValidReturn =
		MetaData.MakeFunc.OutputParams.Num() == 1 &&
		MetaData.MakeFunc.OutputParams[0].ParamProp->IsA<FStructProperty>() &&
		CastFieldChecked<const FStructProperty>(MetaData.MakeFunc.OutputParams[0].ParamProp)->Struct == MetaData.Struct;

	if (!bHasValidReturn)
	{
		REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("Struct '%s' is marked as 'HasNativeMake' but the function '%s' does not return the struct type. Python will use the generic make function to create a Python object of this type."), *MetaData.Struct->GetPathName(), *MetaData.MakeFunc.Func->GetPathName());
		MetaData.MakeFunc.SetFunction(nullptr);
		return;
	}

	// Set the make arguments to be optional to mirror the behavior of struct InitParams
	for (PyGenUtil::FGeneratedWrappedMethodParameter& InputParam : MetaData.MakeFunc.InputParams)
	{
		if (!InputParam.ParamDefaultValue.IsSet())
		{
			InputParam.ParamDefaultValue = FString();
		}
	}
}

void SetBreakFunction(FPyWrapperStructMetaData& MetaData, UFunction* BreakFunc)
{
	check(MetaData.Struct != nullptr && BreakFunc != nullptr);

	MetaData.BreakFunc.SetFunction(BreakFunc);

	const bool bHasValidInput =
		MetaData.BreakFunc.InputParams.Num() == 1 &&
		MetaData.BreakFunc.InputParams[0].ParamProp->IsA<FStructProperty>() &&
		CastFieldChecked<const FStructProperty>(MetaData.BreakFunc.InputParams[0].ParamProp)->Struct == MetaData.Struct;

	if (!bHasValidInput)
	{
		REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("Struct '%s' is marked as 'HasNativeBreak' but the function '%s' does not have the struct type as its only input argument. Python will use the generic break function to convert a Python object of this type into a tuple."), *MetaData.Struct->GetPathName(), *MetaData.BreakFunc.Func->GetPathName());
		MetaData.BreakFunc.SetFunction(nullptr);
	}
};


static bool GGeneratingOnlineDoc = false;

void SetGeneratingOnlineDoc(bool bGeneratingOnlineDoc)
{
	GGeneratingOnlineDoc = bGeneratingOnlineDoc;
}

bool IsGeneratingOnlineDoc()
{
	return GGeneratingOnlineDoc;
}

bool ShouldEllipseParamDefaultValue(const FString& DefaultValue)
{
	// Some default values might not parse because they use undeclared types.
	const bool bEnsureDefaultValueParses = (PyGenUtil::GetTypeHintingMode() == ETypeHintingMode::TypeChecker || IsGeneratingOnlineDoc());

	if (bEnsureDefaultValueParses)
	{
		// Don't ellipse the default value if it is a simple Python expression. We want to the default value with "..." when the
		// expression might refer to types that aren't declared yet. We can use undeclared type for type hint, but not for default
		// value. The stub types are not written in a strict order so one type can refer another that was written later. For sake
		// of simplicity, just keep simple default values, ellipse all others to the user detriment. :(
		return	!(DefaultValue.IsNumeric() ||      // int or float
					DefaultValue == TEXT("True") ||  // bool
					DefaultValue == TEXT("False") || // bool
					DefaultValue == TEXT("[]") ||    // list
					DefaultValue == TEXT("{}") ||    // dict
					DefaultValue == TEXT("()") ||    // tuple.
					(DefaultValue.StartsWith("'") && DefaultValue.EndsWith("'")) || // string
					(DefaultValue.StartsWith("\"") && DefaultValue.EndsWith("\""))); // string
	}
	return false;
};

/**
 * Parses a Python doc string to find a method declaration that starts with the method name and ends with "--". For example
 * if a given doc string is "xyz.log(arg: Any) -> None -- log the given argument as information in the LogPython category"
 * the method "log(arg: Any) -> None" will be returned as the declaration if the searched method was "log".
 */
bool ParseMethodDeclarationFromDocString(const FString& InDocString, const FString& InMethodName, FString& OutMethodDecl)
{
	int32 BeginFuncDeclPos = InDocString.Find(InMethodName, ESearchCase::CaseSensitive, ESearchDir::FromStart, 0);
	if (BeginFuncDeclPos == INDEX_NONE)
	{
		return false;
	}

	int32 EndFuncDeclPos = InDocString.Find(TEXT("--"), ESearchCase::CaseSensitive, ESearchDir::FromStart, BeginFuncDeclPos);
	if (EndFuncDeclPos == INDEX_NONE)
	{
		return false;
	}

	OutMethodDecl = InDocString.Mid(BeginFuncDeclPos, EndFuncDeclPos - BeginFuncDeclPos);
	return true;
}

/**
 * Reads the human written doc string of a non-reflected function and try to extract a well-formed methods declaration containing
 * type hinting. The function aims to generate type hints for the raw C/C++ functions that are not reflected but still exposed. If
 * the function fails to find a well-formed and type hinted declaration in the doc string, it returns the generic signature
 * 'def funcName(*args, **kargs):' that doesn't hint for types, but will still allow a form of auto-completion in Python text editors.
 * Since raw C/C++ function doc strings are handwritten, they need extra caution to avoid errors. The function cannot check the
 * types being handwritten, so the burden is put on the function provider to keep the declaration accurate.
 * 
 * To be successfully parsed, the type hinted declaration from the doc string must be well formed. See format below in the implementation.
 * Any malformed declaration will be rejected and the function will fallback on the genric form.
 */
FString GenerateMethodDecl(const PyMethodDef* MethodDef, const FString& ModuleOrClass, bool bIsClassMethod = false, bool bIsMemberMethod = false)
{
	// Parse the Python method declared in the doc string check if it is a valid type hinted declaration and possibly fix it to make it well-formed.
	auto TryParsingDocStringMethodDecl = [bIsClassMethod, bIsMemberMethod](const FStringView& InMethodDecl, const FStringView& InMethodName, FString& OutMethodDeclWithHints)
	{
		int32 OpenParenthesisPos = InMethodDecl.Find(TEXT("("));
		if (OpenParenthesisPos == INDEX_NONE)
		{
			return false;
		}

		int32 CloseParenthesisPos = INDEX_NONE;
		if (!InMethodDecl.FindLastChar(TEXT(')'), CloseParenthesisPos))
		{
			return false;
		}

		FStringView ReturnTypeDelimiter(TEXT("->"));
		int32 ReturnTypeDelimiterPos = InMethodDecl.Find(ReturnTypeDelimiter, CloseParenthesisPos);
		if (ReturnTypeDelimiterPos == INDEX_NONE)
		{
			return false;
		}

		// Check if the return type is hinted.
		int32 BeginReturnTypePos = ReturnTypeDelimiterPos + ReturnTypeDelimiter.Len(); // After the ->
		FStringView ReturnType = InMethodDecl.SubStr(BeginReturnTypePos, InMethodDecl.Len() - BeginReturnTypePos).TrimStartAndEnd();
		if (ReturnType.Len() <= 0)
		{
			return false;
		}

		int32 ParamCount = 0;
		FString Params;

		bool bClsOrSelfParamFound = false;

		// Check if the parameters are correcly hinted.
		int32 ParamsLen = CloseParenthesisPos - OpenParenthesisPos;
		if (ParamsLen > 1)
		{
			FStringView AllParamsString = InMethodDecl.SubStr(OpenParenthesisPos + 1, ParamsLen - 1); // +1 and -1 to exclude the parenthesis.

			bool bLastParamReached = false;
			int32 StartParamPos = 0;
			do
			{
				// Find the end of a parameter declaration, caring for commas that aren't parameter separator, but part of the type hint, like in
				// Callable[[int, int], str] or Union[int, str, float]. We need to skip the content of [] to find the end of a parameter declaration.
				int BracketCount = 0;
				int CurrPos = StartParamPos;
				int EndParamPos = INDEX_NONE;
				for (TCHAR Ch : AllParamsString.SubStr(StartParamPos, AllParamsString.Len() - StartParamPos))
				{
					if (Ch == TEXT('['))
					{
						++BracketCount;
					}
					else if (Ch == TEXT(']'))
					{
						--BracketCount;
					}
					else if (Ch == TEXT(','))
					{
						if (BracketCount == 0) // Outside [], the comma separates parameters.
						{
							EndParamPos = CurrPos;
							break;
						}
					}
					++CurrPos;
				}

				// Check if the parameters declaration is correctly formed.
				FStringView ParamDecl;

				if (EndParamPos != INDEX_NONE)
				{
					ParamDecl = AllParamsString.SubStr(StartParamPos, EndParamPos - StartParamPos);
				}
				else
				{
					ParamDecl = AllParamsString.SubStr(StartParamPos, AllParamsString.Len() - StartParamPos);
					bLastParamReached = true;
				}

				// Search for parameter type hint separator.
				const int32 NameTypeSeparatorPos = ParamDecl.Find(TEXT(":"));

				// Those special doesn't require type hint, but they can if needed.
				bool bIsSelf = ParamDecl.StartsWith(TEXT("self"));
				bool bIsCls = ParamDecl.StartsWith(TEXT("cls"));
				bool bParamTypeHinted = NameTypeSeparatorPos != INDEX_NONE;
				bClsOrSelfParamFound |= bIsSelf || bIsCls;

				if (!bParamTypeHinted && !bIsSelf && !bIsCls)
				{
					return false; // Invalid declaration: the param is not type hinted.
				}

				FStringView ParamName;
				FStringView ParamTypeAndOptDefaultValue;
				if (bParamTypeHinted)
				{
					ParamName = ParamDecl.SubStr(0, NameTypeSeparatorPos).TrimStartAndEnd();
					ParamTypeAndOptDefaultValue = ParamDecl.SubStr(NameTypeSeparatorPos + 1, ParamDecl.Len() - NameTypeSeparatorPos).TrimStartAndEnd();

					if (ParamName.Len() == 0 || ParamTypeAndOptDefaultValue.Len() == 0)
					{
						return false;
					}
				}
				else // Only 'self' or 'cls' case should remain.
				{
					ParamName = ParamDecl.TrimStartAndEnd();
					if (ParamName != TEXT("self") && ParamName != TEXT("cls"))
					{
						return false;
					}
				}

				// Check if the paramType declaration also embeds a default value.
				const int32 DefaultValueSeparatorPos = ParamTypeAndOptDefaultValue.Find(TEXT("="));
				if (DefaultValueSeparatorPos == INDEX_NONE)
				{
					Params += PyGenUtil::PythonizeMethodParam(FString(ParamName), FString(ParamTypeAndOptDefaultValue)); // No default value found.
				}
				else
				{
					FString ParamType(ParamTypeAndOptDefaultValue.SubStr(0, DefaultValueSeparatorPos).TrimStartAndEnd());
					FString ParamDefaultValue(ParamTypeAndOptDefaultValue.SubStr(DefaultValueSeparatorPos + 1, ParamTypeAndOptDefaultValue.Len() - DefaultValueSeparatorPos).TrimStartAndEnd());
					if (ShouldEllipseParamDefaultValue(ParamDefaultValue))
					{
						ParamDefaultValue = TEXT("...");
					}
					Params += PyGenUtil::PythonizeMethodParam(FString(ParamName), FString(ParamType), FString(ParamDefaultValue));
				}

				if (!bLastParamReached)
				{
					Params += TEXT(", ");
				}

				StartParamPos = EndParamPos + 1; // +1 to start after the comma.
				++ParamCount;
			}
			while (!bLastParamReached);
		}

		OutMethodDeclWithHints = TEXT("def ");
		OutMethodDeclWithHints += InMethodName;
		OutMethodDeclWithHints += TEXT("(");

		if (bIsClassMethod && !bClsOrSelfParamFound)
		{
			OutMethodDeclWithHints += FString::Printf(TEXT("cls%s"), ParamCount > 0 ? TEXT(", ") : TEXT(""));
		}
		else if (bIsMemberMethod && !bClsOrSelfParamFound)
		{
			OutMethodDeclWithHints += FString::Printf(TEXT("self%s"), ParamCount > 0 ? TEXT(", ") : TEXT(""));
		}

		// All parameters type hint format was properly formed. Add them to the declaration.
		OutMethodDeclWithHints += Params;

		OutMethodDeclWithHints += TEXT(")");
		OutMethodDeclWithHints += TEXT(" -> ");
		OutMethodDeclWithHints += ReturnType;
		OutMethodDeclWithHints += TEXT(":");

		// The DocString contains a function declaration with properly formed with type hints. It is assumed to be a 'correct' declaration, but
		// that doc string is handwritten, so human errors are possible and types might be incorrect.
		return true;
	};

	FString DocString = UTF8_TO_TCHAR(MethodDef->ml_doc);
	FString MethodName = UTF8_TO_TCHAR(MethodDef->ml_name);
	FString MethodDecl;

	if (PyGenUtil::IsTypeHintingEnabled())
	{
		// Try extracting type hinting from the doc string. If the Doc string appears to be well-formed, it is likely because the writer
		// took time to type-hint it correctly.
		if (ParseMethodDeclarationFromDocString(DocString, MethodName, MethodDecl))
		{
			FString WellFormedMethodDeclWithHints;
			if (TryParsingDocStringMethodDecl(MethodDecl, MethodName, WellFormedMethodDeclWithHints))
			{
				return WellFormedMethodDeclWithHints;
			}
		}
	}

	// Failed to extract a well-formed type hinted method declaration from the doc string. Fallback to the generic and safe declaration.
	const bool bHasKeywords = !!(MethodDef->ml_flags & METH_KEYWORDS);
	if (bIsClassMethod)
	{
		MethodDecl = FString::Printf(TEXT("def %s(cls, *args%s):"), *MethodName, bHasKeywords ? TEXT(", **kwargs") : TEXT(""));
	}
	else if (bIsMemberMethod)
	{
		MethodDecl = FString::Printf(TEXT("def %s(self, *args%s):"), *MethodName, bHasKeywords ? TEXT(", **kwargs") : TEXT(""));
	}
	else
	{
		MethodDecl = FString::Printf(TEXT("def %s(*args%s):"), *MethodName, bHasKeywords ? TEXT(", **kwargs") : TEXT(""));
	}

	return MethodDecl;
}

} // namespace UE::Python



DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Generate Wrapped Class Total Time"), STAT_GenerateWrappedClassTotalTime, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Class Call Count"), STAT_GenerateWrappedClassCallCount, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Class Obj Count"), STAT_GenerateWrappedClassObjCount, STATGROUP_Python);

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Generate Wrapped Struct Total Time"), STAT_GenerateWrappedStructTotalTime, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Struct Call Count"), STAT_GenerateWrappedStructCallCount, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Struct Obj Count"), STAT_GenerateWrappedStructObjCount, STATGROUP_Python);

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Generate Wrapped Enum Total Time"), STAT_GenerateWrappedEnumTotalTime, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Enum Call Count"), STAT_GenerateWrappedEnumCallCount, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Enum Obj Count"), STAT_GenerateWrappedEnumObjCount, STATGROUP_Python);

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Generate Wrapped Delegate Total Time"), STAT_GenerateWrappedDelegateTotalTime, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Delegate Call Count"), STAT_GenerateWrappedDelegateCallCount, STATGROUP_Python);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Generate Wrapped Delegate Obj Count"), STAT_GenerateWrappedDelegateObjCount, STATGROUP_Python);

FPyWrapperObjectFactory& FPyWrapperObjectFactory::Get()
{
	static FPyWrapperObjectFactory Instance;
	return Instance;
}

FPyWrapperObject* FPyWrapperObjectFactory::FindInstance(UObject* InUnrealInstance) const
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(InUnrealInstance->GetClass());
	return FindInstanceInternal(InUnrealInstance, PyType);
}

FPyWrapperObject* FPyWrapperObjectFactory::CreateInstance(UObject* InUnrealInstance)
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(InUnrealInstance->GetClass());
	return CreateInstanceInternal(InUnrealInstance, PyType, [InUnrealInstance](FPyWrapperObject* InSelf)
	{
		return FPyWrapperObject::Init(InSelf, InUnrealInstance);
	});
}

FPyWrapperObject* FPyWrapperObjectFactory::CreateInstance(UClass* InInterfaceClass, UObject* InUnrealInstance)
{
	if (!InInterfaceClass || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedClassType(InInterfaceClass);
	return CreateInstanceInternal(InUnrealInstance, PyType, [InUnrealInstance](FPyWrapperObject* InSelf)
	{
		return FPyWrapperObject::Init(InSelf, InUnrealInstance);
	});
}


FPyWrapperStructFactory& FPyWrapperStructFactory::Get()
{
	static FPyWrapperStructFactory Instance;
	return Instance;
}

FPyWrapperStruct* FPyWrapperStructFactory::FindInstance(UScriptStruct* InStruct, void* InUnrealInstance) const
{
	if (!InStruct || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(InStruct);
	return FindInstanceInternal(InUnrealInstance, PyType);
}

FPyWrapperStruct* FPyWrapperStructFactory::CreateInstance(UScriptStruct* InStruct, void* InUnrealInstance, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InStruct || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedStructType(InStruct);
	return CreateInstanceInternal(InUnrealInstance, PyType, [InStruct, InUnrealInstance, &InOwnerContext, InConversionMethod](FPyWrapperStruct* InSelf)
	{
		return FPyWrapperStruct::Init(InSelf, InOwnerContext, InStruct, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}


FPyWrapperDelegateFactory& FPyWrapperDelegateFactory::Get()
{
	static FPyWrapperDelegateFactory Instance;
	return Instance;
}

FPyWrapperDelegate* FPyWrapperDelegateFactory::FindInstance(const UFunction* InDelegateSignature, FScriptDelegate* InUnrealInstance) const
{
	if (!InDelegateSignature || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedDelegateType(InDelegateSignature);
	return FindInstanceInternal(InUnrealInstance, PyType);
}

FPyWrapperDelegate* FPyWrapperDelegateFactory::CreateInstance(const UFunction* InDelegateSignature, FScriptDelegate* InUnrealInstance, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InDelegateSignature || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedDelegateType(InDelegateSignature);
	return CreateInstanceInternal(InUnrealInstance, PyType, [InUnrealInstance, &InOwnerContext, InConversionMethod](FPyWrapperDelegate* InSelf)
	{
		return FPyWrapperDelegate::Init(InSelf, InOwnerContext, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}


FPyWrapperMulticastDelegateFactory& FPyWrapperMulticastDelegateFactory::Get()
{
	static FPyWrapperMulticastDelegateFactory Instance;
	return Instance;
}

FPyWrapperMulticastDelegate* FPyWrapperMulticastDelegateFactory::FindInstance(const UFunction* InDelegateSignature, FMulticastScriptDelegate* InUnrealInstance) const
{
	if (!InDelegateSignature || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedDelegateType(InDelegateSignature);
	return FindInstanceInternal(InUnrealInstance, PyType);
}

FPyWrapperMulticastDelegate* FPyWrapperMulticastDelegateFactory::CreateInstance(const UFunction* InDelegateSignature, FMulticastScriptDelegate* InUnrealInstance, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InDelegateSignature || !InUnrealInstance)
	{
		return nullptr;
	}

	PyTypeObject* PyType = FPyWrapperTypeRegistry::Get().GetWrappedDelegateType(InDelegateSignature);
	return CreateInstanceInternal(InUnrealInstance, PyType, [InUnrealInstance, &InOwnerContext, InConversionMethod](FPyWrapperMulticastDelegate* InSelf)
	{
		return FPyWrapperMulticastDelegate::Init(InSelf, InOwnerContext, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}


FPyWrapperNameFactory& FPyWrapperNameFactory::Get()
{
	static FPyWrapperNameFactory Instance;
	return Instance;
}

FPyWrapperName* FPyWrapperNameFactory::FindInstance(const FName InUnrealInstance) const
{
	return FindInstanceInternal(InUnrealInstance, &PyWrapperNameType);
}

FPyWrapperName* FPyWrapperNameFactory::CreateInstance(const FName InUnrealInstance)
{
	return CreateInstanceInternal(InUnrealInstance, &PyWrapperNameType, [InUnrealInstance](FPyWrapperName* InSelf)
	{
		return FPyWrapperName::Init(InSelf, InUnrealInstance);
	});
}


FPyWrapperTextFactory& FPyWrapperTextFactory::Get()
{
	static FPyWrapperTextFactory Instance;
	return Instance;
}

FPyWrapperText* FPyWrapperTextFactory::FindInstance(const FText InUnrealInstance) const
{
	return FindInstanceInternal(InUnrealInstance, &PyWrapperTextType);
}

FPyWrapperText* FPyWrapperTextFactory::CreateInstance(const FText InUnrealInstance)
{
	return CreateInstanceInternal(InUnrealInstance, &PyWrapperTextType, [InUnrealInstance](FPyWrapperText* InSelf)
	{
		return FPyWrapperText::Init(InSelf, InUnrealInstance);
	});
}


FPyWrapperArrayFactory& FPyWrapperArrayFactory::Get()
{
	static FPyWrapperArrayFactory Instance;
	return Instance;
}

FPyWrapperArray* FPyWrapperArrayFactory::FindInstance(void* InUnrealInstance) const
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return FindInstanceInternal(InUnrealInstance, &PyWrapperArrayType);
}

FPyWrapperArray* FPyWrapperArrayFactory::CreateInstance(void* InUnrealInstance, const FArrayProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return CreateInstanceInternal(InUnrealInstance, &PyWrapperArrayType, [InUnrealInstance, InProp, &InOwnerContext, InConversionMethod](FPyWrapperArray* InSelf)
	{
		return FPyWrapperArray::Init(InSelf, InOwnerContext, InProp, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}


FPyWrapperFixedArrayFactory& FPyWrapperFixedArrayFactory::Get()
{
	static FPyWrapperFixedArrayFactory Instance;
	return Instance;
}

FPyWrapperFixedArray* FPyWrapperFixedArrayFactory::FindInstance(void* InUnrealInstance) const
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return FindInstanceInternal(InUnrealInstance, &PyWrapperFixedArrayType);
}

FPyWrapperFixedArray* FPyWrapperFixedArrayFactory::CreateInstance(void* InUnrealInstance, const FProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return CreateInstanceInternal(InUnrealInstance, &PyWrapperFixedArrayType, [InUnrealInstance, InProp, &InOwnerContext, InConversionMethod](FPyWrapperFixedArray* InSelf)
	{
		return FPyWrapperFixedArray::Init(InSelf, InOwnerContext, InProp, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}


FPyWrapperSetFactory& FPyWrapperSetFactory::Get()
{
	static FPyWrapperSetFactory Instance;
	return Instance;
}

FPyWrapperSet* FPyWrapperSetFactory::FindInstance(void* InUnrealInstance) const
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return FindInstanceInternal(InUnrealInstance, &PyWrapperSetType);
}

FPyWrapperSet* FPyWrapperSetFactory::CreateInstance(void* InUnrealInstance, const FSetProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return CreateInstanceInternal(InUnrealInstance, &PyWrapperSetType, [InUnrealInstance, InProp, &InOwnerContext, InConversionMethod](FPyWrapperSet* InSelf)
	{
		return FPyWrapperSet::Init(InSelf, InOwnerContext, InProp, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}


FPyWrapperMapFactory& FPyWrapperMapFactory::Get()
{
	static FPyWrapperMapFactory Instance;
	return Instance;
}

FPyWrapperMap* FPyWrapperMapFactory::FindInstance(void* InUnrealInstance) const
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return FindInstanceInternal(InUnrealInstance, &PyWrapperMapType);
}

FPyWrapperMap* FPyWrapperMapFactory::CreateInstance(void* InUnrealInstance, const FMapProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod)
{
	if (!InUnrealInstance)
	{
		return nullptr;
	}

	return CreateInstanceInternal(InUnrealInstance, &PyWrapperMapType, [InUnrealInstance, InProp, &InOwnerContext, InConversionMethod](FPyWrapperMap* InSelf)
	{
		return FPyWrapperMap::Init(InSelf, InOwnerContext, InProp, InUnrealInstance, InConversionMethod);
	}, InConversionMethod == EPyConversionMethod::Copy || InConversionMethod == EPyConversionMethod::Steal);
}

FPyWrapperFieldPathFactory& FPyWrapperFieldPathFactory::Get()
{
	static FPyWrapperFieldPathFactory Instance;
	return Instance;
}

FPyWrapperFieldPath* FPyWrapperFieldPathFactory::FindInstance(FFieldPath InUnrealInstance) const
{
	return FindInstanceInternal(InUnrealInstance, &PyWrapperFieldPathType);
}

FPyWrapperFieldPath* FPyWrapperFieldPathFactory::CreateInstance(FFieldPath InUnrealInstance)
{
	return CreateInstanceInternal(InUnrealInstance, &PyWrapperFieldPathType, [InUnrealInstance](FPyWrapperFieldPath* InSelf)
	{
		return FPyWrapperFieldPath::Init(InSelf, InUnrealInstance);
	});
}

FPyWrapperTypeReinstancer& FPyWrapperTypeReinstancer::Get()
{
	static FPyWrapperTypeReinstancer Instance;
	return Instance;
}

void FPyWrapperTypeReinstancer::AddPendingClass(UPythonGeneratedClass* OldClass, UPythonGeneratedClass* NewClass)
{
	ClassesToReinstance.Emplace(OldClass, NewClass);
}

void FPyWrapperTypeReinstancer::AddPendingStruct(UPythonGeneratedStruct* OldStruct, UPythonGeneratedStruct* NewStruct)
{
	StructsToReinstance.Emplace(OldStruct, NewStruct);
}

void FPyWrapperTypeReinstancer::ProcessPending()
{
	if (ClassesToReinstance.Num() > 0 || StructsToReinstance.Num() > 0)
	{
		TUniquePtr<FReload> Reload = MakeUnique<FReload>(EActiveReloadType::Reinstancing, TEXT(""), *GLog);

		for (const TTuple<TObjectPtr<UPythonGeneratedClass>, TObjectPtr<UPythonGeneratedClass>>& ClassToReinstancePair : ClassesToReinstance)
		{
			if (ClassToReinstancePair.Key && ClassToReinstancePair.Value)
			{
				// Resolve any CLASS_NewerVersionExists chains, so that any older classes all get reinstanced onto the most recent class
				UPythonGeneratedClass* NewClass = ClassToReinstancePair.Value;
				while (NewClass && NewClass->HasAnyClassFlags(CLASS_NewerVersionExists))
				{
					// We expect to find the replacement for NewClass in ClassesToReinstance, since 
					// that's the only way it should have been marked as CLASS_NewerVersionExists
					NewClass = ClassesToReinstance.FindRef(NewClass);
					ensureAlways(NewClass);
				}

				if (NewClass)
				{
					// Assume the classes have changed
					Reload->NotifyChange(NewClass, ClassToReinstancePair.Key);
				}
				else
				{
					// We failed to resolve the new class; old class won't be purged via Reinstance(),
					// but its property links must still be cleared so that DestroyNonNativeProperties
					// on its CDO doesn't walk FFields that may be freed by the base class FinishDestroy
					ClassToReinstancePair.Key->DestroyChildPropertiesAndResetPropertyLinks();
				}
			}
		}

		// This doesn't do anything ATM
		for (const TTuple<TObjectPtr<UPythonGeneratedStruct>, TObjectPtr<UPythonGeneratedStruct>>& StructToReinstancePair : StructsToReinstance)
		{
			if (StructToReinstancePair.Key && StructToReinstancePair.Value)
			{
				// Assume the structures have changed
				Reload->NotifyChange(StructToReinstancePair.Value, StructToReinstancePair.Key);
			}
		}

		Reload->Reinstance();

		ClassesToReinstance.Reset();
		StructsToReinstance.Reset();

		// Skip the GC in case we're actively loading to avoid a full flush especially during engine startup.
		// The next GC will take care of the cleanup.
		if (!IsAsyncLoading())
		{
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}
}

void FPyWrapperTypeReinstancer::Shutdown()
{
	ClassesToReinstance.Empty();
	StructsToReinstance.Empty();
}

void FPyWrapperTypeReinstancer::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObjects(ClassesToReinstance);
	InCollector.AddReferencedObjects(StructsToReinstance);
}

FString FPyWrapperTypeReinstancer::GetReferencerName() const
{
	return TEXT("FPyWrapperTypeReinstancer");
}


FPyWrapperTypeRegistry::FPyWrapperTypeRegistry()
	: bCanRegisterInlineStructFactories(true)
{
	// When everything is loaded, warn the user if there are still unresolved Make/Break functions.
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
	{
		UnresolvedMakeFuncs.ForEach([](const TPair<FString, TSharedPtr<FPyWrapperStructMetaData>>& UnresolvedMakePair)
		{
			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Struct '%s' is marked as '%s' but the function '%s' could not be found."), *UnresolvedMakePair.Value->Struct->GetPathName(), *PyGenUtil::HasNativeMakeMetaDataKey.ToString(), *UnresolvedMakePair.Key);
		});
		UnresolvedBreakFuncs.ForEach([](const TPair<FString, TSharedPtr<FPyWrapperStructMetaData>>& UnresolvedBreakPair)
		{
			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Struct '%s' is marked as '%s' but the function '%s' could not be found."), *UnresolvedBreakPair.Value->Struct->GetPathName(), *PyGenUtil::HasNativeBreakMetaDataKey.ToString(), *UnresolvedBreakPair.Key);
		});
	});
}

FPyWrapperTypeRegistry& FPyWrapperTypeRegistry::Get()
{
	static FPyWrapperTypeRegistry Instance;
	return Instance;
}

void FPyWrapperTypeRegistry::AddReferencedObjects(FReferenceCollector& InCollector)
{
	auto AddTypeReferencedObjects = [&InCollector](const auto& PythonWrappedTypes)
	{
		PythonWrappedTypes.ForEach([&InCollector](const auto& PythonWrappedTypePair)
		{
			if (FPyWrapperBaseMetaData* PythonWrappedTypeMetaData = FPyWrapperBaseMetaData::GetMetaData(PythonWrappedTypePair.Value))
			{
				PythonWrappedTypeMetaData->AddTypeReferencedObjects(InCollector);
			}
		});
	};

	AddTypeReferencedObjects(PythonWrappedClasses);
	AddTypeReferencedObjects(PythonWrappedStructs);
	AddTypeReferencedObjects(PythonWrappedEnums);
	AddTypeReferencedObjects(PythonWrappedDelegates);
}

FString FPyWrapperTypeRegistry::GetReferencerName() const
{
	return TEXT("FPyWrapperTypeRegistry");
}

void FPyWrapperTypeRegistry::RegisterNativePythonModule(PyGenUtil::FNativePythonModule&& NativePythonModule)
{
	NativePythonModules.Add(MoveTemp(NativePythonModule));
}

void FPyWrapperTypeRegistry::RegisterInlineStructFactory(const TSharedRef<const IPyWrapperInlineStructFactory>& InFactory)
{
	check(bCanRegisterInlineStructFactories.load(std::memory_order_relaxed));
	InlineStructFactories.Add(InFactory->GetStructName(), InFactory);
}

const IPyWrapperInlineStructFactory* FPyWrapperTypeRegistry::GetInlineStructFactory(const FTopLevelAssetPath& StructName) const
{
	return InlineStructFactories.FindRef(StructName).Get();
}

void FPyWrapperTypeRegistry::GenerateWrappedTypes(bool bAllowMultithreadedGeneration)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedTypes)

	FGeneratedWrappedTypeReferences GeneratedWrappedTypeReferences;
	TSet<FName> DirtyModules;

	double GenerateDuration = 0.0;
	{
		FScopedDurationTimer GenerateDurationTimer(GenerateDuration);

		// Need to use Get rather than ForEach as generating wrapped types 
		// may generate new objects which breaks the iteration
		TArray<UObject*> ObjectsToProcess;
		GetObjectsOfClass(UField::StaticClass(), ObjectsToProcess);

		FScopedSlowTask SlowTask(ObjectsToProcess.Num(), LOCTEXT("GeneratingPythonTypes", "Generating Python Types..."));
		SlowTask.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages

#if USING_INSTRUMENTATION
		bool bRaceDetectionActivated = false;
		if (UE::Python::Private::GPythonWrapperGenerationRaceDetector)
		{
			UE::Sanitizer::RaceDetector::ToggleRaceDetection(true);
			bRaceDetectionActivated = true;
		}
		ON_SCOPE_EXIT
		{
			if (bRaceDetectionActivated)
			{
				UE::Sanitizer::RaceDetector::ToggleRaceDetection(false);
			}
		};
#endif

		// Release the GIL if we want other threads to be able to do some work.
		PyThreadState* ThreadState = PyEval_SaveThread();

		UE::FMutex Mutex;
		std::atomic<int32> SharedIndex = 0;
		std::atomic<int32> NumActiveWorkers = 0;
		
		const uint32 NumWorkers = LowLevelTasks::FScheduler::Get().GetNumWorkers();
		const bool bIsMultithreadedGenerationAllowed = !UE::Python::Private::GPythonForceWrapperGenerationOnSingleThread && bAllowMultithreadedGeneration;
		const bool bIsMultithreaded = bIsMultithreadedGenerationAllowed && NumWorkers > 0;
		TGuardValue GuardIsInMultithreadedGeneration(UE::Python::Private::GIsInMultithreadedGeneration, bIsMultithreadedGenerationAllowed);

		PyGenUtil::FMultithreadedGenerationContext MultithreadedGenerationContext;

		auto Worker = 
			[this, &Mutex, &SharedIndex, &ObjectsToProcess, &DirtyModules, &bIsMultithreaded, &MultithreadedGenerationContext, &NumActiveWorkers]()
			{
				NumActiveWorkers++;

				TSet<FName> LocalDirtyModules;
				FGeneratedWrappedTypeReferences LocalGeneratedWrappedTypeReferences;

				while (true)
				{
					int32 Index = SharedIndex.fetch_add(1, std::memory_order_relaxed);
					if (Index >= ObjectsToProcess.Num())
					{
						break;
					}

					UObject* Object = ObjectsToProcess[Index];

					FNameBuilder ObjectPackageName;
					Object->GetPackage()->GetFName().ToString(ObjectPackageName);
					if (FPackageName::IsScriptPackage(ObjectPackageName.ToView()))
					{
						GenerateWrappedTypeForObject(Object, LocalGeneratedWrappedTypeReferences, LocalDirtyModules);
					}

					if (IsInGameThread())
					{
						MultithreadedGenerationContext.ExecuteGameThreadTasks();
					}
				}

				GenerateWrappedTypesForReferences(LocalGeneratedWrappedTypeReferences, LocalDirtyModules);

				NumActiveWorkers--;

				if (!IsInGameThread())
				{
					// Wake up the game thread
					PyGenUtil::FMultithreadedGenerationContext::EnqueueGameThreadTask([]() {});
				}

				UE::TUniqueLock ScopeLock(Mutex);
				DirtyModules.Append(LocalDirtyModules);
			};

		if (bIsMultithreaded)
		{
			UE::Tasks::FTask WorkerTask =
				UE::Tasks::Launch(
					TEXT("GeneratingPythonTypesWorker"),
					[&NumWorkers, &Worker]() 
					{
						ParallelFor(
							FMath::Clamp(NumWorkers, 1u, 24u /* No more gains past a certain point because of highly contended PyGIL. */),
							[&Worker](int32)
							{		
								Worker(); 
							},
							EParallelForFlags::Unbalanced
						);
					}
				);

			// Under multithreaded scenario, the gamethread is dedicated to running thread-unsafe tasks scheduled by other workers
			while (SharedIndex.load(std::memory_order_relaxed) < ObjectsToProcess.Num() || NumActiveWorkers.load(std::memory_order_acquire) != 0)
			{
				MultithreadedGenerationContext.ExecuteGameThreadTasksAndWait();
			}

			WorkerTask.Wait();
		}
		else
		{
			Worker();
		}

		PyEval_RestoreThread(ThreadState);

		MultithreadedGenerationContext.ExecuteGameThreadTasks();
	}

	NotifyModulesDirtied(DirtyModules);

	UE_LOGF(LogPython, Verbose, "Took %f seconds to generate and initialize Python wrapped types for the initial load.", GenerateDuration);
}

void FPyWrapperTypeRegistry::GenerateWrappedTypesForModule(const FName ModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedTypesForModule)

	UPackage* const ModulePackage = FindPackage(nullptr, *(FString("/Script/") + ModuleName.ToString()));
	if (!ModulePackage)
	{
		return;
	}

	FGeneratedWrappedTypeReferences GeneratedWrappedTypeReferences;
	TSet<FName> DirtyModules;

	double GenerateDuration = 0.0;
	{
		FScopedDurationTimer GenerateDurationTimer(GenerateDuration);

		// Need to use Get rather than ForEach as generating wrapped types 
		// may generate new objects which breaks the iteration
		TArray<UObject*> ObjectsToProcess;
		GetObjectsWithPackage(ModulePackage, ObjectsToProcess);

		FScopedSlowTask SlowTask(ObjectsToProcess.Num(), FText::Format(LOCTEXT("GeneratingPythonTypesForModule", "Generating Python Types for {0}..."), FText::FromName(ModuleName)));
		if (ObjectsToProcess.Num() > 1000)
		{
			SlowTask.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages
		}

		for (UObject* ObjectToProcess : ObjectsToProcess)
		{
			SlowTask.EnterProgressFrame();

			GenerateWrappedTypeForObject(ObjectToProcess, GeneratedWrappedTypeReferences, DirtyModules);
		}

		GenerateWrappedTypesForReferences(GeneratedWrappedTypeReferences, DirtyModules);
	}

	// Try resolving missing make/break native functions. Some USTRUCT uses make/break functions implemented in other C++ module(s)
	// that might not be loaded yet when Python glue was generated for the struct.
	UnresolvedMakeFuncs.RemoveIf([](TPair<FString, TSharedPtr<FPyWrapperStructMetaData>>& Pair)
	{
		if (UFunction* MakeFunc = FindObject<UFunction>(nullptr, *Pair.Key, EFindObjectFlags::ExactClass))
		{
			UE::Python::SetMakeFunction(*Pair.Value, MakeFunc);
			return true; // Remove
		}

		return false;
	});
	
	UnresolvedBreakFuncs.RemoveIf([](TPair<FString, TSharedPtr<FPyWrapperStructMetaData>>& Pair)
	{
		if (UFunction* BreakFunc = FindObject<UFunction>(nullptr, *Pair.Key, EFindObjectFlags::ExactClass))
		{
			UE::Python::SetBreakFunction(*Pair.Value, BreakFunc);
			return true; // Remove
		}

		return false;
	});

	NotifyModulesDirtied(DirtyModules);

	UE_LOGF(LogPython, Verbose, "Took %f seconds to generate and initialize Python wrapped types for '%ls'.", GenerateDuration, *ModuleName.ToString());
}

void FPyWrapperTypeRegistry::OrphanWrappedTypesForModule(const FName ModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::OrphanWrappedTypesForModule)

	TArray<FSoftObjectPath> ModuleTypeNames;
	GeneratedWrappedTypesForModule.MultiFind(ModuleName, ModuleTypeNames, true);
	GeneratedWrappedTypesForModule.Remove(ModuleName);

	for (const FSoftObjectPath& ModuleTypeName : ModuleTypeNames)
	{
		TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType;
		if (GeneratedWrappedTypes.RemoveAndCopyValue(ModuleTypeName, GeneratedWrappedType))
		{
			OrphanedWrappedTypes.Add(GeneratedWrappedType);

			UnregisterPythonTypeName(UTF8_TO_TCHAR(GeneratedWrappedType->PyType.tp_name), ModuleTypeName);

			PythonWrappedClasses.Remove(ModuleTypeName);
			PythonWrappedStructs.Remove(ModuleTypeName);
			PythonWrappedEnums.Remove(ModuleTypeName);
			PythonWrappedDelegates.Remove(ModuleTypeName);
		}
	}
}

void FPyWrapperTypeRegistry::GenerateWrappedTypesForReferences(const FGeneratedWrappedTypeReferences& InGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules)
{
	if (!InGeneratedWrappedTypeReferences.HasReferences())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedTypesForReferences);

	static const EPyTypeGenerationFlags ReferenceGenerationFlags = EPyTypeGenerationFlags::ForceShouldExport | EPyTypeGenerationFlags::IncludeBlueprintGeneratedTypes;

	const int32 TotalNumReferences = InGeneratedWrappedTypeReferences.ClassReferences.Num() + InGeneratedWrappedTypeReferences.StructReferences.Num() + InGeneratedWrappedTypeReferences.EnumReferences.Num() + InGeneratedWrappedTypeReferences.DelegateReferences.Num();
	FScopedSlowTask SlowTask(TotalNumReferences, LOCTEXT("GeneratingPythonTypes", "Generating Python Types..."));
	if (TotalNumReferences > 1000)
	{
		SlowTask.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages
	}

	FGeneratedWrappedTypeReferences GeneratedWrappedTypeReferences;

	for (const UClass* Class : InGeneratedWrappedTypeReferences.ClassReferences)
	{
		SlowTask.EnterProgressFrame();
		GenerateWrappedClassType(Class, GeneratedWrappedTypeReferences, OutDirtyModules, ReferenceGenerationFlags);
	}

	for (const UScriptStruct* Struct : InGeneratedWrappedTypeReferences.StructReferences)
	{
		SlowTask.EnterProgressFrame();
		GenerateWrappedStructType(Struct, GeneratedWrappedTypeReferences, OutDirtyModules, ReferenceGenerationFlags);
	}

	for (const UEnum* Enum : InGeneratedWrappedTypeReferences.EnumReferences)
	{
		SlowTask.EnterProgressFrame();
		GenerateWrappedEnumType(Enum, GeneratedWrappedTypeReferences, OutDirtyModules, ReferenceGenerationFlags);
	}

	for (const UFunction* DelegateSignature : InGeneratedWrappedTypeReferences.DelegateReferences)
	{
		SlowTask.EnterProgressFrame();
		checkf(DelegateSignature->HasAnyFunctionFlags(FUNC_Delegate), TEXT("UFunction '%s' was detected as a delegate but doesn't have the 'FUNC_Delegate' flag"), *DelegateSignature->GetPathName());
		GenerateWrappedDelegateType(DelegateSignature, GeneratedWrappedTypeReferences, OutDirtyModules, ReferenceGenerationFlags);
	}

	GenerateWrappedTypesForReferences(GeneratedWrappedTypeReferences, OutDirtyModules);
}

void FPyWrapperTypeRegistry::NotifyModulesDirtied(const TSet<FName>& InDirtyModules) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::NotifyModulesDirtied)

	for (const FName& DirtyModule : InDirtyModules)
	{
		const FString PythonModuleName = PyGenUtil::GetModulePythonName(DirtyModule, false);
		OnModuleDirtiedDelegate.Broadcast(*PythonModuleName);
	}
}

void FPyWrapperTypeRegistry::UpdateGenerateWrappedTypeForRename(const FSoftObjectPath& InOldTypeRegistryName, const UObject* InObj)
{
	const FSoftObjectPath TypeRegistryName(InObj);

	TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType = GeneratedWrappedTypes.FindAndRemoveChecked(InOldTypeRegistryName);
	check(TypeRegistryName.IsValid() && !GeneratedWrappedTypes.Contains(TypeRegistryName));
	GeneratedWrappedTypes.Add(TypeRegistryName, GeneratedWrappedType);
}

void FPyWrapperTypeRegistry::RemoveGenerateWrappedTypeForDelete(const FSoftObjectPath& InTypeRegistryName)
{
	TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType = GeneratedWrappedTypes.FindRef(InTypeRegistryName);
	const FString PythonTypeName = UTF8_TO_TCHAR(GeneratedWrappedType->TypeName.GetData());
	GeneratedWrappedType->Reset();

	// Fill the type with dummy information and re-finalize it as an empty stub
	GeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("%s_DELETED"), *PythonTypeName));
	GeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("%s was deleted!"), *InTypeRegistryName.ToString()));
	GeneratedWrappedType->Finalize();
}

PyTypeObject* FPyWrapperTypeRegistry::GenerateWrappedTypeForObject(const UObject* InObj, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags)
{
	if (const UClass* Class = Cast<const UClass>(InObj))
	{
		return GenerateWrappedClassType(Class, OutGeneratedWrappedTypeReferences, OutDirtyModules, InGenerationFlags);
	}

	if (const UScriptStruct* Struct = Cast<const UScriptStruct>(InObj))
	{
		return GenerateWrappedStructType(Struct, OutGeneratedWrappedTypeReferences, OutDirtyModules, InGenerationFlags);
	}

	if (const UEnum* Enum = Cast<const UEnum>(InObj))
	{
		return GenerateWrappedEnumType(Enum, OutGeneratedWrappedTypeReferences, OutDirtyModules, InGenerationFlags);
	}

	if (const UFunction* Func = Cast<const UFunction>(InObj))
	{
		if (Func->HasAnyFunctionFlags(FUNC_Delegate))
		{
			return GenerateWrappedDelegateType(Func, OutGeneratedWrappedTypeReferences, OutDirtyModules, InGenerationFlags);
		}
	}

	return nullptr;
}

bool FPyWrapperTypeRegistry::HasWrappedTypeForObject(const UObject* InObj) const
{
	if (const UClass* Class = Cast<const UClass>(InObj))
	{
		return HasWrappedClassType(Class);
	}

	if (const UScriptStruct* Struct = Cast<const UScriptStruct>(InObj))
	{
		return HasWrappedStructType(Struct);
	}

	if (const UEnum* Enum = Cast<const UEnum>(InObj))
	{
		return HasWrappedEnumType(Enum);
	}

	if (const UFunction* Func = Cast<const UFunction>(InObj))
	{
		if (Func->HasAnyFunctionFlags(FUNC_Delegate))
		{
			return HasWrappedDelegateType(Func);
		}
	}

	return false;
}

bool FPyWrapperTypeRegistry::HasWrappedTypeForObjectName(const FSoftObjectPath& InName) const
{
	return GeneratedWrappedTypes.Contains(InName);
}

PyTypeObject* FPyWrapperTypeRegistry::GetWrappedTypeForObject(const UObject* InObj) const
{
	if (const UClass* Class = Cast<const UClass>(InObj))
	{
		return GetWrappedClassType(Class);
	}

	if (const UScriptStruct* Struct = Cast<const UScriptStruct>(InObj))
	{
		return GetWrappedStructType(Struct);
	}

	if (const UEnum* Enum = Cast<const UEnum>(InObj))
	{
		return GetWrappedEnumType(Enum);
	}

	if (const UFunction* Func = Cast<const UFunction>(InObj))
	{
		if (Func->HasAnyFunctionFlags(FUNC_Delegate))
		{
			return GetWrappedDelegateType(Func);
		}
	}

	return nullptr;
}

PyTypeObject* FPyWrapperTypeRegistry::GenerateWrappedClassType(const UClass* InClass, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_GenerateWrappedClassTotalTime);
	INC_DWORD_STAT(STAT_GenerateWrappedClassCallCount);

	// Already processed? Nothing more to do
	const FSoftObjectPath TypeRegistryName(InClass);
	if (PyTypeObject* ExistingPyType = PythonWrappedClasses.FindRef(TypeRegistryName))
	{
		return ExistingPyType;
	}

	// todo: allow generation of Blueprint generated classes
	const bool bIsBlueprintGeneratedType = PyGenUtil::IsBlueprintGeneratedClass(InClass);
	if (bIsBlueprintGeneratedType)
	{
		return nullptr;
	}

	// Should this type be exported?
	if (!EnumHasAnyFlags(InGenerationFlags, EPyTypeGenerationFlags::ForceShouldExport) && !PyGenUtil::ShouldExportClass(InClass))
	{
		return nullptr;
	}

	// Make sure the parent class is also wrapped
	PyTypeObject* SuperPyType = nullptr;
	if (const UClass* SuperClass = InClass->GetSuperClass())
	{
		SuperPyType = GenerateWrappedClassType(SuperClass, OutGeneratedWrappedTypeReferences, OutDirtyModules, InGenerationFlags | EPyTypeGenerationFlags::ForceShouldExport);
	}

	INC_DWORD_STAT(STAT_GenerateWrappedClassObjCount);

	TSharedPtr<PyGenUtil::FGeneratedWrappedClassType> GeneratedWrappedType = 
		StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedClassType, PyGenUtil::FGeneratedWrappedType>(
			GeneratedWrappedTypes.FindOrProduce(
				TypeRegistryName,
				[]()
				{
					return MakeShared<PyGenUtil::FGeneratedWrappedClassType>();
				}
			)
		);

	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedClassType);

	// To avoid deadlock caused by wrapped type recursion, some block of code will be run
	// once we exit the lock of this wrapped type.
	TArray<TFunction<void()>> CallOnceUnlocked;
	ON_SCOPE_EXIT
	{
		for (TFunction<void()>& Function : CallOnceUnlocked)
		{
			Function();
		}
	};

	UE::TUniqueLock GeneratedWrappedTypeScopeLock(GeneratedWrappedType->Lock);
	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	TMap<FName, FName> PythonProperties;
	TMap<FName, FString> PythonDeprecatedProperties;
	TMap<FName, FName> PythonMethods;
	TMap<FName, FString> PythonDeprecatedMethods;

	auto GenerateWrappedProperty = [this, InClass, &PythonProperties, &PythonDeprecatedProperties, &GeneratedWrappedType, &OutGeneratedWrappedTypeReferences](const FProperty* InProp)
	{
		const bool bExportPropertyToScript = PyGenUtil::ShouldExportProperty(InProp);
		const bool bExportPropertyToEditor = PyGenUtil::ShouldExportEditorOnlyProperty(InProp);

		if (bExportPropertyToScript || bExportPropertyToEditor)
		{
			GatherWrappedTypesForPropertyReferences(InProp, OutGeneratedWrappedTypeReferences);

			const PyGenUtil::FGeneratedWrappedPropertyDoc& GeneratedPropertyDoc = GeneratedWrappedType->PropertyDocs[GeneratedWrappedType->PropertyDocs.Emplace(InProp)];
			PythonProperties.Add(*GeneratedPropertyDoc.PythonPropName, InProp->GetFName());

			int32 GeneratedWrappedGetSetIndex = INDEX_NONE;
			if (bExportPropertyToScript)
			{
				GeneratedWrappedGetSetIndex = GeneratedWrappedType->GetSets.TypeGetSets.AddDefaulted();

				auto FindGetSetFunction = [InClass, InProp](const FName& InKey) -> const UFunction*
				{
					const FString GetSetName = InProp->GetMetaData(InKey);
					if (!GetSetName.IsEmpty())
					{
						const UFunction* GetSetFunc = InClass->FindFunctionByName(*GetSetName);
						if (!GetSetFunc)
						{
							REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Property '%s.%s' is marked as '%s' but the function '%s' could not be found."), *InClass->GetPathName(), *InProp->GetName(), *InKey.ToString(), *GetSetName);
						}
						return GetSetFunc;
					}
					return nullptr;
				};

				PyGenUtil::FGeneratedWrappedGetSet& GeneratedWrappedGetSet = GeneratedWrappedType->GetSets.TypeGetSets[GeneratedWrappedGetSetIndex];
				GeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*GeneratedPropertyDoc.PythonPropName);
				GeneratedWrappedGetSet.GetSetDoc = PyGenUtil::TCHARToUTF8Buffer(*GeneratedPropertyDoc.DocString);
				GeneratedWrappedGetSet.Prop.SetProperty(InProp);
				GeneratedWrappedGetSet.GetFunc.SetFunction(FindGetSetFunction(PyGenUtil::BlueprintGetterMetaDataKey));
				GeneratedWrappedGetSet.SetFunc.SetFunction(FindGetSetFunction(PyGenUtil::BlueprintSetterMetaDataKey));
				GeneratedWrappedGetSet.GetCallback = (getter)&FPyWrapperObject::Getter_Impl;
				GeneratedWrappedGetSet.SetCallback = (setter)&FPyWrapperObject::Setter_Impl;
				if (GeneratedWrappedGetSet.Prop.DeprecationMessage.IsSet())
				{
					PythonDeprecatedProperties.Add(*GeneratedPropertyDoc.PythonPropName, GeneratedWrappedGetSet.Prop.DeprecationMessage.GetValue());
				}

				GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(GeneratedPropertyDoc.PythonPropName, InProp);
			}

			const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonPropNames = PyGenUtil::GetDeprecatedPropertyPythonNames(InProp);
			for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonPropNamePair : DeprecatedPythonPropNames)
			{
				const FString& DeprecatedPythonPropName = DeprecatedPythonPropNamePair.Value;
				FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonPropName, *GeneratedPropertyDoc.PythonPropName);
				PythonProperties.Add(*DeprecatedPythonPropName, InProp->GetFName());
				PythonDeprecatedProperties.Add(*DeprecatedPythonPropName, DeprecationMessage);

				if (GeneratedWrappedGetSetIndex != INDEX_NONE)
				{
					PyGenUtil::FGeneratedWrappedGetSet DeprecatedGeneratedWrappedGetSet = GeneratedWrappedType->GetSets.TypeGetSets[GeneratedWrappedGetSetIndex];
					DeprecatedGeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonPropName);
					DeprecatedGeneratedWrappedGetSet.GetSetDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
					DeprecatedGeneratedWrappedGetSet.Prop.DeprecationMessage = MoveTemp(DeprecationMessage);
					GeneratedWrappedType->GetSets.TypeGetSets.Add(MoveTemp(DeprecatedGeneratedWrappedGetSet));

					GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(DeprecatedPythonPropName, InProp);
				}
			}
		}
	};

	auto GenerateWrappedDynamicMethod = [this, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, &CallOnceUnlocked](const UFunction* InFunc, const PyGenUtil::FGeneratedWrappedMethod& InTypeMethod)
	{
		// Only static functions can be hoisted onto other types
		if (!InFunc->HasAnyFunctionFlags(FUNC_Static))
		{
			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Non-static function '%s.%s' is marked as 'ScriptMethod' but only static functions can be hoisted."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
			return;
		}

		static auto IsStructOrObjectProperty = [](const FProperty* InProp) { return InProp && (InProp->IsA<FStructProperty>() || InProp->IsA<FObjectPropertyBase>()); };

		// Get the type to hoist this method to (this should be the first parameter)
		PyGenUtil::FGeneratedWrappedMethodParameter SelfParam;
		if (InTypeMethod.MethodFunc.InputParams.Num() > 0 && IsStructOrObjectProperty(InTypeMethod.MethodFunc.InputParams[0].ParamProp))
		{
			SelfParam = InTypeMethod.MethodFunc.InputParams[0];
		}
		if (!SelfParam.ParamProp)
		{
			// Hint that UPRAM(ref) may be missing on first parameter
			const FProperty* PropertyPossiblyMissingMacro = nullptr;
			for (TFieldIterator<const FProperty> ParamIt(InFunc); ParamIt; )
			{
				const FProperty* Param = *ParamIt;
				if (PyUtil::IsOutputParameter(Param) && !PyUtil::IsInputParameter(Param) && !Param->HasAnyPropertyFlags(CPF_ReturnParm) && IsStructOrObjectProperty(Param))
				{
					PropertyPossiblyMissingMacro = Param;
				}
				break; // This is properly placed, we want to only check the first property.
			}

			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethod' but doesn't contain a valid struct or object as its first argument.%s"), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName(), PropertyPossiblyMissingMacro ? TEXT(" UPARAM(ref) may be missing on the first argument.") : TEXT(""));
			return;
		}
		if (const FObjectPropertyBase* SelfPropObj = CastField<FObjectPropertyBase>(SelfParam.ParamProp))
		{
			if (SelfPropObj->PropertyClass->IsChildOf(InFunc->GetOwnerClass()))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethod' but the object argument type (%s) is a child of the the class type of the static function. This is not allowed."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName(), *SelfPropObj->PropertyClass->GetPathName());
				return;
			}
		}

		const FString PythonStructMethodName = PyGenUtil::GetScriptMethodPythonName(InFunc);
		TArray<TUniqueObj<PyGenUtil::FGeneratedWrappedDynamicMethod>, TInlineAllocator<4>> DynamicMethodDefs;

		// Copy the basic wrapped method as we need to adjust some parts of it below
		PyGenUtil::FGeneratedWrappedDynamicMethod& GeneratedWrappedDynamicMethod = DynamicMethodDefs.AddDefaulted_GetRef().Get();
		static_cast<PyGenUtil::FGeneratedWrappedMethod&>(GeneratedWrappedDynamicMethod) = InTypeMethod;
		GeneratedWrappedDynamicMethod.SelfParam = SelfParam;

		// Hoisted methods may use an optional name alias
		GeneratedWrappedDynamicMethod.MethodName = PyGenUtil::TCHARToUTF8Buffer(*PythonStructMethodName);

		// We remove the first function parameter, as that's the 'self' argument and we'll infer that when we call
		GeneratedWrappedDynamicMethod.MethodFunc.InputParams.RemoveAt(0, EAllowShrinking::No);

		// Reference parameters may lead to a 'self' parameter that is also an output parameter
		// In this case we need to remove the output too, and set it as our 'self' return (which will apply the result back onto 'self')
		if (PyUtil::IsOutputParameter(SelfParam.ParamProp))
		{
			for (auto OutParamIt = GeneratedWrappedDynamicMethod.MethodFunc.OutputParams.CreateIterator(); OutParamIt; ++OutParamIt)
			{
				if (SelfParam.ParamProp == OutParamIt->ParamProp)
				{
					GeneratedWrappedDynamicMethod.SelfReturn = MoveTemp(*OutParamIt);
					OutParamIt.RemoveCurrent();
					break;
				}
			}
		}

		// The function may also have been flagged as having a 'self' return
		if (InFunc->HasMetaData(PyGenUtil::ScriptMethodSelfReturnMetaDataKey))
		{
			if (GeneratedWrappedDynamicMethod.SelfReturn.ParamProp)
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethodSelfReturn' but the 'self' argument is also marked as UPARAM(ref). This is not allowed."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
				return;
			}
			else if (GeneratedWrappedDynamicMethod.MethodFunc.OutputParams.Num() == 0 || !GeneratedWrappedDynamicMethod.MethodFunc.OutputParams[0].ParamProp->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethodSelfReturn' but has no return value."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
				return;
			}
			else if (!SelfParam.ParamProp->IsA<FStructProperty>())
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethodSelfReturn' but the 'self' argument is not a struct."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
				return;
			}
			else if (!GeneratedWrappedDynamicMethod.MethodFunc.OutputParams[0].ParamProp->IsA<FStructProperty>())
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethodSelfReturn' but the return value is not a struct."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
				return;
			}
			else if (CastFieldChecked<const FStructProperty>(GeneratedWrappedDynamicMethod.MethodFunc.OutputParams[0].ParamProp)->Struct != CastFieldChecked<const FStructProperty>(SelfParam.ParamProp)->Struct)
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethodSelfReturn' but the return value is not the same type as the 'self' argument."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
				return;
			}
			else
			{
				GeneratedWrappedDynamicMethod.SelfReturn = MoveTemp(GeneratedWrappedDynamicMethod.MethodFunc.OutputParams[0]);
				GeneratedWrappedDynamicMethod.MethodFunc.OutputParams.RemoveAt(0, EAllowShrinking::No);
			}
		}

		// The function may have been flagged as mutable, in which case we always consider it to need a 'self' return
		if (!GeneratedWrappedDynamicMethod.SelfReturn.ParamProp && InFunc->HasMetaData(PyGenUtil::ScriptMethodMutableMetaDataKey))
		{
			if (!SelfParam.ParamProp->IsA<FStructProperty>())
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptMethodMutable' but the 'self' argument is not a struct."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
				return;
			}
			GeneratedWrappedDynamicMethod.SelfReturn = SelfParam;
		}

		// Set-up some data needed to build the tooltip correctly for the hoisted method
		const bool bIsStaticOverride = false;
		TSet<FName> ParamsToIgnore;
		ParamsToIgnore.Add(SelfParam.ParamProp->GetFName());

		// Update the doc string for the method
		FString PythonStructMethodDocString = PyGenUtil::BuildFunctionDocString(InFunc, PythonStructMethodName, GeneratedWrappedDynamicMethod.MethodFunc.InputParams, GeneratedWrappedDynamicMethod.MethodFunc.OutputParams, &bIsStaticOverride);
		PythonStructMethodDocString += LINE_TERMINATOR;
		PythonStructMethodDocString += PyGenUtil::PythonizeFunctionTooltip(PyGenUtil::ParseTooltip(PyGenUtil::GetFieldTooltip(InFunc)), InFunc, ParamsToIgnore);
		GeneratedWrappedDynamicMethod.MethodDoc = PyGenUtil::TCHARToUTF8Buffer(*PythonStructMethodDocString);

		// Update the flags as removing the 'self' argument may have changed the calling convention
		GeneratedWrappedDynamicMethod.MethodFlags = GeneratedWrappedDynamicMethod.MethodFunc.InputParams.Num() > 0 ? METH_VARARGS | METH_KEYWORDS : METH_NOARGS;

		// Set the correct function pointer for calling this function and inject the 'self' argument
		GeneratedWrappedDynamicMethod.MethodCallback = nullptr;
		if (SelfParam.ParamProp->IsA<FObjectPropertyBase>())
		{
			GeneratedWrappedDynamicMethod.MethodCallback = GeneratedWrappedDynamicMethod.MethodFunc.InputParams.Num() > 0 ? PyCFunctionWithClosureCast(&FPyWrapperObject::CallDynamicMethodWithArgs_Impl) : PyCFunctionWithClosureCast(&FPyWrapperObject::CallDynamicMethodNoArgs_Impl);
		}
		else if (SelfParam.ParamProp->IsA<FStructProperty>())
		{
			GeneratedWrappedDynamicMethod.MethodCallback = GeneratedWrappedDynamicMethod.MethodFunc.InputParams.Num() > 0 ? PyCFunctionWithClosureCast(&FPyWrapperStruct::CallDynamicMethodWithArgs_Impl) : PyCFunctionWithClosureCast(&FPyWrapperStruct::CallDynamicMethodNoArgs_Impl);
		}

		// Add any deprecated variants too
		const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonStructMethodNames = PyGenUtil::GetDeprecatedScriptMethodPythonNames(InFunc);
		for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonStructMethodNamePair : DeprecatedPythonStructMethodNames)
		{
			const FString& DeprecatedPythonStructMethodName = DeprecatedPythonStructMethodNamePair.Value;
			FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonStructMethodName, *PythonStructMethodName);

			PyGenUtil::FGeneratedWrappedDynamicMethod& DeprecatedGeneratedWrappedMethod = DynamicMethodDefs.AddDefaulted_GetRef().Get();
			DeprecatedGeneratedWrappedMethod = GeneratedWrappedDynamicMethod;
			DeprecatedGeneratedWrappedMethod.MethodName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonStructMethodName);
			DeprecatedGeneratedWrappedMethod.MethodDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
			DeprecatedGeneratedWrappedMethod.MethodFunc.DeprecationMessage = MoveTemp(DeprecationMessage);
		}

		// Add the dynamic method to either the owner type
		if (SelfParam.ParamProp->IsA<FObjectPropertyBase>())
		{
			CallOnceUnlocked.Emplace(
				[this, SelfParam, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, DynamicMethodDefs = MoveTemp(DynamicMethodDefs), InFunc]() mutable
				{
					// Ensure that we've generated a finalized Python type for this class since we'll be adding this function as a dynamic method on that type
					const UClass* HostedClass = CastFieldChecked<const FObjectPropertyBase>(SelfParam.ParamProp)->PropertyClass;
					if (!GenerateWrappedClassType(HostedClass, OutGeneratedWrappedTypeReferences, OutDirtyModules, EPyTypeGenerationFlags::ForceShouldExport))
					{
						return;
					}

					{
						// Find the wrapped type for the class as that's what we'll actually add the dynamic method to
						TSharedPtr<PyGenUtil::FGeneratedWrappedClassType> HostedClassGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedClassType>(GeneratedWrappedTypes.FindRef(HostedClass));
						check(HostedClassGeneratedWrappedType.IsValid());

						UE::TUniqueLock ScopedLock(HostedClassGeneratedWrappedType->Lock);
						// Add the dynamic methods to the class
						for (TUniqueObj<PyGenUtil::FGeneratedWrappedDynamicMethod>& GeneratedWrappedDynamicMethodToAdd : DynamicMethodDefs)
						{
							HostedClassGeneratedWrappedType->FieldTracker.RegisterPythonFieldName(UTF8_TO_TCHAR(GeneratedWrappedDynamicMethodToAdd->MethodName.GetData()), InFunc);
							HostedClassGeneratedWrappedType->AddDynamicMethod(MoveTemp(GeneratedWrappedDynamicMethodToAdd.Get()));
						}
					}
				}
			);
		}
		else if (SelfParam.ParamProp->IsA<FStructProperty>())
		{
			CallOnceUnlocked.Emplace(
				[this, SelfParam, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, DynamicMethodDefs = MoveTemp(DynamicMethodDefs), InFunc]() mutable
				{
					// Ensure that we've generated a finalized Python type for this struct since we'll be adding this function as a dynamic method on that type
					const UScriptStruct* HostedStruct = CastFieldChecked<const FStructProperty>(SelfParam.ParamProp)->Struct;
					if (!GenerateWrappedStructType(HostedStruct, OutGeneratedWrappedTypeReferences, OutDirtyModules, EPyTypeGenerationFlags::ForceShouldExport))
					{
						return;
					}

					{
						// Find the wrapped type for the struct as that's what we'll actually add the dynamic method to
						TSharedPtr<PyGenUtil::FGeneratedWrappedStructType> HostedStructGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedStructType>(GeneratedWrappedTypes.FindRef(HostedStruct));
						check(HostedStructGeneratedWrappedType.IsValid());

						UE::TUniqueLock ScopedLock(HostedStructGeneratedWrappedType->Lock);
						// Add the dynamic methods to the struct
						for (TUniqueObj<PyGenUtil::FGeneratedWrappedDynamicMethod>& GeneratedWrappedDynamicMethodToAdd : DynamicMethodDefs)
						{
							HostedStructGeneratedWrappedType->FieldTracker.RegisterPythonFieldName(UTF8_TO_TCHAR(GeneratedWrappedDynamicMethodToAdd->MethodName.GetData()), InFunc);
							HostedStructGeneratedWrappedType->AddDynamicMethod(MoveTemp(GeneratedWrappedDynamicMethodToAdd.Get()));
						}
					}
				}
			);
		}
		else
		{
			checkf(false, TEXT("Unexpected SelfParam type!"));
		}
	};

	auto GenerateWrappedOperator = [this, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, &CallOnceUnlocked](const UFunction* InFunc, const PyGenUtil::FGeneratedWrappedMethod& InTypeMethod)
	{
		// Only static functions can be hoisted onto other types
		if (!InFunc->HasAnyFunctionFlags(FUNC_Static))
		{
			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Non-static function '%s.%s' is marked as 'ScriptOperator' but only static functions can be hoisted."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
			return;
		}

		// Get the list of operators to apply this function to
		TArray<FString> ScriptOperators;
		{
			const FString& ScriptOperatorsStr = InFunc->GetMetaData(PyGenUtil::ScriptOperatorMetaDataKey);
			ScriptOperatorsStr.ParseIntoArray(ScriptOperators, TEXT(";"));
		}

		// Go through and try and create a function for each operator, validating that the signature matches what the operator expects
		for (const FString& ScriptOperator : ScriptOperators)
		{
			PyGenUtil::FGeneratedWrappedOperatorSignature OpSignature;
			if (!PyGenUtil::FGeneratedWrappedOperatorSignature::StringToSignature(*ScriptOperator, OpSignature))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptOperator' but uses an unknown operator type '%s'."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName(), *ScriptOperator);
				continue;
			}

			PyGenUtil::FGeneratedWrappedOperatorFunction OpFunc;
			{
				FString SignatureError;
				if (!OpFunc.SetFunction(InTypeMethod.MethodFunc, OpSignature, &SignatureError))
				{
					REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptOperator' but has an invalid signature for the '%s' operator: %s."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName(), *ScriptOperator, *SignatureError);
					continue;
				}
			}

			CallOnceUnlocked.Emplace(
				[this, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, OpFunc, OpSignature]() mutable
				{
					// Ensure that we've generated a finalized Python type for this struct since we'll be adding this function as a operator on that type
					const UScriptStruct* HostedStruct = CastFieldChecked<const FStructProperty>(OpFunc.SelfParam.ParamProp)->Struct;
					if (GenerateWrappedStructType(HostedStruct, OutGeneratedWrappedTypeReferences, OutDirtyModules, EPyTypeGenerationFlags::ForceShouldExport))
					{
						// Find the wrapped type for the struct as that's what we'll actually add the operator to (via its meta-data)
						TSharedPtr<PyGenUtil::FGeneratedWrappedStructType> HostedStructGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedStructType>(GeneratedWrappedTypes.FindRef(HostedStruct));
						check(HostedStructGeneratedWrappedType.IsValid());

						UE::TUniqueLock ScopedLock(HostedStructGeneratedWrappedType->Lock);
						StaticCastSharedPtr<FPyWrapperStructMetaData>(HostedStructGeneratedWrappedType->MetaData)->OpStacks[(int32)OpSignature.OpType].Funcs.Add(MoveTemp(OpFunc));
					}
				}
			);
		}
	};

	auto GenerateWrappedConstant = [this, &GeneratedWrappedType, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, &CallOnceUnlocked](const UFunction* InFunc)
	{
		// Only static functions can be constants
		if (!InFunc->HasAnyFunctionFlags(FUNC_Static))
		{
			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Non-static function '%s.%s' is marked as 'ScriptConstant' but only static functions can be hoisted."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
			return;
		}

		// We might want to hoist this function onto another type rather than its owner class
		const UObject* HostType = nullptr;
		if (InFunc->HasMetaData(PyGenUtil::ScriptConstantHostMetaDataKey))
		{
			const FString ConstantOwnerName = InFunc->GetMetaData(PyGenUtil::ScriptConstantHostMetaDataKey);
			HostType = UClass::TryFindTypeSlow<UStruct>(ConstantOwnerName);
			if (HostType && !(HostType->IsA<UClass>() || HostType->IsA<UScriptStruct>()))
			{
				HostType = nullptr;
			}
			if (!HostType)
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptConstantHost' but the host '%s' could not be found."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName(), *ConstantOwnerName);
				return;
			}
		}
		if (const UClass* HostClass = Cast<UClass>(HostType))
		{
			if (HostClass->IsChildOf(InFunc->GetOwnerClass()))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptConstantHost' but the host type (%s) is a child of the class type of the static function. This is not allowed."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName(), *HostClass->GetPathName());
				return;
			}
		}

		// Verify that the function signature is valid
		PyGenUtil::FGeneratedWrappedFunction ConstantFunc;
		ConstantFunc.SetFunction(InFunc);
		if (ConstantFunc.InputParams.Num() != 0 || ConstantFunc.OutputParams.Num() != 1)
		{
			REPORT_PYTHON_GENERATION_ISSUE(Error, TEXT("Function '%s.%s' is marked as 'ScriptConstant' but has an invalid signature (it must return a value and take no arguments)."), *InFunc->GetOwnerClass()->GetPathName(), *InFunc->GetName());
			return;
		}

		const FString PythonConstantName = PyGenUtil::GetScriptConstantPythonName(InFunc);
		TArray<TUniqueObj<PyGenUtil::FGeneratedWrappedConstant>, TInlineAllocator<4>> ConstantDefs;

		// Build the constant definition
		PyGenUtil::FGeneratedWrappedConstant& GeneratedWrappedConstant = ConstantDefs.AddDefaulted_GetRef().Get();
		GeneratedWrappedConstant.ConstantName = PyGenUtil::TCHARToUTF8Buffer(*PythonConstantName);
		GeneratedWrappedConstant.ConstantDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("(%s): %s"), *PyGenUtil::GetPropertyPythonType(ConstantFunc.OutputParams[0].ParamProp), *PyGenUtil::GetFieldTooltip(InFunc)));
		GeneratedWrappedConstant.ConstantFunc = ConstantFunc;

		// Build any deprecated variants too
		const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonConstantNames = PyGenUtil::GetDeprecatedScriptConstantPythonNames(InFunc);
		for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonConstantNamePair : DeprecatedPythonConstantNames)
		{
			const FString& DeprecatedPythonConstantName = DeprecatedPythonConstantNamePair.Value;
			FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonConstantName, *PythonConstantName);

			PyGenUtil::FGeneratedWrappedConstant& DeprecatedGeneratedWrappedConstant = ConstantDefs.AddDefaulted_GetRef().Get();
			DeprecatedGeneratedWrappedConstant = GeneratedWrappedConstant;
			DeprecatedGeneratedWrappedConstant.ConstantName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonConstantName);
			DeprecatedGeneratedWrappedConstant.ConstantDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
		}

		// Add the constant to either the owner type (if specified) or this class
		if (HostType)
		{
			if (HostType->IsA<UClass>())
			{
				CallOnceUnlocked.Emplace(
					[this, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, HostType, ConstantDefs = MoveTemp(ConstantDefs), InFunc]() mutable
					{
						const UClass* HostClass = CastChecked<UClass>(HostType);

						// Ensure that we've generated a finalized Python type for this class since we'll be adding this constant to that type
						if (!GenerateWrappedClassType(HostClass, OutGeneratedWrappedTypeReferences, OutDirtyModules, EPyTypeGenerationFlags::ForceShouldExport))
						{
							return;
						}

						// Find the wrapped type for the class as that's what we'll actually add the constant to
						TSharedPtr<PyGenUtil::FGeneratedWrappedClassType> HostedClassGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedClassType>(GeneratedWrappedTypes.FindRef(HostClass));
						check(HostedClassGeneratedWrappedType.IsValid());

						UE::TUniqueLock ScopedLock(HostedClassGeneratedWrappedType->Lock);

						// Add the dynamic constants to the struct
						for (TUniqueObj<PyGenUtil::FGeneratedWrappedConstant>& GeneratedWrappedConstantToAdd : ConstantDefs)
						{
							HostedClassGeneratedWrappedType->FieldTracker.RegisterPythonFieldName(UTF8_TO_TCHAR(GeneratedWrappedConstantToAdd->ConstantName.GetData()), InFunc);
							HostedClassGeneratedWrappedType->AddDynamicConstant(MoveTemp(GeneratedWrappedConstantToAdd.Get()));
						}
					}
				);
			}
			else if (HostType->IsA<UScriptStruct>())
			{
				CallOnceUnlocked.Emplace(
					[this, &OutGeneratedWrappedTypeReferences, &OutDirtyModules, HostType, ConstantDefs = MoveTemp(ConstantDefs), InFunc]() mutable
					{
						const UScriptStruct* HostStruct = CastChecked<UScriptStruct>(HostType);

						// Ensure that we've generated a finalized Python type for this struct since we'll be adding this constant to that type
						if (!GenerateWrappedStructType(HostStruct, OutGeneratedWrappedTypeReferences, OutDirtyModules, EPyTypeGenerationFlags::ForceShouldExport))
						{
							return;
						}

						// Find the wrapped type for the struct as that's what we'll actually add the constant to
						TSharedPtr<PyGenUtil::FGeneratedWrappedStructType> HostedStructGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedStructType>(GeneratedWrappedTypes.FindRef(HostStruct));
						check(HostedStructGeneratedWrappedType.IsValid());

						UE::TUniqueLock ScopedLock(HostedStructGeneratedWrappedType->Lock);

						// Add the dynamic constants to the struct
						for (TUniqueObj<PyGenUtil::FGeneratedWrappedConstant>& GeneratedWrappedConstantToAdd : ConstantDefs)
						{
							HostedStructGeneratedWrappedType->FieldTracker.RegisterPythonFieldName(UTF8_TO_TCHAR(GeneratedWrappedConstantToAdd->ConstantName.GetData()), InFunc);
							HostedStructGeneratedWrappedType->AddDynamicConstant(MoveTemp(GeneratedWrappedConstantToAdd.Get()));
						}
					}
				);
			}
			else
			{
				checkf(false, TEXT("Unexpected HostType type!"));
			}
		}
		else
		{
			// Add the static constants to this type
			for (TUniqueObj<PyGenUtil::FGeneratedWrappedConstant>& GeneratedWrappedConstantToAdd : ConstantDefs)
			{
				GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(UTF8_TO_TCHAR(GeneratedWrappedConstantToAdd->ConstantName.GetData()), InFunc);
				GeneratedWrappedType->Constants.TypeConstants.Add(MoveTemp(GeneratedWrappedConstantToAdd.Get()));
			}
		}
	};

	auto GenerateWrappedMethod = [this, &PythonMethods, &PythonDeprecatedMethods, &GeneratedWrappedType, &OutGeneratedWrappedTypeReferences, &GenerateWrappedDynamicMethod, &GenerateWrappedOperator, &GenerateWrappedConstant](const UFunction* InFunc)
	{
		if (!PyGenUtil::ShouldExportFunction(InFunc))
		{
			return;
		}

		for (TFieldIterator<const FProperty> ParamIt(InFunc); ParamIt; ++ParamIt)
		{
			const FProperty* Param = *ParamIt;
			GatherWrappedTypesForPropertyReferences(Param, OutGeneratedWrappedTypeReferences);
		}

		// Constant functions do not export as real functions, so bail once we've generated their wrapped constant data
		if (InFunc->HasMetaData(PyGenUtil::ScriptConstantMetaDataKey))
		{
			GenerateWrappedConstant(InFunc);
			return;
		}

		const FString PythonFunctionName = PyGenUtil::GetFunctionPythonName(InFunc);
		const bool bIsStatic = InFunc->HasAnyFunctionFlags(FUNC_Static);
		
		PythonMethods.Add(*PythonFunctionName, InFunc->GetFName());

		PyGenUtil::FGeneratedWrappedMethod& GeneratedWrappedMethod = GeneratedWrappedType->Methods.TypeMethods.AddDefaulted_GetRef();
		GeneratedWrappedMethod.MethodName = PyGenUtil::TCHARToUTF8Buffer(*PythonFunctionName);
		GeneratedWrappedMethod.MethodFunc.SetFunction(InFunc);
		if (GeneratedWrappedMethod.MethodFunc.DeprecationMessage.IsSet())
		{
			PythonDeprecatedMethods.Add(*PythonFunctionName, GeneratedWrappedMethod.MethodFunc.DeprecationMessage.GetValue());
		}

		GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(PythonFunctionName, InFunc);

		FString FunctionDeclDocString = PyGenUtil::BuildFunctionDocString(InFunc, PythonFunctionName, GeneratedWrappedMethod.MethodFunc.InputParams, GeneratedWrappedMethod.MethodFunc.OutputParams);
		FunctionDeclDocString += LINE_TERMINATOR;
		FunctionDeclDocString += PyGenUtil::PythonizeFunctionTooltip(PyGenUtil::ParseTooltip(PyGenUtil::GetFieldTooltip(InFunc)), InFunc);

		GeneratedWrappedMethod.MethodDoc = PyGenUtil::TCHARToUTF8Buffer(*FunctionDeclDocString);
		GeneratedWrappedMethod.MethodFlags = GeneratedWrappedMethod.MethodFunc.InputParams.Num() > 0 ? METH_VARARGS | METH_KEYWORDS : METH_NOARGS;
		if (bIsStatic)
		{
			GeneratedWrappedMethod.MethodFlags |= METH_CLASS;
			GeneratedWrappedMethod.MethodCallback = GeneratedWrappedMethod.MethodFunc.InputParams.Num() > 0 ? PyCFunctionWithClosureCast(&FPyWrapperObject::CallClassMethodWithArgs_Impl) : PyCFunctionWithClosureCast(&FPyWrapperObject::CallClassMethodNoArgs_Impl);
		}
		else
		{
			GeneratedWrappedMethod.MethodCallback = GeneratedWrappedMethod.MethodFunc.InputParams.Num() > 0 ? PyCFunctionWithClosureCast(&FPyWrapperObject::CallMethodWithArgs_Impl) : PyCFunctionWithClosureCast(&FPyWrapperObject::CallMethodNoArgs_Impl);
		}

		// We must create a copy here because otherwise the reference will get invalidated by 
		// subsequent modifications

		const PyGenUtil::FGeneratedWrappedMethod GeneratedWrappedMethodCopy = GeneratedWrappedMethod;

		const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonFuncNames = PyGenUtil::GetDeprecatedFunctionPythonNames(InFunc);
		for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonFuncNamePair : DeprecatedPythonFuncNames)
		{
			const FString& DeprecatedPythonFuncName = DeprecatedPythonFuncNamePair.Value;
			FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonFuncName, *PythonFunctionName);
			PythonMethods.Add(*DeprecatedPythonFuncName, InFunc->GetFName());
			PythonDeprecatedMethods.Add(*DeprecatedPythonFuncName, DeprecationMessage);

			PyGenUtil::FGeneratedWrappedMethod DeprecatedGeneratedWrappedMethod = GeneratedWrappedMethodCopy;
			DeprecatedGeneratedWrappedMethod.MethodName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonFuncName);
			DeprecatedGeneratedWrappedMethod.MethodDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
			DeprecatedGeneratedWrappedMethod.MethodFunc.DeprecationMessage = MoveTemp(DeprecationMessage);
			GeneratedWrappedType->Methods.TypeMethods.Add(MoveTemp(DeprecatedGeneratedWrappedMethod));

			GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(DeprecatedPythonFuncName, InFunc);
		}

		// Should this function also be hoisted as a struct method or operator?
		if (InFunc->HasMetaData(PyGenUtil::ScriptMethodMetaDataKey))
		{
			GenerateWrappedDynamicMethod(InFunc, GeneratedWrappedMethodCopy);
		}
		if (InFunc->HasMetaData(PyGenUtil::ScriptOperatorMetaDataKey))
		{
			GenerateWrappedOperator(InFunc, GeneratedWrappedMethodCopy);
		}
	};

	const FString PythonClassName = PyGenUtil::GetClassPythonName(InClass);
	GeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*PythonClassName);

	for (TFieldIterator<const FProperty> FieldIt(InClass, EFieldIteratorFlags::ExcludeSuper); FieldIt; ++FieldIt)
	{
		const FProperty* Prop = *FieldIt;
		GenerateWrappedProperty(Prop);
	}

	for (TFieldIterator<const UFunction> FieldIt(InClass, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::IncludeDeprecated); FieldIt; ++FieldIt)
	{
		const UFunction* Func = *FieldIt;
		GenerateWrappedMethod(Func);
	}

	{
		TArray<const UClass*> Interfaces = PyGenUtil::GetExportedInterfacesForClass(InClass);
		for (const UClass* Interface : Interfaces)
		{
			for (TFieldIterator<const UFunction> FieldIt(Interface, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::IncludeDeprecated); FieldIt; ++FieldIt)
			{
				const UFunction* Func = *FieldIt;
				
				// Skip the interface function if the current class already has a reflected function with this name
				if (!InClass->FindFunctionByName(Func->GetFName(), EIncludeSuperFlag::ExcludeSuper))
				{
					GenerateWrappedMethod(Func);
				}
			}
		}
	}

	FString TypeDocString = PyGenUtil::PythonizeTooltip(PyGenUtil::ParseTooltip(PyGenUtil::GetFieldTooltip(InClass)));
	if (const UClass* SuperClass = InClass->GetSuperClass())
	{
		TSharedPtr<PyGenUtil::FGeneratedWrappedClassType> SuperGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedClassType>(GeneratedWrappedTypes.FindRef(SuperClass));
		if (SuperGeneratedWrappedType.IsValid())
		{
			GeneratedWrappedType->PropertyDocs.Append(SuperGeneratedWrappedType->PropertyDocs);
		}
	}
	GeneratedWrappedType->PropertyDocs.Sort(&PyGenUtil::FGeneratedWrappedPropertyDoc::SortPredicate);
	PyGenUtil::AppendSourceInformationDocString(InClass, TypeDocString);
	PyGenUtil::FGeneratedWrappedPropertyDoc::AppendDocString(GeneratedWrappedType->PropertyDocs, TypeDocString);
	GeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*TypeDocString);

	GeneratedWrappedType->PyType.tp_basicsize = sizeof(FPyWrapperObject);
	GeneratedWrappedType->PyType.tp_base = SuperPyType ? SuperPyType : &PyWrapperObjectType;
	GeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;

	TSharedRef<FPyWrapperObjectMetaData> ObjectMetaData = MakeShared<FPyWrapperObjectMetaData>();
	ObjectMetaData->Class = (UClass*)InClass;
	ObjectMetaData->PythonProperties = MoveTemp(PythonProperties);
	ObjectMetaData->PythonDeprecatedProperties = MoveTemp(PythonDeprecatedProperties);
	ObjectMetaData->PythonMethods = MoveTemp(PythonMethods);
	ObjectMetaData->PythonDeprecatedMethods = MoveTemp(PythonDeprecatedMethods);
	{
		FString DeprecationMessageStr;
		if (PyGenUtil::IsDeprecatedClass(InClass, &DeprecationMessageStr))
		{
			ObjectMetaData->DeprecationMessage = MoveTemp(DeprecationMessageStr);
		}
	}
	GeneratedWrappedType->MetaData = ObjectMetaData;

	if (GeneratedWrappedType->Finalize())
	{
		PyObject* PyModule = nullptr;
		const FName UnrealModuleName = *PyGenUtil::GetFieldModule(InClass);
		if (!UnrealModuleName.IsNone())
		{
			GeneratedWrappedTypesForModule.Add(UnrealModuleName, TypeRegistryName);
			OutDirtyModules.Add(UnrealModuleName);

			const FString PyModuleName = PyGenUtil::GetModulePythonName(UnrealModuleName);
			// Execute Python code within this block
			{
				FPyScopedGIL GIL;
				PyModule = PyImport_AddModule(TCHAR_TO_UTF8(*PyModuleName));

				Py_INCREF(&GeneratedWrappedType->PyType);
				PyModule_AddObject(PyModule, GeneratedWrappedType->PyType.tp_name, (PyObject*)&GeneratedWrappedType->PyType);
			}
		}
		RegisterWrappedClassType(TypeRegistryName, &GeneratedWrappedType->PyType, !bIsBlueprintGeneratedType);

		// Also generate and register any deprecated aliases for this type
		const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonClassNames = PyGenUtil::GetDeprecatedClassPythonNames(InClass);
		for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonClassNamePair : DeprecatedPythonClassNames)
		{
			const FString& DeprecatedPythonClassName = DeprecatedPythonClassNamePair.Value;
			FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonClassName, *PythonClassName);
			
			const FSoftObjectPath& DeprecatedTypeRegistryName = DeprecatedPythonClassNamePair.Key;
			if (GeneratedWrappedTypes.Contains(DeprecatedTypeRegistryName))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("Deprecated class name '%s' conflicted with an existing type!"), *DeprecatedTypeRegistryName.ToString());
				continue;
			}

			TSharedRef<PyGenUtil::FGeneratedWrappedClassType> DeprecatedGeneratedWrappedType = MakeShared<PyGenUtil::FGeneratedWrappedClassType>();
			GeneratedWrappedTypes.Add(DeprecatedTypeRegistryName, DeprecatedGeneratedWrappedType);

			DeprecatedGeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonClassName);
			DeprecatedGeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
			DeprecatedGeneratedWrappedType->PyType.tp_basicsize = sizeof(FPyWrapperObject);
			DeprecatedGeneratedWrappedType->PyType.tp_base = &GeneratedWrappedType->PyType;
			DeprecatedGeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;

			TSharedRef<FPyWrapperObjectMetaData> DeprecatedObjectMetaData = MakeShared<FPyWrapperObjectMetaData>(*ObjectMetaData);
			DeprecatedObjectMetaData->DeprecationMessage = MoveTemp(DeprecationMessage);
			DeprecatedGeneratedWrappedType->MetaData = DeprecatedObjectMetaData;

			if (DeprecatedGeneratedWrappedType->Finalize())
			{
				if (!UnrealModuleName.IsNone())
				{
					GeneratedWrappedTypesForModule.Add(UnrealModuleName, DeprecatedTypeRegistryName);
					// Execute Python code within this block
					{
						FPyScopedGIL GIL;
						check(PyModule);

						Py_INCREF(&DeprecatedGeneratedWrappedType->PyType);
						PyModule_AddObject(PyModule, DeprecatedGeneratedWrappedType->PyType.tp_name, (PyObject*)&DeprecatedGeneratedWrappedType->PyType);
					}
				}
				RegisterWrappedClassType(DeprecatedTypeRegistryName, &DeprecatedGeneratedWrappedType->PyType, !bIsBlueprintGeneratedType);
			}
			else
			{
				REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for deprecated class '%s'!"), *DeprecatedTypeRegistryName.ToString());
			}
		}

		GeneratedWrappedType->bIsReadyForUse = true;
		return &GeneratedWrappedType->PyType;
	}
	
	REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for class '%s'!"), *InClass->GetPathName());
	return nullptr;
}

void FPyWrapperTypeRegistry::RegisterWrappedClassType(const FSoftObjectPath& ClassName, PyTypeObject* PyType, const bool InDetectNameConflicts)
{
	if (InDetectNameConflicts)
{
		RegisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), ClassName);
	}
	PythonWrappedClasses.Add(ClassName, PyType);
}

void FPyWrapperTypeRegistry::UnregisterWrappedClassType(const FSoftObjectPath& ClassName, PyTypeObject* PyType, const bool InUnregisterName)
{
	if (InUnregisterName)
	{
		UnregisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), ClassName);
	}
	PythonWrappedClasses.Remove(ClassName);
}

bool FPyWrapperTypeRegistry::HasWrappedClassType(const UClass* InClass) const
{
	return PythonWrappedClasses.Contains(InClass);
}

PyTypeObject* FPyWrapperTypeRegistry::GetWrappedClassType(const UClass* InClass) const
{
	PyTypeObject* PyType = &PyWrapperObjectType;

	for (const UClass* Class = InClass; Class; Class = Class->GetSuperClass())
	{
		if (PyTypeObject* ClassPyType = PythonWrappedClasses.FindRef(Class))
		{
			PyType = ClassPyType;
			break;
		}
	}

	return PyType;
}

PyTypeObject* FPyWrapperTypeRegistry::GenerateWrappedStructType(const UScriptStruct* InStruct, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_GenerateWrappedStructTotalTime);
	INC_DWORD_STAT(STAT_GenerateWrappedStructCallCount);

	// Once we start generating types we can no longer register inline factories as they may affect the size of the generated Python objects
	bCanRegisterInlineStructFactories.store(false, std::memory_order_relaxed);

	struct FFuncs
	{
		static int Init(FPyWrapperStruct* InSelf, PyObject* InArgs, PyObject* InKwds)
		{
			const int SuperResult = PyWrapperStructType.tp_init((PyObject*)InSelf, InArgs, InKwds);
			if (SuperResult != 0)
			{
				return SuperResult;
			}

			return FPyWrapperStruct::MakeStruct(InSelf, InArgs, InKwds);
		}
	};

	// Already processed? Nothing more to do
	const FSoftObjectPath TypeRegistryName(InStruct);
	if (PyTypeObject* ExistingPyType = PythonWrappedStructs.FindRef(TypeRegistryName))
	{
		return ExistingPyType;
	}

	// todo: allow generation of Blueprint generated structs
	const bool bIsBlueprintGeneratedType = PyGenUtil::IsBlueprintGeneratedStruct(InStruct);
	if (bIsBlueprintGeneratedType)
	{
		return nullptr;
	}

	// Should this type be exported?
	if (!EnumHasAnyFlags(InGenerationFlags, EPyTypeGenerationFlags::ForceShouldExport) && !PyGenUtil::ShouldExportStruct(InStruct))
	{
		return nullptr;
	}

	// Make sure the parent struct is also wrapped
	PyTypeObject* SuperPyType = nullptr;
	if (const UScriptStruct* SuperStruct = Cast<UScriptStruct>(InStruct->GetSuperStruct()))
	{
		SuperPyType = GenerateWrappedStructType(SuperStruct, OutGeneratedWrappedTypeReferences, OutDirtyModules, InGenerationFlags | EPyTypeGenerationFlags::ForceShouldExport);
	}

	TSharedPtr<PyGenUtil::FGeneratedWrappedStructType> GeneratedWrappedType =
		StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedStructType, PyGenUtil::FGeneratedWrappedType>(
			GeneratedWrappedTypes.FindOrProduce(
				TypeRegistryName,
				[]()
				{
					return MakeShared<PyGenUtil::FGeneratedWrappedStructType>();
				}
			)
		);

	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedStructType);

	UE::TUniqueLock GeneratedWrappedTypeScopeLock(GeneratedWrappedType->Lock);
	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	INC_DWORD_STAT(STAT_GenerateWrappedStructObjCount);

	TMap<FName, FName> PythonProperties;
	TMap<FName, FString> PythonDeprecatedProperties;

	auto GenerateWrappedProperty = [this, &PythonProperties, &PythonDeprecatedProperties, &GeneratedWrappedType, &OutGeneratedWrappedTypeReferences](const FProperty* InProp)
	{
		const bool bExportPropertyToScript = PyGenUtil::ShouldExportProperty(InProp);
		const bool bExportPropertyToEditor = PyGenUtil::ShouldExportEditorOnlyProperty(InProp);

		if (bExportPropertyToScript || bExportPropertyToEditor)
		{
			GatherWrappedTypesForPropertyReferences(InProp, OutGeneratedWrappedTypeReferences);

			const PyGenUtil::FGeneratedWrappedPropertyDoc& GeneratedPropertyDoc = GeneratedWrappedType->PropertyDocs[GeneratedWrappedType->PropertyDocs.Emplace(InProp)];
			PythonProperties.Add(*GeneratedPropertyDoc.PythonPropName, InProp->GetFName());

			int32 GeneratedWrappedGetSetIndex = INDEX_NONE;
			if (bExportPropertyToScript)
			{
				GeneratedWrappedGetSetIndex = GeneratedWrappedType->GetSets.TypeGetSets.AddDefaulted();

				PyGenUtil::FGeneratedWrappedGetSet& GeneratedWrappedGetSet = GeneratedWrappedType->GetSets.TypeGetSets[GeneratedWrappedGetSetIndex];
				GeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*GeneratedPropertyDoc.PythonPropName);
				GeneratedWrappedGetSet.GetSetDoc = PyGenUtil::TCHARToUTF8Buffer(*GeneratedPropertyDoc.DocString);
				GeneratedWrappedGetSet.Prop.SetProperty(InProp);
				GeneratedWrappedGetSet.GetCallback = (getter)&FPyWrapperStruct::Getter_Impl;
				GeneratedWrappedGetSet.SetCallback = (setter)&FPyWrapperStruct::Setter_Impl;
				if (GeneratedWrappedGetSet.Prop.DeprecationMessage.IsSet())
				{
					PythonDeprecatedProperties.Add(*GeneratedPropertyDoc.PythonPropName, GeneratedWrappedGetSet.Prop.DeprecationMessage.GetValue());
				}

				GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(GeneratedPropertyDoc.PythonPropName, InProp);
			}

			const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonPropNames = PyGenUtil::GetDeprecatedPropertyPythonNames(InProp);
			for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonPropNamePair : DeprecatedPythonPropNames)
			{
				const FString& DeprecatedPythonPropName = DeprecatedPythonPropNamePair.Value;
				FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonPropName, *GeneratedPropertyDoc.PythonPropName);
				PythonProperties.Add(*DeprecatedPythonPropName, InProp->GetFName());
				PythonDeprecatedProperties.Add(*DeprecatedPythonPropName, DeprecationMessage);

				if (GeneratedWrappedGetSetIndex != INDEX_NONE)
				{
					PyGenUtil::FGeneratedWrappedGetSet DeprecatedGeneratedWrappedGetSet = GeneratedWrappedType->GetSets.TypeGetSets[GeneratedWrappedGetSetIndex];
					DeprecatedGeneratedWrappedGetSet.GetSetName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonPropName);
					DeprecatedGeneratedWrappedGetSet.GetSetDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
					DeprecatedGeneratedWrappedGetSet.Prop.DeprecationMessage = MoveTemp(DeprecationMessage);
					GeneratedWrappedType->GetSets.TypeGetSets.Add(MoveTemp(DeprecatedGeneratedWrappedGetSet));

					GeneratedWrappedType->FieldTracker.RegisterPythonFieldName(DeprecatedPythonPropName, InProp);
				}
			}
		}
	};

	const FString PythonStructName = PyGenUtil::GetStructPythonName(InStruct);
	GeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*PythonStructName);

	for (TFieldIterator<const FProperty> PropIt(InStruct, EFieldIteratorFlags::ExcludeSuper); PropIt; ++PropIt)
	{
		const FProperty* Prop = *PropIt;
		GenerateWrappedProperty(Prop);
	}

	FString TypeDocString = PyGenUtil::PythonizeTooltip(PyGenUtil::ParseTooltip(PyGenUtil::GetFieldTooltip(InStruct)));
	if (const UScriptStruct* SuperStruct = Cast<UScriptStruct>(InStruct->GetSuperStruct()))
	{
		TSharedPtr<PyGenUtil::FGeneratedWrappedStructType> SuperGeneratedWrappedType = StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedStructType>(GeneratedWrappedTypes.FindRef(SuperStruct));
		if (SuperGeneratedWrappedType.IsValid())
		{
			GeneratedWrappedType->PropertyDocs.Append(SuperGeneratedWrappedType->PropertyDocs);
		}
	}
	GeneratedWrappedType->PropertyDocs.Sort(&PyGenUtil::FGeneratedWrappedPropertyDoc::SortPredicate);
	PyGenUtil::AppendSourceInformationDocString(InStruct, TypeDocString);
	PyGenUtil::FGeneratedWrappedPropertyDoc::AppendDocString(GeneratedWrappedType->PropertyDocs, TypeDocString);
	GeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*TypeDocString);

	int32 WrappedStructSizeBytes = sizeof(FPyWrapperStruct);
	if (const IPyWrapperInlineStructFactory* InlineStructFactory = GetInlineStructFactory(InStruct->GetStructPathName()))
	{
		WrappedStructSizeBytes = InlineStructFactory->GetPythonObjectSizeBytes();
	}

	GeneratedWrappedType->PyType.tp_basicsize = WrappedStructSizeBytes;
	GeneratedWrappedType->PyType.tp_base = SuperPyType ? SuperPyType : &PyWrapperStructType;
	GeneratedWrappedType->PyType.tp_init = (initproc)&FFuncs::Init;
	GeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;

	TSharedRef<FPyWrapperStructMetaData> StructMetaData = MakeShared<FPyWrapperStructMetaData>();
	StructMetaData->Struct = (UScriptStruct*)InStruct;
	StructMetaData->PythonProperties = MoveTemp(PythonProperties);
	StructMetaData->PythonDeprecatedProperties = MoveTemp(PythonDeprecatedProperties);

	// If the struct has the 'HasNativeMake' meta and the function is found, set it, otherwise, postpone maybe its going to be loaded later.
	if (UFunction* MakeFunc = UE::Python::FindMakeBreakFunction(InStruct, PyGenUtil::HasNativeMakeMetaDataKey, PyGenUtil::ScriptDefaultMakeMetaDataKey, 
		[this, &StructMetaData](const FString& MakeFuncName){ UnresolvedMakeFuncs.Emplace(MakeFuncName, StructMetaData); }))
	{
		UE::Python::SetMakeFunction(*StructMetaData, MakeFunc);
	}

	// If the struct has the 'HasNativeBreak' meta and the function is found, set it, otherwise, postpone maybe its going to be loaded later.
	if (UFunction* BreakFunc = UE::Python::FindMakeBreakFunction(InStruct, PyGenUtil::HasNativeBreakMetaDataKey, PyGenUtil::ScriptDefaultBreakMetaDataKey, 
		[this, &StructMetaData](const FString& BreakFuncName) { UnresolvedBreakFuncs.Emplace(BreakFuncName, StructMetaData); }))
	{
		UE::Python::SetBreakFunction(*StructMetaData, BreakFunc);
	}

	// Build a complete list of init params for this struct (parent struct params + our params)
	if (SuperPyType)
	{
		FPyWrapperStructMetaData* SuperMetaData = FPyWrapperStructMetaData::GetMetaData(SuperPyType);
		if (SuperMetaData)
		{
			StructMetaData->InitParams = SuperMetaData->InitParams;
		}
	}
	for (const PyGenUtil::FGeneratedWrappedGetSet& GeneratedWrappedGetSet : GeneratedWrappedType->GetSets.TypeGetSets)
	{
		if (!GeneratedWrappedGetSet.Prop.DeprecationMessage.IsSet())
		{
			PyGenUtil::FGeneratedWrappedMethodParameter& GeneratedInitParam = StructMetaData->InitParams.AddDefaulted_GetRef();
			GeneratedInitParam.ParamName = GeneratedWrappedGetSet.GetSetName;
			GeneratedInitParam.ParamProp = GeneratedWrappedGetSet.Prop.Prop;
			GeneratedInitParam.ParamDefaultValue = FString();
		}
	}
	GeneratedWrappedType->MetaData = StructMetaData;

	if (GeneratedWrappedType->Finalize())
	{
		PyObject* PyModule = nullptr;
		const FName UnrealModuleName = *PyGenUtil::GetFieldModule(InStruct);
		if (!UnrealModuleName.IsNone())
		{
			GeneratedWrappedTypesForModule.Add(UnrealModuleName, TypeRegistryName);
			OutDirtyModules.Add(UnrealModuleName);

			const FString PyModuleName = PyGenUtil::GetModulePythonName(UnrealModuleName);
			// Execute Python code within this block
			{
				FPyScopedGIL GIL;
				PyModule = PyImport_AddModule(TCHAR_TO_UTF8(*PyModuleName));

				Py_INCREF(&GeneratedWrappedType->PyType);
				PyModule_AddObject(PyModule, GeneratedWrappedType->PyType.tp_name, (PyObject*)&GeneratedWrappedType->PyType);
			}
		}
		RegisterWrappedStructType(TypeRegistryName, &GeneratedWrappedType->PyType, !bIsBlueprintGeneratedType);

		// Also generate and register any deprecated aliases for this type
		const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonStructNames = PyGenUtil::GetDeprecatedStructPythonNames(InStruct);
		for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonStructNamePair : DeprecatedPythonStructNames)
		{
			const FString& DeprecatedPythonStructName = DeprecatedPythonStructNamePair.Value;
			FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonStructName, *PythonStructName);

			const FSoftObjectPath& DeprecatedTypeRegistryName = DeprecatedPythonStructNamePair.Key;
			if (GeneratedWrappedTypes.Contains(DeprecatedTypeRegistryName))
			{
				REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("Deprecated struct name '%s' conflicted with an existing type!"), *DeprecatedTypeRegistryName.ToString());
				continue;
			}

			TSharedRef<PyGenUtil::FGeneratedWrappedStructType> DeprecatedGeneratedWrappedType = MakeShared<PyGenUtil::FGeneratedWrappedStructType>();
			GeneratedWrappedTypes.Add(DeprecatedTypeRegistryName, DeprecatedGeneratedWrappedType);

			DeprecatedGeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonStructName);
			DeprecatedGeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
			DeprecatedGeneratedWrappedType->PyType.tp_basicsize = WrappedStructSizeBytes;
			DeprecatedGeneratedWrappedType->PyType.tp_base = &GeneratedWrappedType->PyType;
			DeprecatedGeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT | Py_TPFLAGS_BASETYPE;

			TSharedRef<FPyWrapperStructMetaData> DeprecatedStructMetaData = MakeShared<FPyWrapperStructMetaData>(*StructMetaData);
			DeprecatedStructMetaData->DeprecationMessage = MoveTemp(DeprecationMessage);
			DeprecatedGeneratedWrappedType->MetaData = DeprecatedStructMetaData;

			if (DeprecatedGeneratedWrappedType->Finalize())
			{
				if (!UnrealModuleName.IsNone())
				{
					GeneratedWrappedTypesForModule.Add(UnrealModuleName, DeprecatedTypeRegistryName);
					// Execute Python code within this block
					{
						FPyScopedGIL GIL;
						check(PyModule);

						Py_INCREF(&DeprecatedGeneratedWrappedType->PyType);
						PyModule_AddObject(PyModule, DeprecatedGeneratedWrappedType->PyType.tp_name, (PyObject*)&DeprecatedGeneratedWrappedType->PyType);
					}
				}
				RegisterWrappedStructType(DeprecatedTypeRegistryName, &DeprecatedGeneratedWrappedType->PyType, !bIsBlueprintGeneratedType);
			}
			else
			{
				REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for deprecated struct '%s'!"), *DeprecatedTypeRegistryName.ToString());
			}
		}

		GeneratedWrappedType->bIsReadyForUse = true;
		return &GeneratedWrappedType->PyType;
	}

	REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for struct '%s'!"), *InStruct->GetPathName());
	return nullptr;
}

void FPyWrapperTypeRegistry::RegisterWrappedStructType(const FSoftObjectPath& StructName, PyTypeObject* PyType, const bool InDetectNameConflicts)
{
	if (InDetectNameConflicts)
	{
		RegisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), StructName);
	}
	PythonWrappedStructs.Add(StructName, PyType);
}

void FPyWrapperTypeRegistry::UnregisterWrappedStructType(const FSoftObjectPath& StructName, PyTypeObject* PyType, const bool InUnregisterName)
{
	if (InUnregisterName)
	{
		UnregisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), StructName);
	}
	PythonWrappedStructs.Remove(StructName);
}

bool FPyWrapperTypeRegistry::HasWrappedStructType(const UScriptStruct* InStruct) const
{
	return PythonWrappedStructs.Contains(InStruct);
}

PyTypeObject* FPyWrapperTypeRegistry::GetWrappedStructType(const UScriptStruct* InStruct) const
{
	PyTypeObject* PyType = &PyWrapperStructType;

	for (const UScriptStruct* Struct = InStruct; Struct; Struct = Cast<UScriptStruct>(Struct->GetSuperStruct()))
	{
		if (PyTypeObject* StructPyType = PythonWrappedStructs.FindRef(InStruct))
		{
			PyType = StructPyType;
			break;
		}
	}

	return PyType;
}

PyTypeObject* FPyWrapperTypeRegistry::GenerateWrappedEnumType(const UEnum* InEnum, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_GenerateWrappedEnumTotalTime);
	INC_DWORD_STAT(STAT_GenerateWrappedEnumCallCount);

	bool bIsNewType = true;

	// Already processed? Nothing more to do
	const FSoftObjectPath TypeRegistryName(InEnum);
	if (PyTypeObject* ExistingPyType = PythonWrappedEnums.FindRef(TypeRegistryName))
	{
		if (!EnumHasAnyFlags(InGenerationFlags, EPyTypeGenerationFlags::OverwriteExisting))
		{
			return ExistingPyType;
		}
		bIsNewType = false;
	}

	// Should we allow Blueprint generated enums?
	const bool bIsBlueprintGeneratedType = PyGenUtil::IsBlueprintGeneratedEnum(InEnum);
	if (!EnumHasAnyFlags(InGenerationFlags, EPyTypeGenerationFlags::IncludeBlueprintGeneratedTypes) && bIsBlueprintGeneratedType)
	{
		return nullptr;
	}

	// Should this type be exported?
	if (!EnumHasAnyFlags(InGenerationFlags, EPyTypeGenerationFlags::ForceShouldExport) && !PyGenUtil::ShouldExportEnum(InEnum))
	{
		return nullptr;
	}

	INC_DWORD_STAT(STAT_GenerateWrappedEnumObjCount);

	TSharedPtr<PyGenUtil::FGeneratedWrappedEnumType> GeneratedWrappedType =
		StaticCastSharedPtr<PyGenUtil::FGeneratedWrappedEnumType, PyGenUtil::FGeneratedWrappedType>(
			GeneratedWrappedTypes.FindOrProduce(
				TypeRegistryName,
				[]()
				{
					return MakeShared<PyGenUtil::FGeneratedWrappedEnumType>();
				}
			)
		);

	if (!bIsNewType)
	{
		// this should never happen during the boot phase since we're multithreaded and we don't support
		// handling new type since all threads rely on forward progress from other threads.
		check(!UE::Python::Private::GIsInMultithreadedGeneration);
		GeneratedWrappedType->bIsReadyForUse = false;
		GeneratedWrappedType->Reset();
	}

	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedEnumType);

	UE::TUniqueLock GeneratedWrappedTypeScopeLock(GeneratedWrappedType->Lock);
	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	FString TypeDocString = PyGenUtil::PythonizeTooltip(PyGenUtil::ParseTooltip(PyGenUtil::GetFieldTooltip(InEnum)));
	PyGenUtil::AppendSourceInformationDocString(InEnum, TypeDocString);

	const FString PythonEnumName = PyGenUtil::GetEnumPythonName(InEnum);
	GeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*PythonEnumName);
	GeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*TypeDocString);
	GeneratedWrappedType->ExtractEnumEntries(InEnum);

	GeneratedWrappedType->PyType.tp_basicsize = sizeof(FPyWrapperEnum);
	GeneratedWrappedType->PyType.tp_base = &PyWrapperEnumType;
	GeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT;

	TSharedRef<FPyWrapperEnumMetaData> EnumMetaData = MakeShared<FPyWrapperEnumMetaData>();
	EnumMetaData->Enum = (UEnum*)InEnum;
	GeneratedWrappedType->MetaData = EnumMetaData;

	if (GeneratedWrappedType->Finalize())
	{
		if (bIsNewType)
		{
			PyObject* PyModule = nullptr;
			const FName UnrealModuleName = *PyGenUtil::GetFieldModule(InEnum);
			if (!UnrealModuleName.IsNone())
			{
				GeneratedWrappedTypesForModule.Add(UnrealModuleName, TypeRegistryName);
				OutDirtyModules.Add(UnrealModuleName);

				const FString PyModuleName = PyGenUtil::GetModulePythonName(UnrealModuleName);
				// Execute Python code within this block
				{
					FPyScopedGIL GIL;
					PyModule = PyImport_AddModule(TCHAR_TO_UTF8(*PyModuleName));

					Py_INCREF(&GeneratedWrappedType->PyType);
					PyModule_AddObject(PyModule, GeneratedWrappedType->PyType.tp_name, (PyObject*)&GeneratedWrappedType->PyType);
				}
			}
			RegisterWrappedEnumType(TypeRegistryName, &GeneratedWrappedType->PyType, !bIsBlueprintGeneratedType);

			// Also generate and register any deprecated aliases for this type
			const TArray<TTuple<FSoftObjectPath, FString>> DeprecatedPythonEnumNames = PyGenUtil::GetDeprecatedEnumPythonNames(InEnum);
			for (const TTuple<FSoftObjectPath, FString>& DeprecatedPythonEnumNamePair : DeprecatedPythonEnumNames)
			{
				const FString& DeprecatedPythonEnumName = DeprecatedPythonEnumNamePair.Value;
				FString DeprecationMessage = FString::Printf(TEXT("'%s' was renamed to '%s'."), *DeprecatedPythonEnumName, *PythonEnumName);

				const FSoftObjectPath& DeprecatedTypeRegistryName = DeprecatedPythonEnumNamePair.Key;
				if (GeneratedWrappedTypes.Contains(DeprecatedTypeRegistryName))
				{
					REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("Deprecated enum name '%s' conflicted with an existing type!"), *DeprecatedTypeRegistryName.ToString());
					continue;
				}

				TSharedRef<PyGenUtil::FGeneratedWrappedEnumType> DeprecatedGeneratedWrappedType = MakeShared<PyGenUtil::FGeneratedWrappedEnumType>();
				GeneratedWrappedTypes.Add(DeprecatedTypeRegistryName, DeprecatedGeneratedWrappedType);

				DeprecatedGeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*DeprecatedPythonEnumName);
				DeprecatedGeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*FString::Printf(TEXT("deprecated: %s"), *DeprecationMessage));
				DeprecatedGeneratedWrappedType->EnumEntries = GeneratedWrappedType->EnumEntries;
				DeprecatedGeneratedWrappedType->PyType.tp_basicsize = sizeof(FPyWrapperEnum);
				DeprecatedGeneratedWrappedType->PyType.tp_base = &PyWrapperEnumType;
				DeprecatedGeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT;

				TSharedRef<FPyWrapperEnumMetaData> DeprecatedEnumMetaData = MakeShared<FPyWrapperEnumMetaData>(*EnumMetaData);
				DeprecatedEnumMetaData->DeprecationMessage = MoveTemp(DeprecationMessage);
				DeprecatedGeneratedWrappedType->MetaData = DeprecatedEnumMetaData;

				if (DeprecatedGeneratedWrappedType->Finalize())
				{
					if (!UnrealModuleName.IsNone())
					{
						GeneratedWrappedTypesForModule.Add(UnrealModuleName, DeprecatedTypeRegistryName);
						// Execute Python code within this block
						{
							FPyScopedGIL GIL;
							check(PyModule);

							Py_INCREF(&DeprecatedGeneratedWrappedType->PyType);
							PyModule_AddObject(PyModule, DeprecatedGeneratedWrappedType->PyType.tp_name, (PyObject*)&DeprecatedGeneratedWrappedType->PyType);
						}
					}
					RegisterWrappedEnumType(DeprecatedTypeRegistryName, &DeprecatedGeneratedWrappedType->PyType, !bIsBlueprintGeneratedType);
				}
				else
				{
					REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for deprecated enum '%s'!"), *DeprecatedTypeRegistryName.ToString());
				}
			}
		}

		GeneratedWrappedType->bIsReadyForUse = true;
		return &GeneratedWrappedType->PyType;
	}

	REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for enum '%s'!"), *InEnum->GetPathName());
	return nullptr;
}

void FPyWrapperTypeRegistry::RegisterWrappedEnumType(const FSoftObjectPath& EnumName, PyTypeObject* PyType, const bool InDetectNameConflicts)
{
	if (InDetectNameConflicts)
{
		RegisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), EnumName);
	}
	PythonWrappedEnums.Add(EnumName, PyType);
}

void FPyWrapperTypeRegistry::UnregisterWrappedEnumType(const FSoftObjectPath& EnumName, PyTypeObject* PyType, const bool InUnregisterName)
{
	if (InUnregisterName)
	{
		UnregisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), EnumName);
	}
	PythonWrappedEnums.Remove(EnumName);
}

bool FPyWrapperTypeRegistry::HasWrappedEnumType(const UEnum* InEnum) const
{
	return PythonWrappedEnums.Contains(InEnum);
}

PyTypeObject* FPyWrapperTypeRegistry::GetWrappedEnumType(const UEnum* InEnum) const
{
	PyTypeObject* PyType = &PyWrapperEnumType;

	if (PyTypeObject* EnumPyType = PythonWrappedEnums.FindRef(InEnum))
	{
		PyType = EnumPyType;
	}

	return PyType;
}

PyTypeObject* FPyWrapperTypeRegistry::GenerateWrappedDelegateType(const UFunction* InDelegateSignature, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_GenerateWrappedDelegateTotalTime);
	INC_DWORD_STAT(STAT_GenerateWrappedDelegateCallCount);

	// Already processed? Nothing more to do
	const FSoftObjectPath TypeRegistryName(InDelegateSignature);
	if (PyTypeObject* ExistingPyType = PythonWrappedDelegates.FindRef(TypeRegistryName))
	{
		return ExistingPyType;
	}

	// Is this actually a delegate signature?
	if (!InDelegateSignature->HasAnyFunctionFlags(FUNC_Delegate))
	{
		return nullptr;
	}

	INC_DWORD_STAT(STAT_GenerateWrappedDelegateObjCount);

	TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType =
		GeneratedWrappedTypes.FindOrProduce(
			TypeRegistryName,
			[]()
			{
				return MakeShared<PyGenUtil::FGeneratedWrappedType>();
			}
		);

	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateWrappedDelegateType);

	UE::TUniqueLock GeneratedWrappedTypeScopeLock(GeneratedWrappedType->Lock);
	if (GeneratedWrappedType->bIsReadyForUse)
	{
		return &GeneratedWrappedType->PyType;
	}

	for (TFieldIterator<const FProperty> ParamIt(InDelegateSignature); ParamIt; ++ParamIt)
	{
		const FProperty* Param = *ParamIt;
		GatherWrappedTypesForPropertyReferences(Param, OutGeneratedWrappedTypeReferences);
	}

	FString TypeDocString = PyGenUtil::PythonizeFunctionTooltip(PyGenUtil::ParseTooltip(PyGenUtil::GetFieldTooltip(InDelegateSignature)), InDelegateSignature);
	PyGenUtil::AppendSourceInformationDocString(InDelegateSignature, TypeDocString);

	const FString DelegateBaseTypename = PyGenUtil::GetDelegatePythonName(InDelegateSignature);
	GeneratedWrappedType->TypeDoc = PyGenUtil::TCHARToUTF8Buffer(*TypeDocString);
	GeneratedWrappedType->PyType.tp_flags = Py_TPFLAGS_DEFAULT;

	// Two different UObject-based classes can each declare in their body a delegate with the same type name, but possibly
	// with different parameters. While they will share the same 'DelegateBaseTypename', they aren't necessarily the same type. Make
	// the name of the 'PythonCallableForDelegateClass' unique because NewObject<> will find pre-existing object with the same name and
	// return the same address, then the code below will basically erase the previous delegate type with the new delegate type, possibly
	// confusing the parameter types if they aren't the same.
	FNameBuilder NameBuilder;
	if (UClass* OuterClass = Cast<UClass>(InDelegateSignature->GetOuter()))
	{
		FString DelegateOuterClassName = PyGenUtil::GetClassPythonName(OuterClass);

		// The name of the UObject proxy wrapping the Python callable.
		NameBuilder += DelegateOuterClassName;
		NameBuilder += TEXT("_");
		NameBuilder += DelegateBaseTypename;

		// The name of the Python type. Note that we don't 'nest' in the pure Python sense. The type is flatten and looks like "unreal.OuterName_DelegateName"
		// A correctly nested type would require us to update the outer object Python data structure to mention it has an inner type which we don't.
		FNameBuilder TypenameBuilder;
		TypenameBuilder += DelegateOuterClassName;
		TypenameBuilder += TEXT("_"); // Cannot use "." because we don't really nest the object.
		TypenameBuilder += DelegateBaseTypename;
		GeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*TypenameBuilder);
	}
	else
	{
		NameBuilder += DelegateBaseTypename;
		GeneratedWrappedType->TypeName = PyGenUtil::TCHARToUTF8Buffer(*DelegateBaseTypename);
	}
	NameBuilder += TEXT("__PythonCallable");

	FName PythonCallableForDelegateObjectName = *NameBuilder;

	// This will be executed on the GameThread after the parallel initialization step
	auto CreatePythonCallableForDelegate_GameThread =
		[InDelegateSignature, PythonCallableForDelegateObjectName]()
		{
			// Generate the proxy class needed to wrap Python callables in Unreal delegates
			UClass* PythonCallableForDelegateClass = NewObject<UClass>(GetPythonTypeContainer(), PythonCallableForDelegateObjectName, RF_Public | RF_Transient);
			UFunction* PythonCallableForDelegateFunc = nullptr;
			{
				FObjectDuplicationParameters FuncDuplicationParams(const_cast<UFunction*>(InDelegateSignature), PythonCallableForDelegateClass);
				FuncDuplicationParams.DestName = UPythonCallableForDelegate::GeneratedFuncName;
				FuncDuplicationParams.InternalFlagMask &= ~EInternalObjectFlags::Native;
				PythonCallableForDelegateFunc = CastChecked<UFunction>(StaticDuplicateObjectEx(FuncDuplicationParams));
			}
			PythonCallableForDelegateFunc->FunctionFlags = (PythonCallableForDelegateFunc->FunctionFlags | FUNC_Native) & ~(FUNC_Delegate | FUNC_MulticastDelegate);
			PythonCallableForDelegateFunc->SetNativeFunc(&UPythonCallableForDelegate::CallPythonNative);
			PythonCallableForDelegateFunc->StaticLink(true);
			PythonCallableForDelegateClass->AddFunctionToFunctionMap(PythonCallableForDelegateFunc, PythonCallableForDelegateFunc->GetFName());
			PythonCallableForDelegateClass->SetSuperStruct(UPythonCallableForDelegate::StaticClass());
			PythonCallableForDelegateClass->ClassFlags |= CLASS_HideDropDown;
			PythonCallableForDelegateClass->Bind();
			PythonCallableForDelegateClass->StaticLink(true);
			PythonCallableForDelegateClass->AssembleReferenceTokenStream();
			PythonCallableForDelegateClass->GetDefaultObject();

			return PythonCallableForDelegateClass;
		};

	if (InDelegateSignature->HasAnyFunctionFlags(FUNC_MulticastDelegate))
	{
		GeneratedWrappedType->PyType.tp_basicsize = sizeof(FPyWrapperMulticastDelegate);
		GeneratedWrappedType->PyType.tp_base = &PyWrapperMulticastDelegateType;

		TSharedRef<FPyWrapperMulticastDelegateMetaData> DelegateMetaData = MakeShared<FPyWrapperMulticastDelegateMetaData>();
		DelegateMetaData->DelegateSignature.SetFunction(InDelegateSignature);
		GeneratedWrappedType->MetaData = DelegateMetaData;

		PyGenUtil::FMultithreadedGenerationContext::EnqueueGameThreadTask(
			[DelegateMetaData, CreatePythonCallableForDelegate_GameThread]()
			{
				DelegateMetaData->PythonCallableForDelegateClass = CreatePythonCallableForDelegate_GameThread();
			}
		);
	}
	else
	{
		GeneratedWrappedType->PyType.tp_basicsize = sizeof(FPyWrapperDelegate);
		GeneratedWrappedType->PyType.tp_base = &PyWrapperDelegateType;

		TSharedRef<FPyWrapperDelegateMetaData> DelegateMetaData = MakeShared<FPyWrapperDelegateMetaData>();
		DelegateMetaData->DelegateSignature.SetFunction(InDelegateSignature);
		GeneratedWrappedType->MetaData = DelegateMetaData;

		PyGenUtil::FMultithreadedGenerationContext::EnqueueGameThreadTask(
			[DelegateMetaData, CreatePythonCallableForDelegate_GameThread]()
			{
				DelegateMetaData->PythonCallableForDelegateClass = CreatePythonCallableForDelegate_GameThread();
			}
		);
	}

	if (GeneratedWrappedType->Finalize())
	{
		const FName UnrealModuleName = *PyGenUtil::GetFieldModule(InDelegateSignature);
		if (!UnrealModuleName.IsNone())
		{
			GeneratedWrappedTypesForModule.Add(UnrealModuleName, TypeRegistryName);
			OutDirtyModules.Add(UnrealModuleName);

			const FString PyModuleName = PyGenUtil::GetModulePythonName(UnrealModuleName);
			// Execute Python code within this block
			{
				FPyScopedGIL GIL;
				PyObject* PyModule = PyImport_AddModule(TCHAR_TO_UTF8(*PyModuleName));

				Py_INCREF(&GeneratedWrappedType->PyType);
				PyModule_AddObject(PyModule, GeneratedWrappedType->PyType.tp_name, (PyObject*)&GeneratedWrappedType->PyType);
			}
		}
		RegisterWrappedDelegateType(TypeRegistryName, &GeneratedWrappedType->PyType);

		GeneratedWrappedType->bIsReadyForUse = true;
		return &GeneratedWrappedType->PyType;
	}

	REPORT_PYTHON_GENERATION_ISSUE(Fatal, TEXT("Failed to generate Python glue code for delegate '%s'!"), *InDelegateSignature->GetPathName());
	return nullptr;
}

void FPyWrapperTypeRegistry::RegisterWrappedDelegateType(const FSoftObjectPath& DelegateName, PyTypeObject* PyType, const bool InDetectNameConflicts)
{
	if (InDetectNameConflicts)
	{
		RegisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), DelegateName);
	}
	PythonWrappedDelegates.Add(DelegateName, PyType);
}

void FPyWrapperTypeRegistry::UnregisterWrappedDelegateType(const FSoftObjectPath& DelegateName, PyTypeObject* PyType, const bool InUnregisterName)
{
	if (InUnregisterName)
	{
		UnregisterPythonTypeName(UTF8_TO_TCHAR(PyType->tp_name), DelegateName);
	}
	PythonWrappedDelegates.Remove(DelegateName);
}

bool FPyWrapperTypeRegistry::HasWrappedDelegateType(const UFunction* InDelegateSignature) const
{
	return PythonWrappedDelegates.Contains(InDelegateSignature);
}

PyTypeObject* FPyWrapperTypeRegistry::GetWrappedDelegateType(const UFunction* InDelegateSignature) const
{
	PyTypeObject* PyType = InDelegateSignature->HasAnyFunctionFlags(FUNC_MulticastDelegate) ? &PyWrapperMulticastDelegateType : &PyWrapperDelegateType;

	if (PyTypeObject* DelegatePyType = PythonWrappedDelegates.FindRef(InDelegateSignature))
	{
		PyType = DelegatePyType;
	}

	return PyType;
}

void FPyWrapperTypeRegistry::GatherWrappedTypesForPropertyReferences(const FProperty* InProp, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences) const
{
	if (const FObjectProperty* ObjProp = CastField<const FObjectProperty>(InProp))
	{
		if (ObjProp->PropertyClass && !PythonWrappedClasses.Contains(ObjProp->PropertyClass))
		{
			OutGeneratedWrappedTypeReferences.ClassReferences.Add(ObjProp->PropertyClass);
		}
		return;
	}

	if (const FStructProperty* StructProp = CastField<const FStructProperty>(InProp))
	{
		if (!PythonWrappedStructs.Contains(StructProp->Struct))
		{
			OutGeneratedWrappedTypeReferences.StructReferences.Add(StructProp->Struct);
		}
		return;
	}

	if (const FEnumProperty* EnumProp = CastField<const FEnumProperty>(InProp))
	{
		if (!PythonWrappedStructs.Contains(EnumProp->GetEnum()))
		{
			OutGeneratedWrappedTypeReferences.EnumReferences.Add(EnumProp->GetEnum());
		}
		return;
	}

	if (const FByteProperty* ByteProp = CastField<const FByteProperty>(InProp))
	{
		if (ByteProp->Enum)
		{
			if (!PythonWrappedStructs.Contains(ByteProp->Enum))
			{
				OutGeneratedWrappedTypeReferences.EnumReferences.Add(ByteProp->Enum);
			}
		}
		return;
	}

	if (const FDelegateProperty* DelegateProp = CastField<const FDelegateProperty>(InProp))
	{
		if (!PythonWrappedStructs.Contains(DelegateProp->SignatureFunction))
		{
			OutGeneratedWrappedTypeReferences.DelegateReferences.Add(DelegateProp->SignatureFunction);
		}
		return;
	}

	if (const FMulticastDelegateProperty* DelegateProp = CastField<const FMulticastDelegateProperty>(InProp))
	{
		if (!PythonWrappedStructs.Contains(DelegateProp->SignatureFunction))
		{
			OutGeneratedWrappedTypeReferences.DelegateReferences.Add(DelegateProp->SignatureFunction);
		}
		return;
	}

	if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(InProp))
	{
		GatherWrappedTypesForPropertyReferences(ArrayProp->Inner, OutGeneratedWrappedTypeReferences);
		return;
	}

	if (const FSetProperty* SetProp = CastField<const FSetProperty>(InProp))
	{
		GatherWrappedTypesForPropertyReferences(SetProp->ElementProp, OutGeneratedWrappedTypeReferences);
		return;
	}

	if (const FMapProperty* MapProp = CastField<const FMapProperty>(InProp))
	{
		GatherWrappedTypesForPropertyReferences(MapProp->KeyProp, OutGeneratedWrappedTypeReferences);
		GatherWrappedTypesForPropertyReferences(MapProp->ValueProp, OutGeneratedWrappedTypeReferences);
		return;
	}
}

void FPyWrapperTypeRegistry::GenerateStubCodeForWrappedTypes(const EPyOnlineDocsFilterFlags InDocGenFlags) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPyWrapperTypeRegistry::GenerateStubCodeForWrappedTypes)

	UE_LOGF(LogPython, Display, "Generating Python API stub file...");

	// Suspend the hang and hitch heartbeats, as this is a long running task.
	FSlowHeartBeatScope SuspendHeartBeat;
	FDisableHitchDetectorScope SuspendGameThreadHitch;

	FScopedSlowTask SlowTask(1, LOCTEXT("GeneratingPythonStubFile", "Generating unreal.py..."));
	SlowTask.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages
	SlowTask.MakeDialog();

	FPyScopedGIL GIL;
	FPyFileWriter PythonScript;

	if (PyGenUtil::IsTypeHintingEnabled())
	{
		// Must be at the top of the file. This allows type hinting using types not yet declared. Like Object, Text, Name, etc.
		PythonScript.WriteLine(TEXT("from __future__ import annotations")); // https://docs.python.org/3.7/whatsnew/3.7.html#pep-563-postponed-evaluation-of-annotations)
	}

	TUniquePtr<FPyOnlineDocsWriter> OnlineDocsWriter;
	TSharedPtr<FPyOnlineDocsModule> OnlineDocsUnrealModule;
	TSharedPtr<FPyOnlineDocsSection> OnlineDocsNativeTypesSection;
	TSharedPtr<FPyOnlineDocsSection> OnlineDocsEnumTypesSection;
	TSharedPtr<FPyOnlineDocsSection> OnlineDocsDelegateTypesSection;
	TSharedPtr<FPyOnlineDocsSection> OnlineDocsStructTypesSection;
	TSharedPtr<FPyOnlineDocsSection> OnlineDocsClassTypesSection;

	if (EnumHasAnyFlags(InDocGenFlags, EPyOnlineDocsFilterFlags::IncludeAll))
	{
		OnlineDocsWriter = MakeUnique<FPyOnlineDocsWriter>();
		OnlineDocsUnrealModule = OnlineDocsWriter->CreateModule(TEXT("unreal"));
		OnlineDocsNativeTypesSection = OnlineDocsWriter->CreateSection(TEXT("Native Types"));
		OnlineDocsStructTypesSection = OnlineDocsWriter->CreateSection(TEXT("Struct Types"));
		OnlineDocsClassTypesSection = OnlineDocsWriter->CreateSection(TEXT("Class Types"));
		OnlineDocsEnumTypesSection = OnlineDocsWriter->CreateSection(TEXT("Enum Types"));
		OnlineDocsDelegateTypesSection = OnlineDocsWriter->CreateSection(TEXT("Delegate Types"));

		UE::Python::SetGeneratingOnlineDoc(true);
	}

	// Process additional Python files
	// We split these up so that imports (excluding "unreal" imports) are listed at the top of the stub file
	// with the remaining code at the bottom (as it may depend on reflected APIs)
	TArray<FString> AdditionalPythonCode;
	{
		TArray<FName> ModuleNames;
		GeneratedWrappedTypesForModule.GetKeys(ModuleNames);
		ModuleNames.Sort(FNameLexicalLess());

		bool bExportedImports = false;
		for (const FName& ModuleName : ModuleNames)
		{
			const FString PythonBaseModuleName = PyGenUtil::GetModulePythonName(ModuleName, false);
			const FString PythonModuleName = FString::Printf(TEXT("unreal_%s"), *PythonBaseModuleName);

			FString ModuleFilename;
			if (PyUtil::IsModuleAvailableForImport(*PythonModuleName, &PyUtil::GetOnDiskUnrealModulesCache(), &ModuleFilename))
			{
				// Adjust .pyc and .pyd files so we try and find the source Python file
				ModuleFilename = FPaths::ChangeExtension(ModuleFilename, TEXT(".py"));
				if (FPaths::FileExists(ModuleFilename))
				{
					TArray<FString> PythonFile;
					if (FFileHelper::LoadFileToStringArray(PythonFile, *ModuleFilename))
					{
						// Process the file, looking for imports, and top-level classes and methods
						for (FString& PythonFileLine : PythonFile)
						{
							PythonFileLine.ReplaceInline(TEXT("\t"), TEXT("    "));
							
							// Write out each import line (excluding "unreal" imports)
							if (PythonFileLine.Contains(TEXT("import "), ESearchCase::CaseSensitive))
							{
								if (!PythonFileLine.Contains(TEXT("unreal"), ESearchCase::CaseSensitive))
								{
									bExportedImports = true;
									PythonScript.WriteLine(PythonFileLine.TrimStart()); // Trim leading spaces, import in .py file might be indented (lazy loaded in a if).
								}
								continue;
							}

							if (OnlineDocsUnrealModule.IsValid())
							{
								// Is this a top-level function?
								if (PythonFileLine.StartsWith(TEXT("def "), ESearchCase::CaseSensitive))
								{
									// Extract the function name
									FString FunctionName;
									for (const TCHAR* CharPtr = *PythonFileLine + 4; *CharPtr && *CharPtr != TEXT('('); ++CharPtr)
									{
										FunctionName += *CharPtr;
									}
									FunctionName.TrimStartAndEndInline();

									OnlineDocsUnrealModule->AccumulateFunction(*FunctionName);
								}
							}

							if (OnlineDocsNativeTypesSection.IsValid())
							{
								// Is this a top-level class?
								if (PythonFileLine.StartsWith(TEXT("class "), ESearchCase::CaseSensitive))
								{
									// Extract the class name
									FString ClassName;
									for (const TCHAR* CharPtr = *PythonFileLine + 6; *CharPtr && *CharPtr != TEXT('(') && *CharPtr != TEXT(':'); ++CharPtr)
									{
										ClassName += *CharPtr;
									}
									ClassName.TrimStartAndEndInline();

									OnlineDocsNativeTypesSection->AccumulateClass(*ClassName);
								}
							}

							// Stash any additional code so that we append it later
							AdditionalPythonCode.Add(MoveTemp(PythonFileLine));
						}
						AdditionalPythonCode.AddDefaulted(); // add an empty line after each file
					}
				}
			}
		}
		if (bExportedImports)
		{
			PythonScript.WriteNewLine();
		}

		if (PyGenUtil::IsTypeHintingEnabled())
		{
			PythonScript.WriteLine(TEXT("from typing import Any, Callable, Dict, ItemsView, Iterable, Iterator, KeysView, List, Mapping, MutableMapping, MutableSequence, MutableSet, Optional, Set, Sequence, Tuple, Type, TypeVar, Union, ValuesView"));
			PythonScript.WriteLine(TEXT("_T = TypeVar('_T')"));                 // For casting operations.
			PythonScript.WriteLine(TEXT("_ElemType = TypeVar('_ElemType')"));   // For Array, FixedArray, Set classes along with casting operations.
			PythonScript.WriteLine(TEXT("_KeyType = TypeVar('_KeyType')"));     // For Mapping[KV, KT] used by Map class.
			PythonScript.WriteLine(TEXT("_ValueType = TypeVar('_ValueType')")); // For Mapping[KV, KT] used by Map class.
			PythonScript.WriteLine(TEXT("_ValueType = TypeVar('_ValueType')")); // For Mapping[KV, KT] used by Map class.
			PythonScript.WriteLine(TEXT("_EngineSubsystemTypeVar = TypeVar('_EngineSubsystemTypeVar', bound='EngineSubsystem')"));
			PythonScript.WriteLine(TEXT("_EditorSubsystemTypeVar  = TypeVar('_EditorSubsystemTypeVar', bound='EditorSubsystem')"));
			PythonScript.WriteNewLine();
		}
	}

	TSet<FSoftObjectPath> GeneratedTypesToSkip;

	// Process native glue code (types)
	UE_LOGF(LogPython, Display, "  ...generating Python API: glue code (types)");
	PythonScript.WriteLine(TEXT("##### Glue Code #####"));
	PythonScript.WriteNewLine();

	for (const PyGenUtil::FNativePythonModule& NativePythonModule : NativePythonModules)
	{
		for (PyTypeObject* PyType : NativePythonModule.PyModuleTypes)
		{
			GenerateStubCodeForWrappedType(PyType, nullptr, PythonScript, OnlineDocsNativeTypesSection.Get());

			if (PyType == &PyWrapperEnumType)
			{
				// Include EPropertyAccessChangeNotifyMode in the native types, as it's needed by _ObjectBase and _StructBase (which are after EnumBase in PyModuleTypes)
				if (const UEnum* PropertyAccessChangeNotifyModeEnum = StaticEnum<EPropertyAccessChangeNotifyMode>())
				{
					FSoftObjectPath PropertyAccessChangeNotifyModeEnumPath = PropertyAccessChangeNotifyModeEnum;
					if (TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType = GeneratedWrappedTypes.FindRef(PropertyAccessChangeNotifyModeEnumPath))
					{
						GenerateStubCodeForWrappedType(&GeneratedWrappedType->PyType, GeneratedWrappedType.Get(), PythonScript, OnlineDocsEnumTypesSection.Get());
						GeneratedTypesToSkip.Add(MoveTemp(PropertyAccessChangeNotifyModeEnumPath));
					}
				}
			}
		}
	}

	// Process generated glue code
	// Also excludes types that don't pass the filters specified in InDocGenFlags using the information about
	// which module it came from and where that module exists on disk.
	auto ProcessWrappedDataArray = [this, &PythonScript, &GeneratedTypesToSkip, InDocGenFlags](const auto& WrappedData, const TSharedPtr<FPyOnlineDocsSection>& OnlineDocsSection)
	{
		if (OnlineDocsSection.IsValid())
		{
			UE_LOGF(LogPython, Display, "  ...generating Python API: %ls", *OnlineDocsSection->GetName());
			PythonScript.WriteLine(FString::Printf(TEXT("##### %s #####"), *OnlineDocsSection->GetName()));
			PythonScript.WriteNewLine();
		}
		
		FString ProjectTopDir;
		if (FPaths::IsProjectFilePathSet())
		{
			ProjectTopDir = FPaths::GetCleanFilename(FPaths::ProjectDir());
		}

		struct FSortableGeneratedWrappedType
		{
			FString TypePathStr;
			TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType;
			PyTypeObject* PyTypeObject = nullptr;
			int32 InheritanceDepth = 0;
		};

		TArray<FSortableGeneratedWrappedType> SortableWrappedDataArray;
		SortableWrappedDataArray.Reserve(WrappedData.Num());

		WrappedData.ForEach([this, InDocGenFlags, &ProjectTopDir, &OnlineDocsSection, &GeneratedTypesToSkip, &SortableWrappedDataArray](const auto& WrappedDataPair)
		{
			if (GeneratedTypesToSkip.Contains(WrappedDataPair.Key))
			{
				return;
			}

			TSharedPtr<PyGenUtil::FGeneratedWrappedType> GeneratedWrappedType = GeneratedWrappedTypes.FindRef(WrappedDataPair.Key);

			if ((InDocGenFlags != EPyOnlineDocsFilterFlags::IncludeAll) && GeneratedWrappedType.IsValid() && OnlineDocsSection.IsValid())
			{
				const UField* MetaType = GeneratedWrappedType->MetaData->GetMetaType();

				FString ModulePath;

				if (MetaType)
				{
					FSourceCodeNavigation::FindModulePath(MetaType->GetPackage(), ModulePath);
				}

				if (!ModulePath.IsEmpty())
				{
					// Is Project class?
					if (!ProjectTopDir.IsEmpty()
						&& (ModulePath.Contains(ProjectTopDir)))
					{
						// Optionally exclude Project classes
						if (!EnumHasAnyFlags(InDocGenFlags, EPyOnlineDocsFilterFlags::IncludeProject))
						{
							return;
						}
					}
					// Is Enterprise class
					else if (ModulePath.Contains(TEXT("/Enterprise/")))
					{
						// Optionally exclude Enterprise classes
						if (!EnumHasAnyFlags(InDocGenFlags, EPyOnlineDocsFilterFlags::IncludeEngine))
						{
							return;
						}
					}
					// is internal class
					else if (FPaths::IsRestrictedPath(ModulePath))
					{
						// Optionally exclude internal classes
						if (!EnumHasAnyFlags(InDocGenFlags, EPyOnlineDocsFilterFlags::IncludeInternal))
						{
							return;
						}
					}
					// Everything else is considered an "Engine" class
					else
					{
						// Optionally exclude engine classes
						if (!EnumHasAnyFlags(InDocGenFlags, EPyOnlineDocsFilterFlags::IncludeEngine))
						{
							return;
						}
					}
				}
				// else if cannot determine origin then include
			}

			int32 InheritanceDepth = 0;
			for (PyTypeObject* PyType = WrappedDataPair.Value; PyType; PyType = PyType->tp_base)
			{
				++InheritanceDepth;
			}

			SortableWrappedDataArray.Add({ WrappedDataPair.Key.ToString(), MoveTemp(GeneratedWrappedType), WrappedDataPair.Value, InheritanceDepth });
		});

		SortableWrappedDataArray.Sort([](const FSortableGeneratedWrappedType& One, const FSortableGeneratedWrappedType& Two)
		{
			// Primary sort by inheritance depth, so that parent classes appear before child classes
			if (One.InheritanceDepth < Two.InheritanceDepth)
			{
				return true;
			}
			if (One.InheritanceDepth > Two.InheritanceDepth)
			{
				return false;
			}

			// Secondary sort by path name, just for determinism
			return One.TypePathStr < Two.TypePathStr;
		});

		for (const FSortableGeneratedWrappedType& SortableWrappedData : SortableWrappedDataArray)
		{
			GenerateStubCodeForWrappedType(SortableWrappedData.PyTypeObject, SortableWrappedData.GeneratedWrappedType.Get(), PythonScript, OnlineDocsSection.Get());
		}
	};

	ProcessWrappedDataArray(PythonWrappedEnums, OnlineDocsEnumTypesSection);
	ProcessWrappedDataArray(PythonWrappedDelegates, OnlineDocsDelegateTypesSection);
	ProcessWrappedDataArray(PythonWrappedStructs, OnlineDocsStructTypesSection);
	ProcessWrappedDataArray(PythonWrappedClasses, OnlineDocsClassTypesSection);

	// Process native glue code (methods)
	UE_LOGF(LogPython, Display, "  ...generating Python API: glue code (methods)");
	PythonScript.WriteLine(TEXT("##### Glue Code #####"));
	PythonScript.WriteNewLine();

	for (const PyGenUtil::FNativePythonModule& NativePythonModule : NativePythonModules)
	{
		for (PyMethodDef* MethodDef = NativePythonModule.PyModuleMethods; MethodDef && MethodDef->ml_name; ++MethodDef)
		{
			// Try parsing the doc string to extract the function declaration with type hinting.
			FString MethodDecl = UE::Python::GenerateMethodDecl(MethodDef, TEXT("unreal"));
			PythonScript.WriteLine(MethodDecl);
			PythonScript.IncreaseIndent();
			PythonScript.WriteDocString(UTF8_TO_TCHAR(MethodDef->ml_doc));
			PythonScript.WriteLine(TEXT("...")); // pass
			PythonScript.DecreaseIndent();
			PythonScript.WriteNewLine();

			if (OnlineDocsUnrealModule.IsValid())
			{
				OnlineDocsUnrealModule->AccumulateFunction(UTF8_TO_TCHAR(MethodDef->ml_name));
			}
		}
	}

	// Append any additional Python code now that all the reflected API has been exported
	UE_LOGF(LogPython, Display, "  ...generating Python API: additional code");
	PythonScript.WriteLine(TEXT("##### Additional Code #####"));
	PythonScript.WriteNewLine();

	for (const FString& AdditionalPythonLine : AdditionalPythonCode)
	{
		PythonScript.WriteLine(AdditionalPythonLine);
	}

	const FString PythonSourceFilename = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir()) / TEXT("PythonStub") / TEXT("unreal.py");
	PythonScript.SaveFile(*PythonSourceFilename);
	UE_LOGF(LogPython, Display, "  ...generated: %ls", *PythonSourceFilename);

	if (OnlineDocsWriter.IsValid())
	{
		// Generate Sphinx files used to generate static HTML for Python API docs.
		OnlineDocsWriter->GenerateFiles(PythonSourceFilename);
	}
}

void FPyWrapperTypeRegistry::Shutdown()
{
	PythonWrappedTypeNameToUnrealTypeName.Empty();
	PythonWrappedClasses.Empty();
	PythonWrappedStructs.Empty();
	PythonWrappedEnums.Empty();
	PythonWrappedDelegates.Empty();
	GeneratedWrappedTypes.Empty();
	GeneratedWrappedTypesForModule.Empty();
	UnresolvedMakeFuncs.Empty();
	UnresolvedBreakFuncs.Empty();
	OrphanedWrappedTypes.Empty();
	NativePythonModules.Empty();
	OnModuleDirtiedDelegate.Clear();
	InlineStructFactories.Empty();

	bCanRegisterInlineStructFactories.store(true);
}

void FPyWrapperTypeRegistry::GenerateStubCodeForWrappedType(PyTypeObject* PyType, const PyGenUtil::FGeneratedWrappedType* GeneratedTypeData, FPyFileWriter& OutPythonScript, FPyOnlineDocsSection* OutOnlineDocsSection)
{
	const FString PyTypeName = UTF8_TO_TCHAR(PyType->tp_name);

	// If type hinting is configured for IDE auto-completion:
	//     - Be nice to the user, don't bother them with type coercion (typing.Union) or None type validation (typing.Optional), show the raw Unreal types.
	//     - Be nice to the user, don't ellipse non-parsing default values.
	// 
	// If type hinting is configure for type checking:
	//    - Support known type coercions for method parameters (typing.Union[])
	//    - Tag Object-like object as potentially 'None' using typing.Optional[]. We cannot know when an Object-like an input parameter or return value can be None,
	//      so opt for the safe side and mark them all as Optional. That's might mislead users into thinking that some parameter/values can be None while in reality, they can't.
	//    - Don't coerce returned type. Our API which largely reflect C++ returns one defined type, not something like Union[Name, str]
	//    - Returned unreal.Array/unreal.Set/unreal.Map are a bit annoying to hint. For example, Array[Name] is hinted to accepts Name
	//      param for example 'array.append("a name")' will fail type checker even though this works. User will have to do 'a.append(Name("my name"))'
	//    - If an API returns an unreal.Array[T]/unreal.Set[T]/unreal.Map[T] and T is a type that can None such as an unreal.Array[unreal.Object],
	//      then the return type will be Array[Optional[Object]] to allow having None type in there, which is also a bit annoying to work with.
	//
	// If type hinting is off:
	//    - Use strict typing for return values to aid auto-complete (we also only care about the type and
	//      not the value, so structs can be default constructed)
	//
	// Always default construct date as default value to avoid setting the generated date as the default value.

	const PyGenUtil::EPythonizeFlags PythonizeMethodParamFlags = PyGenUtil::IsTypeHintingEnabled()
		? (PyGenUtil::GetTypeHintingMode() == ETypeHintingMode::AutoCompletion
			? PyGenUtil::EPythonizeFlags::DefaultConstructDateTime | PyGenUtil::EPythonizeFlags::WithTypeHinting // For IDE auto-completion.
			: PyGenUtil::EPythonizeFlags::DefaultConstructDateTime | PyGenUtil::EPythonizeFlags::WithTypeHinting | PyGenUtil::EPythonizeFlags::WithTypeCoercion | PyGenUtil::EPythonizeFlags::WithOptionalType) // For type checker.
		: PyGenUtil::EPythonizeFlags::DefaultConstructDateTime; // hinting is off.

	const PyGenUtil::EPythonizeFlags PythonizeMethodReturnTypeFlags = PyGenUtil::IsTypeHintingEnabled()
		? (PyGenUtil::GetTypeHintingMode() == ETypeHintingMode::AutoCompletion
			? PyGenUtil::EPythonizeFlags::WithTypeHinting  // For IDE auto-completion.
			: PyGenUtil::EPythonizeFlags::WithTypeHinting | PyGenUtil::EPythonizeFlags::WithOptionalType) // For type checker.
		: PyGenUtil::EPythonizeFlags::DefaultConstructStructs | PyGenUtil::EPythonizeFlags::DefaultConstructDateTime | PyGenUtil::EPythonizeFlags::UseStrictTyping; // hinting is off.

	if (PyGenUtil::IsTypeHintingEnabled() &&  PyTypeName == TEXT("Array"))
	{
		// Implies that _ElemType = TypeVar('_ElemType') is already written to the file.
		OutPythonScript.WriteLine(FString::Printf(TEXT("class %s(%s, MutableSequence[_ElemType]):"), *PyTypeName, UTF8_TO_TCHAR(PyType->tp_base->tp_name)));
	}
	else if (PyGenUtil::IsTypeHintingEnabled() && PyTypeName == TEXT("FixedArray"))
	{
		// Implies that _ElemType = TypeVar('_ElemType') is already written to the file.
		OutPythonScript.WriteLine(FString::Printf(TEXT("class %s(%s, Sequence[_ElemType]):"), *PyTypeName, UTF8_TO_TCHAR(PyType->tp_base->tp_name)));
	}
	else if (PyGenUtil::IsTypeHintingEnabled() && PyTypeName == TEXT("Set"))
	{
		// Implies that _ElemType = TypeVar('_ElemType') is already written to the file.
		OutPythonScript.WriteLine(FString::Printf(TEXT("class %s(%s, MutableSet[_ElemType]):"), *PyTypeName, UTF8_TO_TCHAR(PyType->tp_base->tp_name)));
	}
	else if (PyGenUtil::IsTypeHintingEnabled() && PyTypeName == TEXT("Map"))
	{
		// Implies that: _KeyType = TypeVar('_KeyType') and _ValueType = TypeVar('_ValueType') are already written to the file.
		OutPythonScript.WriteLine(FString::Printf(TEXT("class %s(%s, MutableMapping[_KeyType, _ValueType]):"), *PyTypeName, UTF8_TO_TCHAR(PyType->tp_base->tp_name)));
	}
	else
	{
		OutPythonScript.WriteLine(FString::Printf(TEXT("class %s(%s):"), *PyTypeName, UTF8_TO_TCHAR(PyType->tp_base->tp_name)));
	}

	OutPythonScript.IncreaseIndent();
	OutPythonScript.WriteDocString(UTF8_TO_TCHAR(PyType->tp_doc));

	if (OutOnlineDocsSection)
	{
		OutOnlineDocsSection->AccumulateClass(*PyTypeName);
	}

	auto GetFunctionReturnValue = [](const void* InBaseParamsAddr, const TArray<PyGenUtil::FGeneratedWrappedMethodParameter>& InOutputParams) -> FString
	{
		if (InOutputParams.Num() == 0)
		{
			return TEXT("None");
		}
		
		// We use strict typing for return values to aid auto-complete (we also only care about the type and not the value, so structs can be default constructed)
		static const PyGenUtil::EPythonizeFlags PythonizeValueFlags = PyGenUtil::EPythonizeFlags::UseStrictTyping | PyGenUtil::EPythonizeFlags::DefaultConstructStructs;

		// If we have multiple return values and the main return value is a bool, skip it (to mimic PyGenUtils::PackReturnValues)
		int32 ReturnPropIndex = 0;
		if (InOutputParams.Num() > 1 && InOutputParams[0].ParamProp->HasAnyPropertyFlags(CPF_ReturnParm) && InOutputParams[0].ParamProp->IsA<FBoolProperty>())
		{
			ReturnPropIndex = 1; // Start packing at the 1st out value
		}

		// Do we need to return a packed tuple, or just a single value?
		const int32 NumPropertiesToPack = InOutputParams.Num() - ReturnPropIndex;
		if (NumPropertiesToPack == 1)
		{
			const PyGenUtil::FGeneratedWrappedMethodParameter& ReturnParam = InOutputParams[ReturnPropIndex];
			return PyGenUtil::PythonizeValue(ReturnParam.ParamProp, ReturnParam.ParamProp->ContainerPtrToValuePtr<void>(InBaseParamsAddr), PythonizeValueFlags);
		}
		else // Returns a tuple
		{
			FString FunctionReturnStr = TEXT("(");
			for (int32 PackedPropIndex = 0; ReturnPropIndex < InOutputParams.Num(); ++ReturnPropIndex, ++PackedPropIndex)
			{
				if (PackedPropIndex > 0)
				{
					FunctionReturnStr += TEXT(", ");
				}
				const PyGenUtil::FGeneratedWrappedMethodParameter& ReturnParam = InOutputParams[ReturnPropIndex];
				FunctionReturnStr += PyGenUtil::PythonizeValue(ReturnParam.ParamProp, ReturnParam.ParamProp->ContainerPtrToValuePtr<void>(InBaseParamsAddr), PythonizeValueFlags);
			}
			FunctionReturnStr += TEXT(")");
			return FunctionReturnStr;
		}
	};

	auto ExportConstant = [&OutPythonScript, &PyTypeName](const TCHAR* InConstantName, const FString& InConstantType, const FString& InConstantValue, const TCHAR* InConstantDocString)
	{
		FString ConstantValue(InConstantValue);

		// Ensure that constant type is not same type as host type
		if (ConstantValue.StartsWith(PyTypeName, ESearchCase::CaseSensitive) && (ConstantValue[PyTypeName.Len()] == TEXT('(')))
		{
			ConstantValue = PyGenUtil::IsTypeHintingEnabled() ? TEXT("") : TEXT("None");
		}

		if (*InConstantDocString == 0)
		{
			// No docstring
			ConstantValue.IsEmpty()
				? OutPythonScript.WriteLine(FString::Printf(TEXT("%s%s"), InConstantName, InConstantType.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(": %s"), *InConstantType)))
				: OutPythonScript.WriteLine(FString::Printf(TEXT("%s%s = %s"), InConstantName, InConstantType.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(": %s"), *InConstantType), *ConstantValue));
			
		}
		else
		{
			if (FCString::Strchr(InConstantDocString, TEXT('\n')))
			{
				// Multi-line docstring
				ConstantValue.IsEmpty()
					? OutPythonScript.WriteLine(FString::Printf(TEXT("%s%s"), InConstantName, InConstantType.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(": %s"), *InConstantType)))
					: OutPythonScript.WriteLine(FString::Printf(TEXT("%s%s = %s"), InConstantName, InConstantType.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(": %s"), *InConstantType), *ConstantValue));

				OutPythonScript.WriteDocString(InConstantDocString);
				OutPythonScript.WriteNewLine();
			}
			else
			{
				// Single-line docstring
				ConstantValue.IsEmpty()
					? OutPythonScript.WriteLine(FString::Printf(TEXT("%s%s #: %s"), InConstantName, InConstantType.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(": %s"), *InConstantType), InConstantDocString))
					: OutPythonScript.WriteLine(FString::Printf(TEXT("%s%s = %s #: %s"), InConstantName, InConstantType.IsEmpty() ? TEXT("") : *FString::Printf(TEXT(": %s"), *InConstantType), *ConstantValue, InConstantDocString));
			}
		}
	};

	auto ExportGetSet = [&OutPythonScript](const TCHAR* InGetSetMethodName, const TCHAR* InGetSetDocString, const TCHAR* InGetterReturnValue, const TCHAR* InGetterReturnType, const TCHAR* InSetterParam, const bool InReadOnly)
	{
		// Getter
		OutPythonScript.WriteLine(TEXT("@property"));
		OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(self)%s%s:"), InGetSetMethodName, *InGetterReturnType != 0 ? TEXT(" -> ") : TEXT(""), InGetterReturnType));
		OutPythonScript.IncreaseIndent();
		OutPythonScript.WriteDocString(InGetSetDocString);
		PyGenUtil::IsTypeHintingEnabled()
			? OutPythonScript.WriteLine(TEXT("...")) // No need to add a return value to help auto-complete/type checker when the type is hinted in the signature.
			: OutPythonScript.WriteLine(FString::Printf(TEXT("return %s"), InGetterReturnValue));
		OutPythonScript.DecreaseIndent();

		if (!InReadOnly)
		{
			// Setter
			OutPythonScript.WriteLine(FString::Printf(TEXT("@%s.setter"), InGetSetMethodName));
			PyGenUtil::IsTypeHintingEnabled()
				? OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(self, %s) -> None:"), InGetSetMethodName, *InSetterParam != 0 ? InSetterParam : TEXT("value")))
				: OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(self, value):"), InGetSetMethodName));
			OutPythonScript.IncreaseIndent();
			OutPythonScript.WriteLine(TEXT("...")); // pass
			OutPythonScript.DecreaseIndent();
		}
	};

	auto ExportGeneratedMethod = [PythonizeMethodParamFlags, PythonizeMethodReturnTypeFlags, &OutPythonScript, &GetFunctionReturnValue](const PyGenUtil::FGeneratedWrappedMethod& InTypeMethod)
	{
		FString MethodArgsStr;
		FString FirstDefaultedParamName;
		for (const PyGenUtil::FGeneratedWrappedMethodParameter& MethodParam : InTypeMethod.MethodFunc.InputParams)
		{
			MethodArgsStr += TEXT(", ");
			if (MethodParam.ParamDefaultValue.IsSet())
			{
				MethodArgsStr += PyGenUtil::PythonizeMethodParam(MethodParam, PythonizeMethodParamFlags, UE::Python::ShouldEllipseParamDefaultValue);
				if (FirstDefaultedParamName.IsEmpty())
				{
					FirstDefaultedParamName = MethodParam.ParamProp->GetName();
				}
			}
			else if (!FirstDefaultedParamName.IsEmpty())
			{
				// Some UFUNCTION has their default values declared as 'meta' invisible to C++ compiler, but Python generated glue will use that. Sometime, the next parameter(s) are
				// not defaulted. This creates invalid Python declarations. In stub, this is legal to use "..." to indicate a default param value without specifying the exact value.
				// That's a workaround for faulty method declarations and the C++ code should be fixed.
				MethodArgsStr += PyGenUtil::PythonizeMethodParam(MethodParam, PythonizeMethodParamFlags | PyGenUtil::EPythonizeFlags::AddMissingDefaultValueEllipseWorkaround);

				// Left as 'Verbose' until we fix known cases. Once fixed, we should turn this into a 'Warning'.
				REPORT_PYTHON_GENERATION_ISSUE(Verbose, TEXT("The function '%s' is missing default values for param '%s'. All params after the first defaulted param '%s' (likely from meta data) should be defaulted. The Python stub defaulted missing default values."), 
					InTypeMethod.MethodFunc.Func != nullptr ? *InTypeMethod.MethodFunc.Func->GetPathName() : TEXT("Unknown"),
					MethodParam.ParamProp != nullptr ? *MethodParam.ParamProp->GetName() : TEXT("Unknown"),
					*FirstDefaultedParamName);
			}
			else
			{
				MethodArgsStr += PyGenUtil::PythonizeMethodParam(MethodParam, PythonizeMethodParamFlags, UE::Python::ShouldEllipseParamDefaultValue);
			}
		}

		// Either the type or the value, depending if type hinting is on/off.
		FString MethodReturnValueStr;
		FString MethodReturnType;
		if (InTypeMethod.MethodFunc.Func)
		{
			if (PyGenUtil::IsTypeHintingEnabled())
			{
				MethodReturnType = PyGenUtil::PythonizeMethodReturnType(InTypeMethod.MethodFunc.OutputParams, PythonizeMethodReturnTypeFlags);
			}
			else
			{
				PY_UFUNCTION_STACK(FuncParams, InTypeMethod.MethodFunc.Func);
				MethodReturnValueStr = GetFunctionReturnValue(FuncParams.GetMemory(), InTypeMethod.MethodFunc.OutputParams);
			}
		}
		else
		{
			MethodReturnType = TEXT("None");
			MethodReturnValueStr = TEXT("None");
		}

		const bool bIsClassMethod = !!(InTypeMethod.MethodFlags & METH_CLASS);
		if (bIsClassMethod)
		{
			OutPythonScript.WriteLine(TEXT("@classmethod"));
			OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(cls%s)%s%s:"), UTF8_TO_TCHAR(InTypeMethod.MethodName.GetData()), *MethodArgsStr, !MethodReturnType.IsEmpty() ? TEXT(" -> ") : TEXT(""), *MethodReturnType));
		}
		else
		{
			OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(self%s)%s%s:"), UTF8_TO_TCHAR(InTypeMethod.MethodName.GetData()), *MethodArgsStr, !MethodReturnType.IsEmpty() ? TEXT(" -> ") : TEXT(""), *MethodReturnType));
		}
		OutPythonScript.IncreaseIndent();
		OutPythonScript.WriteDocString(UTF8_TO_TCHAR(InTypeMethod.MethodDoc.GetData()));
		PyGenUtil::IsTypeHintingEnabled()
			? OutPythonScript.WriteLine(TEXT("...")) // Omit the return value, the return type is hinted in the method signature.
			: OutPythonScript.WriteLine(FString::Printf(TEXT("return %s"), *MethodReturnValueStr));
		OutPythonScript.DecreaseIndent();
	};

	auto ExportGeneratedConstant = [PythonizeMethodReturnTypeFlags, &ExportConstant, &GetFunctionReturnValue](const PyGenUtil::FGeneratedWrappedConstant& InTypeConstant)
	{
		// Resolve the constant value
		FString ConstantValueStr;
		FString ConstantType;
		if (InTypeConstant.ConstantFunc.Func && InTypeConstant.ConstantFunc.InputParams.Num() == 0)
		{
			UClass* Class = InTypeConstant.ConstantFunc.Func->GetOwnerClass();
			UObject* Obj = Class->GetDefaultObject();

			PY_UFUNCTION_STACK(FuncParams, InTypeConstant.ConstantFunc.Func);
			PyUtil::InvokeFunctionCall(Obj, InTypeConstant.ConstantFunc.Func, FuncParams.GetMemory(), TEXT("export generated constant"));
			PyErr_Clear(); // Clear any errors in case InvokeFunctionCall failed

			if (PyGenUtil::IsTypeHintingEnabled())
			{
				// If type hinting is enabled, just export the constant name and type.
				ConstantType = PyGenUtil::PythonizeMethodReturnType(InTypeConstant.ConstantFunc.OutputParams, PythonizeMethodReturnTypeFlags);
			}
			else
			{
				ConstantValueStr = GetFunctionReturnValue(FuncParams.GetMemory(), InTypeConstant.ConstantFunc.OutputParams);
			}
		}
		else // Cannot resove type/value.
		{
			ConstantType = TEXT("object");
			ConstantValueStr = TEXT("None");
		}
		ExportConstant(UTF8_TO_TCHAR(InTypeConstant.ConstantName.GetData()), ConstantType, ConstantValueStr, UTF8_TO_TCHAR(InTypeConstant.ConstantDoc.GetData()));
	};

	auto ExportGeneratedGetSet = [PythonizeMethodParamFlags, PythonizeMethodReturnTypeFlags, &ExportGetSet](const PyGenUtil::FGeneratedWrappedGetSet& InGetSet)
	{
		const bool bIsReadOnly = InGetSet.Prop.Prop->HasAnyPropertyFlags(PropertyAccessUtil::RuntimeReadOnlyFlags);
		const FString SetterParamDecl = PyGenUtil::PythonizeMethodParam(TEXT("value"), InGetSet.Prop.Prop, PythonizeMethodParamFlags);
		if (PyGenUtil::IsTypeHintingEnabled())
		{
			// No need to add a return value to help auto-complete/type checker when the type is hinted in the signature.
			const FString GetterReturnType = PyGenUtil::PythonizeMethodReturnType(InGetSet.Prop.Prop, PythonizeMethodReturnTypeFlags);
			ExportGetSet(UTF8_TO_TCHAR(InGetSet.GetSetName.GetData()), UTF8_TO_TCHAR(InGetSet.GetSetDoc.GetData()), /*ReturnValue*/TEXT(""), *GetterReturnType, *SetterParamDecl, bIsReadOnly);
		}
		else
		{
			// We use strict typing for return values to aid auto-complete (we also only care about the type and not the value, so structs can be default constructed)
			static const PyGenUtil::EPythonizeFlags PythonizeValueFlags = PyGenUtil::EPythonizeFlags::UseStrictTyping | PyGenUtil::EPythonizeFlags::DefaultConstructStructs;
			const FString GetterReturnValue = PyGenUtil::PythonizeDefaultValue(InGetSet.Prop.Prop, FString(), PythonizeValueFlags);
			ExportGetSet(UTF8_TO_TCHAR(InGetSet.GetSetName.GetData()), UTF8_TO_TCHAR(InGetSet.GetSetDoc.GetData()), *GetterReturnValue, /*ReturnType*/TEXT(""), *SetterParamDecl, bIsReadOnly);
		}
	};

	auto ExportGeneratedOperator = [&OutPythonScript, &PyTypeName](const PyGenUtil::FGeneratedWrappedOperatorStack& InOpStack, const PyGenUtil::FGeneratedWrappedOperatorSignature& InOpSignature)
	{
		auto AppendFunctionTooltip = [](const UFunction* InFunc, const TCHAR* InIdentation, FString& OutStr)
		{
			const FString FuncTooltip = PyGenUtil::GetFieldTooltip(InFunc);
			TArray<FString> FuncTooltipLines;
			FuncTooltip.ParseIntoArrayLines(FuncTooltipLines, /*bCullEmpty*/false);

			bool bMultipleLines = false;
			for (const FString& FuncTooltipLine : FuncTooltipLines)
			{
				if (bMultipleLines)
				{
					OutStr += LINE_TERMINATOR;
					OutStr += InIdentation;
				}
				bMultipleLines = true;

				OutStr += FuncTooltipLine;
			}
		};

		FString OpDocString;
		if (InOpSignature.OtherType != PyGenUtil::FGeneratedWrappedOperatorSignature::EType::None)
		{
			OpDocString += TEXT("**Overloads:**") LINE_TERMINATOR;
			for (const PyGenUtil::FGeneratedWrappedOperatorFunction& OpFunc : InOpStack.Funcs)
			{
				if (OpFunc.OtherParam.ParamProp)
				{
					OpDocString += LINE_TERMINATOR TEXT("- ``");  // add as a list and code style
					OpDocString += PyGenUtil::GetPropertyTypePythonName(OpFunc.OtherParam.ParamProp);
					OpDocString += TEXT("`` ");
					AppendFunctionTooltip(OpFunc.Func, TEXT("  "), OpDocString);
				}
			}
		}
		else if (InOpStack.Funcs.Num() > 0)
		{
			AppendFunctionTooltip(InOpStack.Funcs[0].Func, TEXT(""), OpDocString);
		}

		// Python complains if __eq__() and __ne__() overriding the 'object' methods don't have the other parameter as 'object' type.
		FString OtherParamType = (FCString::Strcmp(InOpSignature.PyFuncName, TEXT("__eq__")) == 0 || FCString::Strcmp(InOpSignature.PyFuncName, TEXT("__ne__")) == 0) ? TEXT("object") : PyTypeName;

		PyGenUtil::IsTypeHintingEnabled()
			? OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(self%s) -> %s:"),
				InOpSignature.PyFuncName,
				(InOpSignature.OtherType == PyGenUtil::FGeneratedWrappedOperatorSignature::EType::None ? TEXT("") : *FString::Printf(TEXT(", other: %s"),*OtherParamType)),
				(InOpSignature.ReturnType == PyGenUtil::FGeneratedWrappedOperatorSignature::EType::Bool ? TEXT("bool") : TEXT("None"))))
			: OutPythonScript.WriteLine(FString::Printf(TEXT("def %s(self%s):"), InOpSignature.PyFuncName, (InOpSignature.OtherType == PyGenUtil::FGeneratedWrappedOperatorSignature::EType::None ? TEXT("") : TEXT(", other"))));
		OutPythonScript.IncreaseIndent();
		OutPythonScript.WriteDocString(OpDocString);

		PyGenUtil::IsTypeHintingEnabled()
			? OutPythonScript.WriteLine(TEXT("...")) // No need to add a return value to help auto-complete/type checker when the type is hinted in the signature.
			: OutPythonScript.WriteLine(InOpSignature.ReturnType == PyGenUtil::FGeneratedWrappedOperatorSignature::EType::Bool ? TEXT("return False") : TEXT("return None"));
		OutPythonScript.DecreaseIndent();
	};

	auto ExportNoBodyMethod = [&OutPythonScript](const TCHAR* MethodDecl)
	{
		OutPythonScript.WriteLine(MethodDecl);
		OutPythonScript.IncreaseIndent();
		OutPythonScript.WriteLine(TEXT("...")); // pass
		OutPythonScript.DecreaseIndent();
	};

	bool bHasExportedClassData = false;

	// Export the __init__ function for this type
	{
		bool bWriteDefaultInit = true;

		if (GeneratedTypeData)
		{
			const FGuid MetaGuid = GeneratedTypeData->MetaData->GetTypeId();

			if (MetaGuid == FPyWrapperObjectMetaData::StaticTypeId())
			{
				// Skip the __init__ function on derived object types as the base one is already correct
				bWriteDefaultInit = false;
			}
			else if (MetaGuid == FPyWrapperStructMetaData::StaticTypeId())
			{
				TSharedPtr<const FPyWrapperStructMetaData> StructMetaData = StaticCastSharedPtr<FPyWrapperStructMetaData>(GeneratedTypeData->MetaData);
				check(StructMetaData.IsValid());

				// Python can only support 255 parameters, so if we have more than that for this struct just use the generic __init__ function
				FString InitParamsStr;
				if (StructMetaData->MakeFunc.Func)
				{
					bWriteDefaultInit = false;

					for (const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam : StructMetaData->MakeFunc.InputParams)
					{
						InitParamsStr += TEXT(", ");
						InitParamsStr += PyGenUtil::PythonizeMethodParam(InitParam, PythonizeMethodParamFlags, UE::Python::ShouldEllipseParamDefaultValue);
					}
				}
				else if (StructMetaData->InitParams.Num() <= 255)
				{
					bWriteDefaultInit = false;

					FStructOnScope StructData(StructMetaData->Struct);
					for (const PyGenUtil::FGeneratedWrappedMethodParameter& InitParam : StructMetaData->InitParams)
					{
						InitParamsStr += TEXT(", ");
						InitParamsStr += PyGenUtil::PythonizeMethodParam(InitParam, PythonizeMethodParamFlags, UE::Python::ShouldEllipseParamDefaultValue);
					}
				}

				if (!bWriteDefaultInit)
				{
					bHasExportedClassData = true;
					PyGenUtil::IsTypeHintingEnabled()
						? ExportNoBodyMethod(*FString::Printf(TEXT("def __init__(self%s) -> None:"), *InitParamsStr))
						: ExportNoBodyMethod(*FString::Printf(TEXT("def __init__(self%s):"), *InitParamsStr));
				}
			}
			else if (MetaGuid == FPyWrapperEnumMetaData::StaticTypeId())
			{
				// Skip the __init__ function on derived enums
				bWriteDefaultInit = false;
			}
			// todo: have correct __init__ signatures for the other intrinsic types?
		}
		else if (PyType == &PyWrapperObjectType)
		{
			bHasExportedClassData = true;
			bWriteDefaultInit = false;

			PyGenUtil::IsTypeHintingEnabled()
				? ExportNoBodyMethod(TEXT("def __init__(self, outer: Optional[Object] = None, name: Union[Name, str]=\"None\") -> None:"))
				: ExportNoBodyMethod(TEXT("def __init__(self, outer=None, name=\"None\"):"));
		}
		else if (PyType == &PyWrapperEnumType)
		{
			// Enums don't really have an __init__ function at runtime, so just give them a default one (with no arguments)
			bHasExportedClassData = true;
			bWriteDefaultInit = false;
			PyGenUtil::IsTypeHintingEnabled()
				? ExportNoBodyMethod(TEXT("def __init__(self) -> None:"))
				: ExportNoBodyMethod(TEXT("def __init__(self):"));
		}
		else if (PyType == &PyWrapperEnumValueDescrType)
		{
			bHasExportedClassData = true;
			bWriteDefaultInit = false;

			// This is a special internal decorator type used to define enum entries, which is why it has __get__ as well as __init__
			PyGenUtil::IsTypeHintingEnabled()
				? OutPythonScript.WriteLine(TEXT("def __init__(self, enum: Any, name: str, value: int) -> None:"))
				: OutPythonScript.WriteLine(TEXT("def __init__(self, enum, name, value):"));
			OutPythonScript.IncreaseIndent();
			OutPythonScript.WriteLine(TEXT("self.enum = enum"));
			OutPythonScript.WriteLine(TEXT("self.name = name"));
			OutPythonScript.WriteLine(TEXT("self.value = value"));
			OutPythonScript.DecreaseIndent();

			OutPythonScript.WriteLine(TEXT("def __get__(self, obj, type=None):"));
			OutPythonScript.IncreaseIndent();
			OutPythonScript.WriteLine(TEXT("return self"));
			OutPythonScript.DecreaseIndent();

			// It also needs a __repr__ function for Sphinx to generate docs correctly
			OutPythonScript.WriteLine(TEXT("def __repr__(self):"));
			OutPythonScript.IncreaseIndent();
			OutPythonScript.WriteLine(TEXT("return \"{0}.{1}\".format(self.enum, self.name)"));
			OutPythonScript.DecreaseIndent();
		}

		if (PyGenUtil::IsTypeHintingEnabled())
		{
			if (PyType == &PyWrapperArrayType || PyType == &PyWrapperFixedArrayType)
			{
				bWriteDefaultInit = false;
				bHasExportedClassData = true;

				// WARNING: The parameter name used with __init__ must be 'type' and 'len' to allow named parameters.
				PyType == &PyWrapperFixedArrayType
					? ExportNoBodyMethod(TEXT("def __init__(self, type: type, len: int) -> None:"))
					: ExportNoBodyMethod(TEXT("def __init__(self, type: type) -> None:"));

				ExportNoBodyMethod(TEXT("def __setitem__(self, index: int, value: _ElemType) -> None:"));
				ExportNoBodyMethod(TEXT("def __getitem__(self, index: int) -> _ElemType:"));
				ExportNoBodyMethod(TEXT("def __len__(self) -> int:"));
				if (PyType == &PyWrapperArrayType)
				{
					ExportNoBodyMethod(TEXT("def __delitem__(self, key: Union[int, slice, Any]) -> None:"));
				}
			}
			else if (PyType == &PyWrapperSetType)
			{
				// WARNING: The parameter name used with __init__ must be 'type' to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, type: type) -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;

				ExportNoBodyMethod(TEXT("def __len__(self) -> int:"));
				ExportNoBodyMethod(TEXT("def __contains__(self, x: object) -> bool:"));
				ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[Any]:"));
			}
			else if (PyType == &PyWrapperMapType)
			{
				// WARNING: The parameter name used with __init__ must be 'key' and 'value' to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, key: type, value: type) -> None:"));
				bWriteDefaultInit = false;

				ExportNoBodyMethod(TEXT("def __setitem__(self, key: _KeyType, value: _ValueType) -> None:"));
				ExportNoBodyMethod(TEXT("def __getitem__(self, key: _KeyType) -> _ValueType:"));
				ExportNoBodyMethod(TEXT("def __len__(self) -> int:"));
				ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[Any]:"));
				ExportNoBodyMethod(TEXT("def __delitem__(self, key: Union[int, slice, Any]) -> None:"));
			}
			else if (PyType == &PyWrapperNameType)
			{
				ExportNoBodyMethod(TEXT("def __init__(self, value: Union[Name, str] = \"None\") -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyWrapperTextType)
			{
				ExportNoBodyMethod(TEXT("def __init__(self, value: Union[Text, str] = \"\") -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyWrapperFieldPathType)
			{
				ExportNoBodyMethod(TEXT("def __init__(self, value: Union[FieldPath, str] = \"\") -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyScopedSlowTaskType) // Type defined in PyCore.cpp
			{
				// WARNING: Parameter names must match the ones from PyCore.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, work: float, desc: Union[Text, str] = \"\", enabled: bool = True) -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyScopedEditorTransactionType) // Type defined in PyEditor.cpp
			{
				// WARNING: Parameter names must match the ones from PyEditor.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, desc: Union[Text, str]) -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyActorIteratorType || PyType == &PySelectedActorIteratorType) // Type defined in PyEngine.cpp
			{
				// WARNING: Parameter names must match the ones from PyEngine.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, world: World, type: Union[Class, type] = ...) -> None:"));
				ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[Any]:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyObjectIteratorType)
			{
				// WARNING: Parameter names must match the ones from PyEngine.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, type: Union[Class, type] = ...) -> None:"));
				ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[Any]:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyTypeIteratorType || PyType == &PyClassIteratorType || PyType == &PyStructIteratorType)
			{
				// WARNING: Parameter names must match the ones from PyEngine.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, type: type) -> None:"));
				if (PyType == &PyClassIteratorType)
				{
					ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[Class]:"));
				}
				else if (PyType == &PyStructIteratorType)
				{
					ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[ScriptStruct]:"));
				}
				else// if (PyType == &PyTypeIteratorType) -> The last remaining.
				{
					ExportNoBodyMethod(TEXT("def __iter__(self) -> Iterator[type]:"));
				}
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyFPropertyDefType)
			{
				// WARNING: Parameter names must match the ones from PyEngine.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, type: type, meta: Optional[Dict[str, str]], getter: Optional[str], setter: Optional[str]) -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyUFunctionDefType)
			{
				// WARNING: Parameter names must match the ones from PyEngine.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, func: Any, meta: Optional[Dict[str, str]], ret: Optional[Any], params: Optional[Sequence[Any]], override: Optional[bool], static: Optional[bool], pure: Optional[bool], getter: Optional[bool], setter: Optional[bool]) -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
			else if (PyType == &PyUValueDefType)
			{
				// WARNING: Parameter names must match the ones from PyEngine.cpp to allow named parameters.
				ExportNoBodyMethod(TEXT("def __init__(self, val: object, meta: Optional[Dict[str, str]]) -> None:"));
				bWriteDefaultInit = false;
				bHasExportedClassData = true;
			}
		}

		if (bWriteDefaultInit)
		{
			bHasExportedClassData = true;

			PyGenUtil::IsTypeHintingEnabled()
				? OutPythonScript.WriteLine(TEXT("def __init__(self, *args: Any, **kwargs: Any) -> None:"))
				: OutPythonScript.WriteLine(TEXT("def __init__(self, *args, **kwargs):"));
			OutPythonScript.IncreaseIndent();
			OutPythonScript.WriteLine(TEXT("...")); // pass
			OutPythonScript.DecreaseIndent();
		}
	}

	TSet<const PyGetSetDef*> ExportedGetSets;

	if (GeneratedTypeData)
	{
		const FGuid MetaGuid = GeneratedTypeData->MetaData->GetTypeId();

		if (MetaGuid == FPyWrapperObjectMetaData::StaticTypeId())
		{
			// Export class get/sets
			const PyGenUtil::FGeneratedWrappedClassType* ClassType = static_cast<const PyGenUtil::FGeneratedWrappedClassType*>(GeneratedTypeData);

			check(ClassType->GetSets.TypeGetSets.Num() == (ClassType->GetSets.PyGetSets.Num() - 1)); // -1 as PyGetSets has a null terminator
			for (int32 GetSetIndex = 0; GetSetIndex < ClassType->GetSets.TypeGetSets.Num(); ++GetSetIndex)
			{
				bHasExportedClassData = true;
				ExportGeneratedGetSet(ClassType->GetSets.TypeGetSets[GetSetIndex]);
				ExportedGetSets.Add(&ClassType->GetSets.PyGetSets[GetSetIndex]);
			}
		}
		else if (MetaGuid == FPyWrapperStructMetaData::StaticTypeId())
		{
			// Export struct get/sets
			const PyGenUtil::FGeneratedWrappedStructType* StructType = static_cast<const PyGenUtil::FGeneratedWrappedStructType*>(GeneratedTypeData);

			check(StructType->GetSets.TypeGetSets.Num() == (StructType->GetSets.PyGetSets.Num() - 1)); // -1 as PyGetSets has a null terminator
			for (int32 GetSetIndex = 0; GetSetIndex < StructType->GetSets.TypeGetSets.Num(); ++GetSetIndex)
			{
				bHasExportedClassData = true;
				ExportGeneratedGetSet(StructType->GetSets.TypeGetSets[GetSetIndex]);
				ExportedGetSets.Add(&StructType->GetSets.PyGetSets[GetSetIndex]);
			}
		}
	}

	for (PyGetSetDef* GetSetDef = PyType->tp_getset; GetSetDef && GetSetDef->name; ++GetSetDef)
	{
		if (ExportedGetSets.Contains(GetSetDef))
		{
			continue;
		}
		ExportedGetSets.Add(GetSetDef);

		bHasExportedClassData = true;

		ExportGetSet(UTF8_TO_TCHAR(GetSetDef->name), UTF8_TO_TCHAR(GetSetDef->doc), /*ReturnValue*/TEXT("None"), /*GetterReturnTypeDecl*/TEXT(""), /*ParamNameAndTypeDecl*/TEXT(""), /*IsReadOnly*/false);
	}

	for (PyMethodDef* MethodDef = PyType->tp_methods; MethodDef && MethodDef->ml_name; ++MethodDef)
	{
		bHasExportedClassData = true;

		const bool bIsClassMethod = !!(MethodDef->ml_flags & METH_CLASS);
		const bool bHasKeywords = !!(MethodDef->ml_flags & METH_KEYWORDS);
		if (bIsClassMethod)
		{
			OutPythonScript.WriteLine(TEXT("@classmethod"));
			OutPythonScript.WriteLine(UE::Python::GenerateMethodDecl(MethodDef, PyTypeName, /*bIsClassMethod*/true));
		}
		else
		{
			OutPythonScript.WriteLine(UE::Python::GenerateMethodDecl(MethodDef, PyTypeName, /*bIsClassMethod*/false, /*bIsMemberMethod*/true));
		}
		OutPythonScript.IncreaseIndent();
		OutPythonScript.WriteDocString(UTF8_TO_TCHAR(MethodDef->ml_doc));
		OutPythonScript.WriteLine(TEXT("...")); // pass
		OutPythonScript.DecreaseIndent();
	}

	if (GeneratedTypeData)
	{
		const FGuid MetaGuid = GeneratedTypeData->MetaData->GetTypeId();

		if (MetaGuid == FPyWrapperObjectMetaData::StaticTypeId())
		{
			// Export class methods and constants
			const PyGenUtil::FGeneratedWrappedClassType* ClassType = static_cast<const PyGenUtil::FGeneratedWrappedClassType*>(GeneratedTypeData);

			for (const PyGenUtil::FGeneratedWrappedMethod& TypeMethod : ClassType->Methods.TypeMethods)
			{
				bHasExportedClassData = true;
				ExportGeneratedMethod(TypeMethod);
			}

			for (const TSharedRef<PyGenUtil::FGeneratedWrappedDynamicMethodWithClosure>& DynamicMethod : ClassType->DynamicMethods)
			{
				bHasExportedClassData = true;
				ExportGeneratedMethod(*DynamicMethod);
			}

			for (const PyGenUtil::FGeneratedWrappedConstant& TypeConstant : ClassType->Constants.TypeConstants)
			{
				bHasExportedClassData = true;
				ExportGeneratedConstant(TypeConstant);
			}

			for (const TSharedRef<PyGenUtil::FGeneratedWrappedDynamicConstantWithClosure>& DynamicConstant : ClassType->DynamicConstants)
			{
				bHasExportedClassData = true;
				ExportGeneratedConstant(*DynamicConstant);
			}
		}
		else if (MetaGuid == FPyWrapperStructMetaData::StaticTypeId())
		{
			// Export struct methods and constants
			const PyGenUtil::FGeneratedWrappedStructType* StructType = static_cast<const PyGenUtil::FGeneratedWrappedStructType*>(GeneratedTypeData);

			TSharedPtr<const FPyWrapperStructMetaData> StructMetaData = StaticCastSharedPtr<FPyWrapperStructMetaData>(GeneratedTypeData->MetaData);
			check(StructMetaData.IsValid());

			for (const TSharedRef<PyGenUtil::FGeneratedWrappedDynamicMethodWithClosure>& DynamicMethod : StructType->DynamicMethods)
			{
				bHasExportedClassData = true;
				ExportGeneratedMethod(*DynamicMethod);
			}

			// Avoid exporting method __truediv__ twice becauses it is used for both / and /= operator.
			bool bTrueDivExported = false;
			for (int32 OpTypeIndex = 0; OpTypeIndex < (int32)PyGenUtil::EGeneratedWrappedOperatorType::Num; ++OpTypeIndex)
			{
				const PyGenUtil::FGeneratedWrappedOperatorStack& OpStack = StructMetaData->OpStacks[OpTypeIndex];
				if (OpStack.Funcs.Num() > 0)
				{
					const PyGenUtil::FGeneratedWrappedOperatorSignature& OpSignature = PyGenUtil::FGeneratedWrappedOperatorSignature::OpTypeToSignature((PyGenUtil::EGeneratedWrappedOperatorType)OpTypeIndex);

					if (FCString::Strcmp(OpSignature.PyFuncName, TEXT("__truediv__")) != 0)
					{
						ExportGeneratedOperator(OpStack, OpSignature);
					}
					else if (!bTrueDivExported)
					{
						ExportGeneratedOperator(OpStack, OpSignature);
						bTrueDivExported = true;
					}
				}
			}

			for (const TSharedRef<PyGenUtil::FGeneratedWrappedDynamicConstantWithClosure>& DynamicConstant : StructType->DynamicConstants)
			{
				bHasExportedClassData = true;
				ExportGeneratedConstant(*DynamicConstant);
			}
		}
		else if (MetaGuid == FPyWrapperEnumMetaData::StaticTypeId())
		{
			// Export enum entries
			const PyGenUtil::FGeneratedWrappedEnumType* EnumType = static_cast<const PyGenUtil::FGeneratedWrappedEnumType*>(GeneratedTypeData);

			if (EnumType->EnumEntries.Num() > 0)
			{
				// Add extra line break for first enum member
				OutPythonScript.WriteNewLine();

				for (const PyGenUtil::FGeneratedWrappedEnumEntry& EnumMember : EnumType->EnumEntries)
				{
					const FString EntryName = UTF8_TO_TCHAR(EnumMember.EntryName.GetData());
					const FString EntryValue = LexToString(EnumMember.EntryValue);

					FString EntryDoc = UTF8_TO_TCHAR(EnumMember.EntryDoc.GetData());
					if (EntryDoc.IsEmpty())
					{
						EntryDoc = EntryValue;
					}
					else
					{
						EntryDoc.InsertAt(0, *FString::Printf(TEXT("%s: "), *EntryValue));
					}

					// The enum value types reported by the engine using print(type(unreal.SomeEnum.SOME_VALUE)) would be 'class SomeEnum' but _EnumEntry class is unrelated.
					PyGenUtil::IsTypeHintingEnabled()
						? ExportConstant(*EntryName, PyTypeName, TEXT("..."), *EntryDoc)
						: ExportConstant(*EntryName, /*TypeHint*/FString(), FString::Printf(TEXT("_EnumEntry(\"%s\", \"%s\", %s)"), *PyTypeName, *EntryName, *EntryValue), *EntryDoc);

					for (const PyGenUtil::FUTF8Buffer& EntryDeprecatedAlias : EnumMember.EntryDeprecatedAliases)
					{
						const FString DeprecatedEntryName = UTF8_TO_TCHAR(EntryDeprecatedAlias.GetData());
						const FString DeprecatedEntryDoc = FString::Printf(TEXT("%s (deprecated: Use %s)"), *EntryDoc, *EntryName);

						PyGenUtil::IsTypeHintingEnabled()
							? ExportConstant(*DeprecatedEntryName, PyTypeName, EntryName, *DeprecatedEntryDoc)
							: ExportConstant(*DeprecatedEntryName, /*TypeHint*/FString(), EntryName, *DeprecatedEntryDoc);
					}

					bHasExportedClassData = true;
				}
			}
		}
	}

	if (!bHasExportedClassData)
	{
		OutPythonScript.WriteLine(TEXT("...")); // pass
	}

	OutPythonScript.DecreaseIndent();
	OutPythonScript.WriteNewLine();
}

void FPyWrapperTypeRegistry::RegisterPythonTypeName(const FString& InPythonTypeName, const FSoftObjectPath& InUnrealTypeName)
{
	bool bAlreadyExists = true;
	PythonWrappedTypeNameToUnrealTypeName.FindOrProduceAndApply(
		InPythonTypeName,
		[&InUnrealTypeName, &bAlreadyExists]()
		{
			// If the value is being added, then we don't have to perform the additional test in Apply
			bAlreadyExists = false;
			return InUnrealTypeName;
		},
		[&InPythonTypeName, &InUnrealTypeName, &bAlreadyExists](const FSoftObjectPath& Value)
		{
			if (bAlreadyExists && Value != InUnrealTypeName)
			{
				REPORT_PYTHON_GENERATION_ISSUE(Warning, TEXT("'%s' and '%s' have the same name (%s) when exposed to Python. Rename one of them using 'ScriptName' meta-data."), *Value.ToString(), *InUnrealTypeName.ToString(), *InPythonTypeName);
			}
		}
	);
}

void FPyWrapperTypeRegistry::UnregisterPythonTypeName(const FString& InPythonTypeName, const FSoftObjectPath& InUnrealTypeName)
{
	PythonWrappedTypeNameToUnrealTypeName.RemoveIf(InPythonTypeName,
		[&InUnrealTypeName](const FSoftObjectPath& ExistingUnrealTypeName)
		{
			return ExistingUnrealTypeName == InUnrealTypeName;
		}
	);
}

void ShutdownPyWrappers()
{
	FPyWrapperObjectFactory::Get().Shutdown();
	FPyWrapperStructFactory::Get().Shutdown();
	FPyWrapperDelegateFactory::Get().Shutdown();
	FPyWrapperMulticastDelegateFactory::Get().Shutdown();
	FPyWrapperNameFactory::Get().Shutdown();
	FPyWrapperTextFactory::Get().Shutdown();
	FPyWrapperArrayFactory::Get().Shutdown();
	FPyWrapperFixedArrayFactory::Get().Shutdown();
	FPyWrapperSetFactory::Get().Shutdown();
	FPyWrapperMapFactory::Get().Shutdown();
	FPyWrapperFieldPathFactory::Get().Shutdown();
	FPyWrapperTypeReinstancer::Get().Shutdown();
	FPyWrapperTypeRegistry::Get().Shutdown();
}

#endif	// WITH_PYTHON

#undef LOCTEXT_NAMESPACE
