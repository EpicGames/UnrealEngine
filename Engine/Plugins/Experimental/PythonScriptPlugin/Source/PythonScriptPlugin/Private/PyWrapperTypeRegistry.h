// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "CoreMinimal.h"
#include "PyPtr.h"
#include "PyUtil.h"
#include "PyGenUtil.h"
#include "PyConversionMethod.h"
#include "PyOnlineDocsWriter.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"
#include "Containers/StripedMap.h"
#include "Templates/Function.h"

#include <atomic>

#if WITH_PYTHON

struct FPyWrapperObject;
struct FPyWrapperStruct;
struct FPyWrapperDelegate;
struct FPyWrapperMulticastDelegate;
struct FPyWrapperName;
struct FPyWrapperText;
struct FPyWrapperArray;
struct FPyWrapperFixedArray;
struct FPyWrapperSet;
struct FPyWrapperMap;
struct FPyWrapperStructMetaData;
struct FPyWrapperFieldPath;

class FPyWrapperOwnerContext;
class UPythonGeneratedClass;
class UPythonGeneratedStruct;
class IPyWrapperInlineStructFactory;

class FPyFileWriter;
class FReferenceCollector;

enum class EPyTypeGenerationFlags : uint8
{
	/** No behavior */
	None = 0,
	/** Generate the Python wrapper for this type, even if it fails the PyGenUtil::ShouldExportX check */
	ForceShouldExport = 1<<0,
	/** Generate the Python wrapper for this type, even if it passes the PyGenUtil::IsBlueprintGeneratedX check */
	IncludeBlueprintGeneratedTypes = 1<<1,
	/** Generate the Python wrapper for this type, re-using the existing type if it already exists */
	OverwriteExisting = 1<<2,
};
ENUM_CLASS_FLAGS(EPyTypeGenerationFlags);

/** Type conversion for TPyWrapperTypeFactory */
template <typename UnrealType, typename KeyType>
struct TPyWrapperTypeFactoryConversion
{
	static KeyType UnrealTypeToKeyType(UnrealType InUnrealInstance)
	{
		return InUnrealInstance;
	}
};

/** Type conversion specialization for FPyWrapperTextFactory */
template <>
struct TPyWrapperTypeFactoryConversion<FText, const void*>
{
	static const void* UnrealTypeToKeyType(FText InUnrealInstance)
	{
		return FTextInspector::GetSharedDataId(InUnrealInstance);
	}
};

/** Generic factory implementation for Python wrapped types. Types should derive from this and implement a CreateInstance and FindInstance function */
template <typename UnrealType, typename PythonType, typename KeyType = UnrealType>
class TPyWrapperTypeFactory
{
protected:
	struct FInternalKey
	{
	public:
		FInternalKey(KeyType InWrapperKey, PyTypeObject* InPyType)
			: WrapperKey(InWrapperKey)
			, PyType(InPyType)
			, Hash(HashCombine(GetTypeHash(WrapperKey), GetTypeHash(InPyType)))
		{
		}

		FORCEINLINE bool operator==(const FInternalKey& Other) const
		{
			return WrapperKey == Other.WrapperKey
				&& PyType == Other.PyType;
		}

		FORCEINLINE bool operator!=(const FInternalKey& Other) const
		{
			return !(*this == Other);
		}

		friend inline uint32 GetTypeHash(const FInternalKey& Key)
		{
			return Key.Hash;
		}

	private:
		KeyType WrapperKey;
		PyTypeObject* PyType;
		uint32 Hash;
	};

public:
	/** Map a wrapped Python instance associated with the given Unreal instance (called internally by the Python type) */
	void MapInstance(UnrealType InUnrealInstance, PythonType* InPythonInstance)
	{
		MappedInstances.Add(FInternalKey(TPyWrapperTypeFactoryConversion<UnrealType, KeyType>::UnrealTypeToKeyType(InUnrealInstance), Py_TYPE(InPythonInstance)), InPythonInstance);
	}

	/** Unmap the wrapped instance associated with the given UObject instance (called internally by the Python type) */
	void UnmapInstance(UnrealType InUnrealInstance, PyTypeObject* InWrappedPyType)
	{
		MappedInstances.Remove(FInternalKey(TPyWrapperTypeFactoryConversion<UnrealType, KeyType>::UnrealTypeToKeyType(InUnrealInstance), InWrappedPyType));
	}

	void Shutdown()
	{
		MappedInstances.Reset();
	}

protected:
	/** Callback used to initialize this type */
	typedef TFunctionRef<int(PythonType*)> FCreateInstanceInitializerFunc;

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	PythonType* FindInstanceInternal(UnrealType InUnrealInstance, PyTypeObject* InWrappedPyType) const
	{
		return MappedInstances.FindRef(FInternalKey(TPyWrapperTypeFactoryConversion<UnrealType, KeyType>::UnrealTypeToKeyType(InUnrealInstance), InWrappedPyType));
	}

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	PythonType* CreateInstanceInternal(UnrealType InUnrealInstance, PyTypeObject* InWrappedPyType, FCreateInstanceInitializerFunc CreateInstanceInitializerFunc, const bool bForceCreate = false)
	{
		if (!bForceCreate)
		{
			if (PythonType* ExistingInstance = MappedInstances.FindRef(FInternalKey(TPyWrapperTypeFactoryConversion<UnrealType, KeyType>::UnrealTypeToKeyType(InUnrealInstance), InWrappedPyType)))
			{
				Py_INCREF(ExistingInstance);
				return ExistingInstance;
			}
		}

		TPyPtr<PythonType> NewInstance = TPyPtr<PythonType>::StealReference(PythonType::New(InWrappedPyType));
		if (NewInstance)
		{
			if (CreateInstanceInitializerFunc(NewInstance) != 0)
			{
				PyUtil::LogPythonError();
				return nullptr;
			}
		}
		return NewInstance.Release();
	}

	/** Map from the internal key to wrapped Python instance */
	TMap<FInternalKey, PythonType*> MappedInstances;
};

/** Factory for wrapped UObject instances */
class FPyWrapperObjectFactory : public TPyWrapperTypeFactory<UObject*, FPyWrapperObject>
{
public:
	/** Access the singleton instance */
	static FPyWrapperObjectFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperObject* FindInstance(UObject* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperObject* CreateInstance(UObject* InUnrealInstance);

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperObject* CreateInstance(UClass* InInterfaceClass, UObject* InUnrealInstance);
};

/** Factory for wrapped UScriptStruct instances */
class FPyWrapperStructFactory : public TPyWrapperTypeFactory<void*, FPyWrapperStruct>
{
public:
	/** Access the singleton instance */
	static FPyWrapperStructFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperStruct* FindInstance(UScriptStruct* InStruct, void* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperStruct* CreateInstance(UScriptStruct* InStruct, void* InUnrealInstance, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped delegate instances */
class FPyWrapperDelegateFactory : public TPyWrapperTypeFactory<FScriptDelegate*, FPyWrapperDelegate>
{
public:
	/** Access the singleton instance */
	static FPyWrapperDelegateFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperDelegate* FindInstance(const UFunction* InDelegateSignature, FScriptDelegate* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperDelegate* CreateInstance(const UFunction* InDelegateSignature, FScriptDelegate* InUnrealInstance, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped multicast delegate instances */
class FPyWrapperMulticastDelegateFactory : public TPyWrapperTypeFactory<FMulticastScriptDelegate*, FPyWrapperMulticastDelegate>
{
public:
	/** Access the singleton instance */
	static FPyWrapperMulticastDelegateFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperMulticastDelegate* FindInstance(const UFunction* InDelegateSignature, FMulticastScriptDelegate* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperMulticastDelegate* CreateInstance(const UFunction* InDelegateSignature, FMulticastScriptDelegate* InUnrealInstance, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped FName instances */
class FPyWrapperNameFactory : public TPyWrapperTypeFactory<FName, FPyWrapperName>
{
public:
	/** Access the singleton instance */
	static FPyWrapperNameFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperName* FindInstance(const FName InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperName* CreateInstance(const FName InUnrealInstance);
};

/** Factory for wrapped FText instances */
class FPyWrapperTextFactory : public TPyWrapperTypeFactory<FText, FPyWrapperText, const void*>
{
public:
	/** Access the singleton instance */
	static FPyWrapperTextFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperText* FindInstance(const FText InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperText* CreateInstance(const FText InUnrealInstance);
};

/** Factory for wrapped array instances */
class FPyWrapperArrayFactory : public TPyWrapperTypeFactory<void*, FPyWrapperArray>
{
public:
	/** Access the singleton instance */
	static FPyWrapperArrayFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperArray* FindInstance(void* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperArray* CreateInstance(void* InUnrealInstance, const FArrayProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped fixed-array instances */
class FPyWrapperFixedArrayFactory : public TPyWrapperTypeFactory<void*, FPyWrapperFixedArray>
{
public:
	/** Access the singleton instance */
	static FPyWrapperFixedArrayFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperFixedArray* FindInstance(void* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperFixedArray* CreateInstance(void* InUnrealInstance, const FProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped set instances */
class FPyWrapperSetFactory : public TPyWrapperTypeFactory<void*, FPyWrapperSet>
{
public:
	/** Access the singleton instance */
	static FPyWrapperSetFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperSet* FindInstance(void* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperSet* CreateInstance(void* InUnrealInstance, const FSetProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped map instances */
class FPyWrapperMapFactory : public TPyWrapperTypeFactory<void*, FPyWrapperMap>
{
public:
	/** Access the singleton instance */
	static FPyWrapperMapFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperMap* FindInstance(void* InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperMap* CreateInstance(void* InUnrealInstance, const FMapProperty* InProp, const FPyWrapperOwnerContext& InOwnerContext, const EPyConversionMethod InConversionMethod);
};

/** Factory for wrapped field type instances */
class FPyWrapperFieldPathFactory : public TPyWrapperTypeFactory<FFieldPath, FPyWrapperFieldPath>
{
public:
	/** Access the singleton instance */
	static FPyWrapperFieldPathFactory& Get();

	/** Find the wrapped Python instance associated with the given Unreal instance (if any, returns borrowed reference) */
	FPyWrapperFieldPath* FindInstance(FFieldPath InUnrealInstance) const;

	/** Find the wrapped Python instance associated with the given Unreal instance, or create one if needed (returns new reference) */
	FPyWrapperFieldPath* CreateInstance(FFieldPath InUnrealInstance);
};


/** Singleton instance that handles re-instancing Python types */
class FPyWrapperTypeReinstancer : public FGCObject
{
public:
	/** Access the singleton instance */
	static FPyWrapperTypeReinstancer& Get();

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;

	/** Add a pending pair of classes to be re-instanced */
	void AddPendingClass(UPythonGeneratedClass* OldClass, UPythonGeneratedClass* NewClass);

	/** Add a pending pair of structs to be re-instanced */
	void AddPendingStruct(UPythonGeneratedStruct* OldStruct, UPythonGeneratedStruct* NewStruct);

	/** Process any pending re-instance requests */
	void ProcessPending();

	void Shutdown();

private:
	/** Pending pairs of classes that to be re-instanced */
	TMap<TObjectPtr<UPythonGeneratedClass>, TObjectPtr<UPythonGeneratedClass>> ClassesToReinstance;

	/** Pending pairs of structs that to be re-instanced */
	TMap<TObjectPtr<UPythonGeneratedStruct>, TObjectPtr<UPythonGeneratedStruct>> StructsToReinstance;
};

/** Singleton instance that maps Unreal types to Python types */
class FPyWrapperTypeRegistry : public FGCObject
{
public:
	/** Struct used to build up a list of wrapped type references that still need to be generated */
	struct FGeneratedWrappedTypeReferences
	{
		bool HasReferences() const
		{
			return (ClassReferences.Num() + StructReferences.Num() + EnumReferences.Num() + DelegateReferences.Num()) > 0;
		}

		TSet<const UClass*> ClassReferences;
		TSet<const UScriptStruct*> StructReferences;
		TSet<const UEnum*> EnumReferences;
		TSet<const UFunction*> DelegateReferences;
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnModuleDirtied, FName);

	/** Access the singleton instance */
	static FPyWrapperTypeRegistry& Get();

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;

	/** Callback for when a Python module is dirtied */
	FOnModuleDirtied& OnModuleDirtied()
	{
		return OnModuleDirtiedDelegate;
	}

	/** Register the information about a native Python module */
	void RegisterNativePythonModule(PyGenUtil::FNativePythonModule&& NativePythonModule);

	/**
	 * Register the factory for an inline struct (ie, a struct known at compile time that will allocate its instance data inlined within the Python object).
	 * @note Inline struct registration must happen before the first call to GenerateWrappedStructType, and this function will assert if that is not the case!
	 */
	void RegisterInlineStructFactory(const TSharedRef<const IPyWrapperInlineStructFactory>& InFactory);

	/** Get the factory for an inline struct (if known) from its Unreal struct name */
	const IPyWrapperInlineStructFactory* GetInlineStructFactory(const FTopLevelAssetPath& StructName) const;

	/** Generate a wrapped type for all currently available Unreal types */
	void GenerateWrappedTypes(bool bAllowMultithreadedGeneration = false);

	/** Generate a wrapped type for all available Unreal types in the given Unreal module */
	void GenerateWrappedTypesForModule(const FName ModuleName);

	/** Orphan the wrapped types associated with the given Unreal module (when the module is unloaded/reloaded) */
	void OrphanWrappedTypesForModule(const FName ModuleName);

	/** Generate a wrapped type for all referenced types */
	void GenerateWrappedTypesForReferences(const FGeneratedWrappedTypeReferences& InGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules);

	/** Generate notifications to (re)load the dirtied modules in Python */
	void NotifyModulesDirtied(const TSet<FName>& InDirtyModules) const;

	/** Called when an asset that is registered with the type registry has been renamed */
	void UpdateGenerateWrappedTypeForRename(const FSoftObjectPath& InOldTypeRegistryName, const UObject* InObj);

	/** Called when an asset that is registered with the type registry is deleted */
	void RemoveGenerateWrappedTypeForDelete(const FSoftObjectPath& InTypeRegistryName);

	/** Generate a wrapped type for the given object (if it's a valid type to be wrapped) */
	PyTypeObject* GenerateWrappedTypeForObject(const UObject* InObj, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags = EPyTypeGenerationFlags::None);

	/** True if we have wrapped type for the given object */
	bool HasWrappedTypeForObject(const UObject* InObj) const;

	/** True if we have wrapped type for the given object name */
	bool HasWrappedTypeForObjectName(const FSoftObjectPath& InName) const;

	/** Get the best wrapped type for the given object */
	PyTypeObject* GetWrappedTypeForObject(const UObject* InObj) const;

	/** Generate a wrapped type for the given class (only if this class has not yet been registered; will also register the type) */
	PyTypeObject* GenerateWrappedClassType(const UClass* InClass, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags = EPyTypeGenerationFlags::None);

	/** Register the wrapped type associated with the given class name */
	void RegisterWrappedClassType(const FSoftObjectPath& ClassName, PyTypeObject* PyType, const bool InDetectNameConflicts = true);

	/** Unregister the wrapped type associated with the given class name */
	void UnregisterWrappedClassType(const FSoftObjectPath& ClassName, PyTypeObject* PyType, const bool InUnregisterName = true);

	/** True if we have wrapped type for the exact given class */
	bool HasWrappedClassType(const UClass* InClass) const;

	/** Get the best wrapped type for the given class */
	PyTypeObject* GetWrappedClassType(const UClass* InClass) const;

	/** Generate a wrapped type for the given struct (only if this struct has not yet been registered; will also register the type) */
	PyTypeObject* GenerateWrappedStructType(const UScriptStruct* InStruct, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags = EPyTypeGenerationFlags::None);

	/** Register the wrapped type associated with the given struct name */
	void RegisterWrappedStructType(const FSoftObjectPath& StructName, PyTypeObject* PyType, const bool InDetectNameConflicts = true);

	/** Unregister the wrapped type associated with the given struct name */
	void UnregisterWrappedStructType(const FSoftObjectPath& StructName, PyTypeObject* PyType, const bool InUnregisterName = true);

	/** True if we have wrapped type for the exact given struct */
	bool HasWrappedStructType(const UScriptStruct* InStruct) const;

	/** Get the best wrapped type for the given struct */
	PyTypeObject* GetWrappedStructType(const UScriptStruct* InStruct) const;

	/** Generate a wrapped type for the given enum (only if this enum has not yet been registered; will also register the type) */
	PyTypeObject* GenerateWrappedEnumType(const UEnum* InEnum, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags = EPyTypeGenerationFlags::None);

	/** Register the wrapped type associated with the given enum name */
	void RegisterWrappedEnumType(const FSoftObjectPath& EnumName, PyTypeObject* PyType, const bool InDetectNameConflicts = true);

	/** Unregister the wrapped type associated with the given enum name */
	void UnregisterWrappedEnumType(const FSoftObjectPath& EnumName, PyTypeObject* PyType, const bool InUnregisterName = true);

	/** True if we have wrapped type for the exact given enum */
	bool HasWrappedEnumType(const UEnum* InEnum) const;

	/** Get the best wrapped type for the given enum */
	PyTypeObject* GetWrappedEnumType(const UEnum* InEnum) const;

	/** Generate a wrapped type for the given delegate signature (only if this delegate has not yet been registered; will also register the type) */
	PyTypeObject* GenerateWrappedDelegateType(const UFunction* InDelegateSignature, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences, TSet<FName>& OutDirtyModules, const EPyTypeGenerationFlags InGenerationFlags = EPyTypeGenerationFlags::None);

	/** Register the wrapped type associated with the given delegate name */
	void RegisterWrappedDelegateType(const FSoftObjectPath& DelegateName, PyTypeObject* PyType, const bool InDetectNameConflicts = true);

	/** Unregister the wrapped type associated with the given delegate name */
	void UnregisterWrappedDelegateType(const FSoftObjectPath& DelegateName, PyTypeObject* PyType, const bool InUnregisterName = true);

	/** True if we have wrapped type for the exact given delegate signature */
	bool HasWrappedDelegateType(const UFunction* InDelegateSignature) const;

	/** Get the best wrapped type for the given delegate signature */
	PyTypeObject* GetWrappedDelegateType(const UFunction* InDelegateSignature) const;

	/** Generate stub Python code for our wrapped types */
	void GenerateStubCodeForWrappedTypes(const EPyOnlineDocsFilterFlags InDocGenFlags = EPyOnlineDocsFilterFlags::IncludeNone) const;

	void Shutdown();

private:
	/** Call the game-thread only logic */
	void ExecutePostInitOnGameThread();

	/** Gather any types referenced by the given property are still need to be wrapped for use in Python */
	void GatherWrappedTypesForPropertyReferences(const FProperty* InProp, FGeneratedWrappedTypeReferences& OutGeneratedWrappedTypeReferences) const;

	/** Generate stub Python code for the given type */
	static void GenerateStubCodeForWrappedType(PyTypeObject* PyType, const PyGenUtil::FGeneratedWrappedType* GeneratedTypeData, FPyFileWriter& OutPythonScript, FPyOnlineDocsSection* OutOnlineDocsSection = nullptr);

	/** Register a Python type name, and detect if a name conflict has occurred */
	void RegisterPythonTypeName(const FString& InPythonTypeName, const FSoftObjectPath& InUnrealTypeName);

	/** Unregister a Python type name if it corresponds to the given Unreal type name */
	void UnregisterPythonTypeName(const FString& InPythonTypeName, const FSoftObjectPath& InUnrealTypeName);

	FPyWrapperTypeRegistry();

	/** True if we can register inline struct factories, False if we've already started generating types */
	std::atomic<bool> bCanRegisterInlineStructFactories;

	/** Map from the Unreal struct name to the factory data for an inline struct (ie, a struct known at compile time that will allocate its instance data inlined within the Python object) */
	TMap<FTopLevelAssetPath, TSharedPtr<const IPyWrapperInlineStructFactory>> InlineStructFactories;

	// Good balance for high core count CPUs.
	static constexpr int32 StripeCount = 32;

	/** Map from the Python wrapped type name to the Unreal type name it was generated from (for conflict detection) */
	typedef TStripedMap<StripeCount, FString, FSoftObjectPath, FDefaultSetAllocator, PyGenUtil::FCaseSensitiveStringMapFuncs<FSoftObjectPath>> FCaseSensitiveStringToNameMap;
	FCaseSensitiveStringToNameMap PythonWrappedTypeNameToUnrealTypeName;

	/** Map from the Unreal class name to the Python type */
	TStripedMap<StripeCount, FSoftObjectPath, PyTypeObject*> PythonWrappedClasses;

	/** Map from the Unreal struct name to the Python type */
	TStripedMap<StripeCount, FSoftObjectPath, PyTypeObject*> PythonWrappedStructs;

	/** Map from the Unreal enum name to the Python type */
	TStripedMap<StripeCount, FSoftObjectPath, PyTypeObject*> PythonWrappedEnums;

	/** Map from the Unreal delegate signature name to the Python type */
	TStripedMap<StripeCount, FSoftObjectPath, PyTypeObject*> PythonWrappedDelegates;

	/** Map from the Unreal type name to the generated Python type data */
	TStripedMap<StripeCount, FSoftObjectPath, TSharedPtr<PyGenUtil::FGeneratedWrappedType>> GeneratedWrappedTypes;

	/** Map from the Unreal module name to its generated type names (names are the Unreal names) */
	TStripedMultiMap<StripeCount, FName, FSoftObjectPath> GeneratedWrappedTypesForModule;

	/** Map make func name to the meta data of the struct to make. Used when meta 'HasNativeMake' references a function not loaded yet. */
	TStripedMap<StripeCount, FString, TSharedPtr<FPyWrapperStructMetaData>> UnresolvedMakeFuncs;

	/** Map break func name to the meta data of the struct to break. Used when meta 'HasNativeBreak' references a function not loaded yet. */
	TStripedMap<StripeCount, FString, TSharedPtr<FPyWrapperStructMetaData>> UnresolvedBreakFuncs;

	/** Array of generated Python type data that has been orphaned (due to its owner module being unloaded/reloaded) */
	TArray<TSharedPtr<PyGenUtil::FGeneratedWrappedType>> OrphanedWrappedTypes;

	/** Array of information about native Python modules */
	TArray<PyGenUtil::FNativePythonModule> NativePythonModules;

	/** Callback for when a Python module is dirtied */
	FOnModuleDirtied OnModuleDirtiedDelegate;
};

void ShutdownPyWrappers();

#endif	// WITH_PYTHON
