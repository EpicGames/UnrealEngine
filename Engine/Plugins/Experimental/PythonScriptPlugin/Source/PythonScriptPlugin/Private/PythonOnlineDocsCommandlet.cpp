// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonOnlineDocsCommandlet.h"
#include "PyWrapperTypeRegistry.h"
#include "PyGenUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PythonOnlineDocsCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogPythonOnlineDocsCommandlet, Log, All);

UPythonOnlineDocsCommandlet::UPythonOnlineDocsCommandlet()
{
	IsServer = true;
	IsClient = true;
	IsEditor = true;
	LogToConsole = false;
	ShowErrorCount = false;
}

int32 UPythonOnlineDocsCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamVals;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamVals);

#if WITH_PYTHON
	EPyOnlineDocsFilterFlags DocGenFlags = EPyOnlineDocsFilterFlags::IncludeNone;

	// Apply the filter flags from the command line
#define APPLY_FILTER_FLAG(NAME)							\
	if (Switches.Contains(TEXT(#NAME)))					\
	{													\
		DocGenFlags |= EPyOnlineDocsFilterFlags::NAME;	\
	}
	APPLY_FILTER_FLAG(IncludeEngine)
	APPLY_FILTER_FLAG(IncludeInternal)
	APPLY_FILTER_FLAG(IncludeProject)
#undef APPLY_FILTER_FLAG

	if (Switches.Contains(TEXT("IncludeEnterprise")))
	{
		UE_LOGF(LogPythonOnlineDocsCommandlet, Warning, "\n\nIncludeEnterprise is deprecated, as Enterprise is now considered part of Engine. Please update callsites to use IncludeEngine.");
		DocGenFlags |= EPyOnlineDocsFilterFlags::IncludeEngine;
	}

	// If we weren't given any filter flags, include everything
	if (DocGenFlags == EPyOnlineDocsFilterFlags::IncludeNone)
	{
		DocGenFlags = EPyOnlineDocsFilterFlags::IncludeAll;
	}

	// For the documentation, use the the light typing version, avoid listing all type coercions which make the API harder to read.
	PyGenUtil::SetTypeHintingMode(ETypeHintingMode::AutoCompletion);

	UE_LOGF(LogPythonOnlineDocsCommandlet, Display, "\n\nGenerating Python documentation...");
	FPyWrapperTypeRegistry::Get().GenerateStubCodeForWrappedTypes(DocGenFlags);

#else	// WITH_PYTHON
	
	UE_LOGF(LogPythonOnlineDocsCommandlet, Error, "Python docs cannot be generated as the plugin was built as a stub!");
	return -1;

#endif	// WITH_PYTHON

	return 0;
}

