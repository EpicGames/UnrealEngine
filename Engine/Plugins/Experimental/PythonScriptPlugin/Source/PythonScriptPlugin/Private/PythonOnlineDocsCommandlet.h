// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "PythonOnlineDocsCommandlet.generated.h"

/**
 * Minimal commandlet to format and write Python API online docs.
 * Can be passed the following flags to filter which types are included in the docs:
 *	-IncludeEngine
 *	-IncludeEnterprise
 *	-IncludeInternal
 *	-IncludeProject
 */
UCLASS()
class UPythonOnlineDocsCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	/** Default constructor. */
	UPythonOnlineDocsCommandlet();

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
