// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonScriptCommandlet.h"
#include "PythonScriptPlugin.h"
#include "Logging/LogMacros.h"
#include "Containers/Ticker.h"
#if WITH_EDITOR
#include "SourceControlHelpers.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(PythonScriptCommandlet)

DEFINE_LOG_CATEGORY_STATIC(LogPythonScriptCommandlet, Log, All);

int32 UPythonScriptCommandlet::Main(const FString& Params)
{
	// We do this parsing manually rather than using the normal command line parsing, as the Python scripts may be quoted and contain escape sequences that the command line parsing doesn't handle well
	FString PythonScript;
	{
		const FString ScriptTag = TEXT("-Script=");
		const int32 ScriptTagPos = Params.Find(ScriptTag);
		if (ScriptTagPos != INDEX_NONE)
		{
			const TCHAR* ScriptTagValue = &Params[ScriptTagPos + ScriptTag.Len()];
			if (*ScriptTagValue == TEXT('"'))
			{
				FParse::QuotedString(ScriptTagValue, PythonScript);
			}
			else
			{
				FParse::Token(ScriptTagValue, PythonScript, false);
			}
		}
	}
	if (PythonScript.IsEmpty())
	{
		UE_LOGF(LogPythonScriptCommandlet, Error, "-Script argument not specified");
		return -1;
	}

#if WITH_PYTHON
	// We can assume the user wanted Python support enabled when using this commandlet
	IPythonScriptPlugin::Get()->ForceEnablePythonAtRuntime();

	if (!IPythonScriptPlugin::Get()->IsPythonAvailable())
	{
		UE_LOGF(LogPythonScriptCommandlet, Error, "Python script cannot run as Python support is disabled!");
		return -1;
	}

	// Ensure that any start-up scripts have been run
	FPythonScriptPlugin::Get()->RunStartupScripts();

	{
#if WITH_EDITOR
		// Ensure that source control is enabled, as commandlets aren't guaranteed to do that
		FScopedSourceControl SourceControl;
#endif

		UE_LOGF(LogPythonScriptCommandlet, Display, "Running Python script: %ls", *PythonScript);

		FPythonCommandEx PythonCommand;
		PythonCommand.Flags |= EPythonCommandFlags::Unattended;
		PythonCommand.Command = PythonScript;
		if (!IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommand))
		{
			UE_LOGF(LogPythonScriptCommandlet, Error, "Python script executed with errors");
			return -1;
		}
	}

	UE_LOGF(LogPythonScriptCommandlet, Display, "Python script executed successfully");
	return 0;
#else
	UE_LOGF(LogPythonScriptCommandlet, Error, "Python script cannot run as Python support was disabled at compile-time!");
	return -1;
#endif
}
