// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/Commandlet.h"
#include "PythonScriptCommandlet.generated.h"

/** Minimal commandlet to invoke a Python script and exit */
UCLASS()
class UPythonScriptCommandlet : public UCommandlet
{
	GENERATED_BODY()

public:
	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& Params) override;
	//~ End UCommandlet Interface
};
