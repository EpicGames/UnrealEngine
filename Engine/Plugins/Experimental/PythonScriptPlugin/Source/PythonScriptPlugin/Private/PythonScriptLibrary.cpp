// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonScriptLibrary.h"
#include "PythonScriptPlugin.h"
#include "PyGIL.h"
#include "PyUtil.h"
#include "PyGenUtil.h"
#include "PyConversion.h"
#include "PyWrapperTypeRegistry.h"
#include "UObject/Script.h"
#include "UObject/Stack.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PythonScriptLibrary)

bool UPythonScriptLibrary::IsPythonAvailable()
{
	return IPythonScriptPlugin::Get()->IsPythonAvailable();
}

bool UPythonScriptLibrary::IsPythonConfigured()
{
	return IPythonScriptPlugin::Get()->IsPythonConfigured();
}

bool UPythonScriptLibrary::IsPythonInitialized()
{
	return IPythonScriptPlugin::Get()->IsPythonInitialized();
}

bool UPythonScriptLibrary::ForceEnablePythonAtRuntime()
{
	FString Location;
#if DO_BLUEPRINT_GUARD
	if (TConstArrayView<const FFrame*> ScriptStack = FBlueprintContextTracker::Get().GetCurrentScriptStack();
		ScriptStack.Num() > 0)
	{
		TStringBuilder<256> StackDescription;
		ScriptStack.Last()->GetStackDescription(StackDescription);
		Location = StackDescription;
	}
	else
#endif	// DO_BLUEPRINT_GUARD
	{
		Location = TEXT("<Unknown Blueprint>");
	}
	return FPythonScriptPlugin::Get()->ForceEnablePythonAtRuntime(Location);
}

bool UPythonScriptLibrary::ExecutePythonCommand(const FString& PythonCommand)
{
	return IPythonScriptPlugin::Get()->ExecPythonCommand(*PythonCommand);
}

bool UPythonScriptLibrary::ExecutePythonCommandEx(const FString& PythonCommand, FString& CommandResult, TArray<FPythonLogOutputEntry>& LogOutput, const EPythonCommandExecutionMode ExecutionMode, const EPythonFileExecutionScope FileExecutionScope)
{
	FPythonCommandEx PythonCommandEx;
	PythonCommandEx.Command = PythonCommand;
	PythonCommandEx.ExecutionMode = ExecutionMode;
	PythonCommandEx.FileExecutionScope = FileExecutionScope;

	const bool bResult = IPythonScriptPlugin::Get()->ExecPythonCommandEx(PythonCommandEx);
		
	CommandResult = MoveTemp(PythonCommandEx.CommandResult);
	LogOutput = MoveTemp(PythonCommandEx.LogOutput);
		
	return bResult;
}

bool UPythonScriptLibrary::ExecutePythonScript(const FString& PythonScript, const TArray<FString>& PythonInputs, const TArray<FString>& PythonOutputs)
{
	// We should never hit this!
	check(0);
	return false;
}

DEFINE_FUNCTION(UPythonScriptLibrary::execExecutePythonScript)
{
	struct FParamInfo
	{
		FProperty* Property = nullptr;
		uint8* PropAddr = nullptr;
		const TCHAR* ParamName = nullptr;
	};
	using FParamInfoArray = TArray<FParamInfo, TInlineAllocator<16>>;

	// Read the function params
	// We must always do this, regardless of whether Python is available/enabled, so that we step the bytecode correctly
	P_GET_PROPERTY_REF(FStrProperty, PythonScript);
	P_GET_TARRAY_REF(FString, PythonInputs);
	P_GET_TARRAY_REF(FString, PythonOutputs);
	// Read the input values
	FParamInfoArray InputParams;
	for (const FString& PythonInput : PythonInputs)
	{
		// Note: Medium term the Blueprint interpreter will change to provide us with a list of properties and 
		// instance pointers, rather than forcing us to jump in and out of its interpreter loop (via StepCompiledIn)
		Stack.StepCompiledIn<FProperty>(nullptr);
			
		FParamInfo& InputParam = InputParams.AddDefaulted_GetRef();
		InputParam.Property = Stack.MostRecentProperty;
		InputParam.PropAddr = Stack.MostRecentPropertyAddress;
		InputParam.ParamName = *PythonInput;
	}
	// Read the output values
	FParamInfoArray OutputParams;
	for (const FString& PythonOutput : PythonOutputs)
	{
		// Note: Medium term the Blueprint interpreter will change to provide us with a list of properties and 
		// instance pointers, rather than forcing us to jump in and out of its interpreter loop (via StepCompiledIn)
		Stack.StepCompiledIn<FProperty>(nullptr);
		check(Stack.MostRecentProperty && Stack.MostRecentPropertyAddress);

		FParamInfo& OutputParam = OutputParams.AddDefaulted_GetRef();
		OutputParam.Property = Stack.MostRecentProperty;
		OutputParam.PropAddr = Stack.MostRecentPropertyAddress;
		OutputParam.ParamName = *PythonOutput;
	}
	P_FINISH;

	// Note: Is is safe to return now that P_FINISH has been called

	if (!FPythonScriptPlugin::Get()->IsPythonAvailable())
	{
		*(bool*)RESULT_PARAM = false;
		return;
	}

	if (!FPythonScriptPlugin::Get()->IsPythonInitialized())
	{
		UE_LOGF(LogPython, Warning, "Attempt to execute python command before PythonScriptPlugin is initialized. Ensure your call is after OnPythonInitialized.");
		*(bool*)RESULT_PARAM = false;
		return;
	}

#if WITH_PYTHON
	auto ExecuteCustomPythonScriptImpl = [&]() -> bool
	{
		const FString FunctionErrorName = Stack.Node->GetName();
		const FString FunctionErrorCtxt = Stack.Node->GetOutermost()->GetName();

		// Local Python context used when executing this script
		// Has the inputs written into it prior to execution, and the outputs read from it after execution
		FPyObjectPtr PyTempGlobalDict = FPyObjectPtr::StealReference(PyDict_Copy(FPythonScriptPlugin::Get()->GetDefaultGlobalDict()));
		FPyObjectPtr PyTempLocalDict = PyTempGlobalDict;

		// Write the input values to the Python context
		for (const FParamInfo& InputParam : InputParams)
		{
			if (!InputParam.Property)
			{
				// an exception was generated when evaluating the input, provide the user script with None:
				PyDict_SetItemString(PyTempLocalDict, TCHAR_TO_UTF8(InputParam.ParamName), Py_None);
				continue;
			}
			check(InputParam.PropAddr);

			FPyObjectPtr PyInput;
			if (PyConversion::PythonizeProperty_Direct(InputParam.Property, InputParam.PropAddr, PyInput.Get()))
			{
				PyDict_SetItemString(PyTempLocalDict, TCHAR_TO_UTF8(InputParam.ParamName), PyInput);
			}
			else
			{
				PyUtil::SetPythonError(PyExc_TypeError, *FunctionErrorCtxt, *FString::Printf(TEXT("Failed to convert input property '%s' (%s) to attribute '%s' when calling function '%s' on '%s'"), *InputParam.Property->GetName(), *InputParam.Property->GetClass()->GetName(), InputParam.ParamName, *FunctionErrorName, *P_THIS_OBJECT->GetName()));
				return false;
			}
		}

		// Execute the Python command
		FPyObjectPtr PyResult = FPyObjectPtr::StealReference(FPythonScriptPlugin::Get()->EvalString(*PythonScript, TEXT("<string>"), Py_file_input, PyTempGlobalDict, PyTempLocalDict));

		// Read the output values from the Python context
		if (PyResult)
		{
			for (const FParamInfo& OutputParam : OutputParams)
			{
				PyObject* PyOutput = PyDict_GetItemString(PyTempLocalDict, TCHAR_TO_UTF8(OutputParam.ParamName));
				if (!PyOutput)
				{
					PyUtil::SetPythonError(PyExc_TypeError, *FunctionErrorCtxt, *FString::Printf(TEXT("Failed to find attribute '%s' for output property '%s' (%s) when calling function '%s' on '%s'"), OutputParam.ParamName, *OutputParam.Property->GetName(), *OutputParam.Property->GetClass()->GetName() , *FunctionErrorName, *P_THIS_OBJECT->GetName()));
					return false;
				}

				if (!PyConversion::NativizeProperty_Direct(PyOutput, OutputParam.Property, OutputParam.PropAddr))
				{
					PyUtil::SetPythonError(PyExc_TypeError, *FunctionErrorCtxt, *FString::Printf(TEXT("Failed to convert output property '%s' (%s) from attribute '%s' when calling function '%s' on '%s'"), *OutputParam.Property->GetName(), *OutputParam.Property->GetClass()->GetName(), OutputParam.ParamName, *FunctionErrorName, *P_THIS_OBJECT->GetName()));
					return false;
				}
			}

			return true;
		}

		return false;
	};

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		if (ExecuteCustomPythonScriptImpl())
		{
			Py_BEGIN_ALLOW_THREADS
			FPyWrapperTypeReinstancer::Get().ProcessPending();
			Py_END_ALLOW_THREADS
			*(bool*)RESULT_PARAM = true;
		}
		else
		{
			PyUtil::ReThrowPythonError();
			*(bool*)RESULT_PARAM = false;
		}
	}
#endif	// WITH_PYTHON
}
