// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PythonScriptTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "PythonScriptLibrary.generated.h"

UCLASS()
class UPythonScriptLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Check to see whether Python support is available in the current environment.
	 * @note This may return false until IsPythonConfigured is true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Python|Execution")
	static bool IsPythonAvailable();

	/**
	 * Check to see whether Python has been configured.
	 * @note Python may be configured but not yet be initialized (@see IsPythonInitialized).
	 */
	UFUNCTION(BlueprintCallable, Category = "Python|Execution")
	static bool IsPythonConfigured();

	/**
	 * Check to see whether Python has been initialized and is ready to use.
	 */
	UFUNCTION(BlueprintCallable, Category = "Python|Execution")
	static bool IsPythonInitialized();

	/**
	 * Force Python to be enabled and initialized, regardless of the settings that control its default enabled state.
	 * @return True if Python was requested to be enabled. Use IsPythonInitialized to verify that it actually initialized.
	 */
	UFUNCTION(BlueprintCallable, Category = "Python|Execution")
	static bool ForceEnablePythonAtRuntime();

	/**
	 * Execute the given Python command.
	 * @param PythonCommand The command to run. This may be literal Python code, or a file (with optional arguments) that you want to run.
	 * @return true if the command ran successfully, false if there were errors (the output log will show the errors).
	 */
	UFUNCTION(BlueprintCallable, Category = "Python|Execution")
	static bool ExecutePythonCommand(UPARAM(meta=(MultiLine=True)) const FString& PythonCommand);

	/**
	 * Execute the given Python command.
	 * @param PythonCommand The command to run. This may be literal Python code, or a file (with optional arguments) that you want to run.
	 * @param ExecutionMode Controls the mode used to execute the command.
	 * @param FileExecutionScope Controls the scope used when executing Python files.
	 * @param CommandResult The result of running the command. On success, for EvaluateStatement mode this will be the actual result of running the command, and will be None in all other cases. On failure, this will be the error information (typically a Python exception trace).
	 * @param LogOutput The log output captured while running the command.
	 * @return true if the command ran successfully, false if there were errors (the output log will show the errors).
	 */
	UFUNCTION(BlueprintCallable, Category = "Python|Execution", meta=(DisplayName="Execute Python Command (Advanced)", AdvancedDisplay="ExecutionMode,FileExecutionScope"))
	static bool ExecutePythonCommandEx(UPARAM(meta=(MultiLine=True)) const FString& PythonCommand, FString& CommandResult, TArray<FPythonLogOutputEntry>& LogOutput, const EPythonCommandExecutionMode ExecutionMode = EPythonCommandExecutionMode::ExecuteFile, const EPythonFileExecutionScope FileExecutionScope = EPythonFileExecutionScope::Private);

	/**
	 * Execute a Python script with argument marshaling.
	 * @param PythonScript This literal Python code to run.
	 * @param PythonInputs The variadic input argument names (internal; set by UK2Node_ExecutePythonScript).
	 * @param PythonInputs The variadic output argument names (internal; set by UK2Node_ExecutePythonScript).
	 * @return true if the command ran successfully, false if there were errors (the output log will show the errors).
	 */
    UFUNCTION(BlueprintCallable, CustomThunk, Category = "Python|Execution", meta=(Variadic, BlueprintInternalUseOnly="true"), DisplayName = "Execute Python Script")
    static bool ExecutePythonScript(UPARAM(meta=(MultiLine=True)) const FString& PythonScript, const TArray<FString>& PythonInputs, const TArray<FString>& PythonOutputs);
	DECLARE_FUNCTION(execExecutePythonScript);
};
