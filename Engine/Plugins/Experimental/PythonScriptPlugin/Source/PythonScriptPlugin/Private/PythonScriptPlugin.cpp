// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonScriptPlugin.h"
#include "PythonScriptPluginSettings.h"
#include "PythonScriptPluginStyle.h"
#include "PythonScriptRemoteExecution.h"
#include "PyGIL.h"
#include "PyCore.h"
#include "PySlate.h"
#include "PyEngine.h"
#include "PyEditor.h"
#include "PyConstant.h"
#include "PyConversion.h"
#include "PyMethodWithClosure.h"
#include "PyReferenceCollector.h"
#include "PyWrapperTypeRegistry.h"
#include "EngineAnalytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Logging/MessageLog.h"
#include "Modules/ModuleManager.h"
#include "UObject/PackageReload.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Char.h"
#include "Misc/CString.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/SourceLocationUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "Containers/Ticker.h"
#include "Features/IModularFeatures.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/Stats.h"
#include "String/Find.h"

#if WITH_EDITOR
#include "EditorSupportDelegates.h"
#include "EditorUtilities/EditorPythonExecuter.h"
#include "DesktopPlatformModule.h"
#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "Misc/ConfigCacheIni.h"
#include "Tasks/Task.h"
#include "Misc/MessageDialog.h"
#include "AssetViewUtils.h"
#include "IAssetTools.h"
#include "IContentBrowserDataModule.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserFileDataCore.h"
#include "ContentBrowserFileDataSource.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Misc/FeedbackContext.h"
#include "IPipInstall.h"
#include "PipInstallLauncher.h"
#endif	// WITH_EDITOR

#if PLATFORM_WINDOWS
	#include <stdio.h>
	#include <fcntl.h>
	#include <io.h>
#endif	// PLATFORM_WINDOWS
#include <locale.h>

#define LOCTEXT_NAMESPACE "PythonScriptPlugin"

#if WITH_PYTHON

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
	PyUtil::FPyApiBuffer FPythonScriptPlugin::Utf8String= PyUtil::TCHARToPyApiBuffer(TEXT("utf-8"));
#endif

namespace UE::Python::Private
{

static PyUtil::FPyApiBuffer NullPyArg = PyUtil::TCHARToPyApiBuffer(TEXT(""));
static PyUtil::FPyApiChar* NullPyArgPtrs[] = { NullPyArg.GetData() };

FPyObjectPtr MakeEmptyArgvList()
{
	// Make a list = [""]
	FPyObjectPtr PyArgvList = FPyObjectPtr::StealReference(PyList_New(1));
	PyList_SetItem(PyArgvList.Get(), 0, PyUnicode_FromString(""));

	return PyArgvList;
}

static bool bIsPythonLifeCycleDynamic = false;
static FAutoConsoleVariableRef CVarIsPythonLifeCycleDynamic(TEXT("Engine.Python.IsPythonLifeCycleDynamic"), bIsPythonLifeCycleDynamic, TEXT("If true python will not be initialized on boot in the editor. This allow the project to control when python should be initialized or not."), ECVF_ReadOnly);

static bool bIsPythonInRestrictiveMode = false;
static FAutoConsoleVariableRef CVarIsPythonInRestrictiveMode(TEXT("Engine.Python.IsPythonInRestrictiveMode"), bIsPythonInRestrictiveMode, TEXT("If true python will be initialized in the restrive mode of the editor. This allow the project to control what is acessible to Python and is still a work in progress."));

static bool bIsEnabledByDefault = true;
static FAutoConsoleVariableRef CVarIsPythonEnabledByDefault(TEXT("Engine.Python.IsEnabledByDefault"), bIsEnabledByDefault, TEXT("True if Python is enabled by default (after checking any command line overrides), or False if it should be considered disabled. This must be set prior to OnPostEngineInit."));

bool IsPythonEnabled()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("ForceEnablePython")))
	{
		UE_LOGF(LogPython, Log, "Python enabled via command-line flag '-ForceEnablePython'");
		return true;
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("DisablePython")))
	{
		UE_LOGF(LogPython, Log, "Python disabled via command-line flag '-DisablePython'");
		return false;
	}

	if (IsRunningCommandlet())
	{
		TArray<FString> DisablePythonForCommandlets;
		GConfig->GetArray(TEXT("PythonScriptPlugin"), TEXT("DisablePythonForCommandlet"), DisablePythonForCommandlets, GEditorIni);

		FString RunningCommandletName;
		if (!DisablePythonForCommandlets.IsEmpty() && FParse::Value(FCommandLine::Get(), TEXT("-run="), RunningCommandletName))
		{
			auto CleanCommandletName = [](FString& InOutCommandletName)
			{
				const FStringView CommandletSuffix = TEXTVIEW("Commandlet");
				if (InOutCommandletName.EndsWith(CommandletSuffix))
				{
					InOutCommandletName.LeftChopInline(CommandletSuffix.Len(), EAllowShrinking::No);
				}
			};

			CleanCommandletName(RunningCommandletName);
			for (FString& DisablePythonForCommandlet : DisablePythonForCommandlets)
			{
				CleanCommandletName(DisablePythonForCommandlet);
				if (DisablePythonForCommandlet == RunningCommandletName)
				{
					UE_LOGF(LogPython, Log, "Python disabled via config setting 'DisablePythonForCommandlet'");
					return false;
				}
			}
		}
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("EnablePython")))
	{
		UE_LOGF(LogPython, Log, "Python enabled via command-line flag '-EnablePython'");
		return true;
	}

	switch (GetDefault<UPythonScriptPluginUserSettings>()->EnablePythonOverride)
	{
		case EPythonEnabledOverrideState::Enable:
			UE_LOGF(LogPython, Log, "Python enabled via PythonScriptPluginUserSettings");
			return true;

		case EPythonEnabledOverrideState::Disable:
			UE_LOGF(LogPython, Log, "Python disabled via PythonScriptPluginUserSettings");
			return false;

		default:
			break;
	}

	if (UE::Python::Private::bIsEnabledByDefault)
	{
		UE_LOGF(LogPython, Log, "Python enabled via CVar 'Engine.Python.IsEnabledByDefault'");
		return true;
	}

	UE_LOGF(LogPython, Log, "Python disabled via CVar 'Engine.Python.IsEnabledByDefault'");
	return false;
}

void ShowPythonDisabledNotification()
{
	FMessageLog MessageLog("EditorErrors");
	MessageLog.SetCurrentPage(LOCTEXT("MessageLogPage", "Python"));
	
	const FText ErrorMessage = LOCTEXT("PythonCouldNotBeInitialized_Text", "Python could not be initialized and was disabled");

	MessageLog.Error(ErrorMessage);
	MessageLog.Info(LOCTEXT("PythonCouldNotBeInitialized_SubText", "See OutputLog for more information"));
	MessageLog.Notify(ErrorMessage, EMessageSeverity::Error, true /*bForce*/);
}

} // namespace UE::Python::Private

/** Util struct to set the sys.argv data for Python when executing a file with arguments */
struct FPythonScopedArgv
{
	FPythonScopedArgv(const TCHAR* InArgs)
	{
		if (InArgs && *InArgs)
		{
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
			// Moved argv changes to direct sys.argv object access since PySys_SetArgv is deprecated
			// Make new list and set it to sys.argv
			FPyObjectPtr PyArgvList = FPyObjectPtr::StealReference(PyList_New(0));
			FString NextToken;
			while (FParse::Token(InArgs, NextToken, false))
			{
				FPyObjectPtr PyArg;
				PyConversion::Pythonize(NextToken, PyArg.Get(), PyConversion::ESetErrorState::No);
				PyList_Append(PyArgvList.Get(), PyArg.Get());
			}

			PySys_SetObject("argv", PyArgvList.Get());
#else
			FString NextToken;
			while (FParse::Token(InArgs, NextToken, false))
			{
				PyCommandLineArgs.Add(PyUtil::TCHARToPyApiBuffer(*NextToken));
			}

			PyCommandLineArgPtrs.Reserve(PyCommandLineArgs.Num());
			for (PyUtil::FPyApiBuffer& PyCommandLineArg : PyCommandLineArgs)
			{
				PyCommandLineArgPtrs.Add(PyCommandLineArg.GetData());
			}

			PySys_SetArgvEx(PyCommandLineArgPtrs.Num(), PyCommandLineArgPtrs.GetData(), 0);
#endif 
		}
	}

	~FPythonScopedArgv()
	{
#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
		FPyObjectPtr PyArgvList = UE::Python::Private::MakeEmptyArgvList();
		PySys_SetObject("argv", PyArgvList.Get());
#else
		PySys_SetArgvEx(1, UE::Python::Private::NullPyArgPtrs, 0);
#endif
	}

	TArray<PyUtil::FPyApiBuffer> PyCommandLineArgs;
	TArray<PyUtil::FPyApiChar*> PyCommandLineArgPtrs;
};

struct FScopedEncodingGuard
{
	FScopedEncodingGuard()
	{
		// Python 3 changes the console mode from O_TEXT to O_BINARY which affects other uses of the console
		// So change the console mode back to its current setting after Py_Initialize has been called
#if PLATFORM_WINDOWS
		// We call _setmode here to cache the current state
		CA_SUPPRESS(6031)
		fflush(stdin);
		StdInMode = _setmode(_fileno(stdin), _O_TEXT);
		CA_SUPPRESS(6031)
		fflush(stdout);
		StdOutMode = _setmode(_fileno(stdout), _O_TEXT);
		CA_SUPPRESS(6031)
		fflush(stderr);
		StdErrMode = _setmode(_fileno(stderr), _O_TEXT);
#endif	// PLATFORM_WINDOWS

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 7
		// Python 3.7+ changes the C locale which affects functions using C string APIs
		// So change the C locale back to its current setting after Py_Initialize has been called
		if (const char* CurrentLocalePtr = setlocale(LC_ALL, nullptr))
		{
			CurrentLocale = ANSI_TO_TCHAR(CurrentLocalePtr);
		}
#endif	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 7
	}

	~FScopedEncodingGuard()
	{
#if PLATFORM_WINDOWS
		// We call _setmode here to restore the previous state
		if (StdInMode != -1)
		{
			CA_SUPPRESS(6031)
				fflush(stdin);
			CA_SUPPRESS(6031)
				_setmode(_fileno(stdin), StdInMode);
		}
		if (StdOutMode != -1)
		{
			CA_SUPPRESS(6031)
				fflush(stdout);
			CA_SUPPRESS(6031)
				_setmode(_fileno(stdout), StdOutMode);
		}
		if (StdErrMode != -1)
		{
			CA_SUPPRESS(6031)
				fflush(stderr);
			CA_SUPPRESS(6031)
				_setmode(_fileno(stderr), StdErrMode);
		}
#endif	// PLATFORM_WINDOWS

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 7
		// We call setlocale here to restore the previous state
		if (!CurrentLocale.IsEmpty())
		{
			setlocale(LC_ALL, TCHAR_TO_ANSI(*CurrentLocale));
		}
#endif	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 7
	}

private:
	FString CurrentLocale;
#if PLATFORM_WINDOWS
	int StdErrMode = -1;
	int StdOutMode = -1;
	int StdInMode = -1;
#endif	// PLATFORM_WINDOWS
};

FPythonCommandExecutor::FPythonCommandExecutor(IPythonScriptPlugin* InPythonScriptPlugin)
	: PythonScriptPlugin(InPythonScriptPlugin)
{
}

FName FPythonCommandExecutor::StaticName()
{
	static const FName CmdExecName = TEXT("Python");
	return CmdExecName;
}

FName FPythonCommandExecutor::GetName() const
{
	return StaticName();
}

FText FPythonCommandExecutor::GetDisplayName() const
{
	return LOCTEXT("PythonCommandExecutorDisplayName", "Python");
}

FText FPythonCommandExecutor::GetDescription() const
{
	return LOCTEXT("PythonCommandExecutorDescription", "Execute Python scripts (including files)");
}

FText FPythonCommandExecutor::GetHintText() const
{
	return LOCTEXT("PythonCommandExecutorHintText", "Enter Python script or a filename");
}

void FPythonCommandExecutor::GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out)
{
}

void FPythonCommandExecutor::GetExecHistory(TArray<FString>& Out)
{
	IConsoleManager::Get().GetConsoleHistory(TEXT("Python"), Out);
}

bool FPythonCommandExecutor::Exec(const TCHAR* Input)
{
	IConsoleManager::Get().AddConsoleHistoryEntry(TEXT("Python"), Input);

	UE_LOGF(LogPython, Log, "%ls", Input);

	PythonScriptPlugin->ExecPythonCommand(Input);

	return true;
}

bool FPythonCommandExecutor::AllowHotKeyClose() const
{
	return true;
}

bool FPythonCommandExecutor::AllowMultiLine() const
{
	return true;
}

FInputChord FPythonCommandExecutor::GetHotKey() const
{
#if WITH_EDITOR
	return FGlobalEditorCommonCommands::Get().OpenConsoleCommandBox->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
#else
	return FInputChord();
#endif
}

FInputChord FPythonCommandExecutor::GetIterateExecutorHotKey() const
{
#if WITH_EDITOR
	return FGlobalEditorCommonCommands::Get().SelectNextConsoleExecutor->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
#else
	return FInputChord();
#endif
}

FPythonREPLCommandExecutor::FPythonREPLCommandExecutor(IPythonScriptPlugin* InPythonScriptPlugin)
	: PythonScriptPlugin(InPythonScriptPlugin)
{
}

FName FPythonREPLCommandExecutor::StaticName()
{
	static const FName CmdExecName = TEXT("PythonREPL");
	return CmdExecName;
}

FName FPythonREPLCommandExecutor::GetName() const
{
	return StaticName();
}

FText FPythonREPLCommandExecutor::GetDisplayName() const
{
	return LOCTEXT("PythonREPLCommandExecutorDisplayName", "Python (REPL)");
}

FText FPythonREPLCommandExecutor::GetDescription() const
{
	return LOCTEXT("PythonREPLCommandExecutorDescription", "Execute a single Python statement and show its result");
}

FText FPythonREPLCommandExecutor::GetHintText() const
{
	return LOCTEXT("PythonREPLCommandExecutorHintText", "Enter a Python statement");
}

void FPythonREPLCommandExecutor::GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out)
{
}

void FPythonREPLCommandExecutor::GetExecHistory(TArray<FString>& Out)
{
	IConsoleManager::Get().GetConsoleHistory(TEXT("PythonREPL"), Out);
}

bool FPythonREPLCommandExecutor::Exec(const TCHAR* Input)
{
	IConsoleManager::Get().AddConsoleHistoryEntry(TEXT("PythonREPL"), Input);

	UE_LOGF(LogPython, Log, "%ls", Input);

	FPythonCommandEx PythonCommand;
	PythonCommand.ExecutionMode = EPythonCommandExecutionMode::ExecuteStatement;
	PythonCommand.Command = Input;
	PythonScriptPlugin->ExecPythonCommandEx(PythonCommand);

	return true;
}

bool FPythonREPLCommandExecutor::AllowHotKeyClose() const
{
	return true;
}

bool FPythonREPLCommandExecutor::AllowMultiLine() const
{
	return true;
}

FInputChord FPythonREPLCommandExecutor::GetHotKey() const
{
#if WITH_EDITOR
	return FGlobalEditorCommonCommands::Get().OpenConsoleCommandBox->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
#else
	return FInputChord();
#endif
}

FInputChord FPythonREPLCommandExecutor::GetIterateExecutorHotKey() const
{
#if WITH_EDITOR
	return FGlobalEditorCommonCommands::Get().SelectNextConsoleExecutor->GetActiveChord(EMultipleKeyBindingIndex::Primary).Get();
#else
	return FInputChord();
#endif
}

#if WITH_EDITOR
class FPythonCommandMenuImpl : public IPythonCommandMenu
{
public:
	FPythonCommandMenuImpl()
		: bRecentsFilesDirty(false)
	{
	}

	virtual void OnStartupMenu() override
	{
		LoadConfig();

		RegisterMenus();
	}

	virtual void OnShutdownMenu() override
	{
		UToolMenus::UnregisterOwner(this);

		// Write to file
		if (bRecentsFilesDirty)
		{
			SaveConfig();
			bRecentsFilesDirty = false;
		}
	}

	virtual void OnRunFile(const FString& InFile, bool bAdd) override
	{
		if (bAdd)
		{
			int32 Index = RecentsFiles.Find(InFile);
			if (Index != INDEX_NONE)
			{
				// If already in the list but not at the last position
				if (Index != RecentsFiles.Num() - 1)
				{
					RecentsFiles.RemoveAt(Index);
					RecentsFiles.Add(InFile);
					bRecentsFilesDirty = true;
				}
			}
			else
			{
				if (RecentsFiles.Num() >= MaxNumberOfFiles)
				{
					RecentsFiles.RemoveAt(0);
				}
				RecentsFiles.Add(InFile);
				bRecentsFilesDirty = true;
			}
		}
		else
		{
			if (RecentsFiles.RemoveSingle(InFile) > 0)
			{
				bRecentsFilesDirty = true;
			}
		}
	}

private:
	const TCHAR* STR_ConfigSection = TEXT("Python");
	const TCHAR* STR_ConfigDirectoryKey = TEXT("LastDirectory");
	const FName NAME_ConfigRecentsFilesyKey = TEXT("RecentsFiles");
	static const int32 MaxNumberOfFiles = 10;

	void LoadConfig()
	{
		RecentsFiles.Reset();

		GConfig->GetString(STR_ConfigSection, STR_ConfigDirectoryKey, LastDirectory, GEditorPerProjectIni);

		const FConfigSection* Sec = GConfig->GetSection(STR_ConfigSection, false, GEditorPerProjectIni);
		if (Sec)
		{
			TArray<FConfigValue> List;
			Sec->MultiFind(NAME_ConfigRecentsFilesyKey, List);

			int32 ListNum = FMath::Min(List.Num(), MaxNumberOfFiles);

			RecentsFiles.Reserve(ListNum);
			for (int32 Index = 0; Index < ListNum; ++Index)
			{
				RecentsFiles.Add(List[Index].GetValue());
			}
		}
	}

	void SaveConfig() const
	{
		GConfig->SetString(STR_ConfigSection, STR_ConfigDirectoryKey, *LastDirectory, GEditorPerProjectIni);

		GConfig->RemoveKeyFromSection(STR_ConfigSection, NAME_ConfigRecentsFilesyKey, GEditorPerProjectIni);
		for (int32 Index = RecentsFiles.Num() - 1; Index >= 0; --Index)
		{
			GConfig->AddToSection(STR_ConfigSection, NAME_ConfigRecentsFilesyKey, RecentsFiles[Index], GEditorPerProjectIni);
		}

		GConfig->Flush(false);
	}

	void MakeRecentPythonScriptMenu(UToolMenu* InMenu)
	{
		FToolMenuOwnerScoped OwnerScoped(this);
		FToolMenuSection& Section = InMenu->AddSection("Files");
		for (int32 Index = RecentsFiles.Num() - 1; Index >= 0; --Index)
		{
			Section.AddMenuEntry(
				NAME_None,
				FText::FromString(RecentsFiles[Index]),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateRaw(this, &FPythonCommandMenuImpl::Menu_ExecutePythonRecent, Index))
			);
		}

		FToolMenuSection& ClearSection = InMenu->AddSection("Clear");
		Section.AddMenuEntry(
			"ClearRecentPython",
			LOCTEXT("ClearRecentPython", "Clear Recent Python Scripts"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &FPythonCommandMenuImpl::Menu_ClearRecentPython))
		);
	}

	void RegisterMenus()
	{
		FToolMenuOwnerScoped OwnerScoped(this);
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");
		FToolMenuSection& Section = Menu->AddSection("Python", LOCTEXT("Python", "Python"));//, FToolMenuInsert("FileLoadAndSave", EToolMenuInsertType::After));
		Section.AddMenuEntry(
			"OpenPython",
			LOCTEXT("OpenPython", "Execute Python Script..."),
			LOCTEXT("OpenPythonTooltip", "Open a Python Script file and Execute it."),
			FSlateIcon(FPythonScriptPluginEditorStyle::Get().GetStyleSetName(), "Icons.PythonExecute"),
			FUIAction(FExecuteAction::CreateRaw(this, &FPythonCommandMenuImpl::Menu_ExecutePython))
		);
		Section.AddSubMenu(
			"RecentPythonsSubMenu",
			LOCTEXT("RecentPythonsSubMenu", "Recent Python Scripts"),
			LOCTEXT("RecentPythonsSubMenu_ToolTip", "Select a recent Python Script file and Execute it."),
			FNewToolMenuDelegate::CreateRaw(this, &FPythonCommandMenuImpl::MakeRecentPythonScriptMenu), false,
			FSlateIcon(FPythonScriptPluginEditorStyle::Get().GetStyleSetName(), "Icons.PythonRecent")
		);
	}

	void Menu_ExecutePythonRecent(int32 Index)
	{
		if (RecentsFiles.IsValidIndex(Index))
		{
			FString PyCopied = RecentsFiles[Index];
			GEngine->Exec(NULL, *FString::Printf(TEXT("py \"%s\""), *PyCopied));
		}
	}

	void Menu_ClearRecentPython()
	{
		if (RecentsFiles.Num() > 0)
		{
			RecentsFiles.Reset();
			bRecentsFilesDirty = true;
		}
	}

	void Menu_ExecutePython()
	{
		TArray<FString> OpenedFiles;
		FString DefaultDirectory = LastDirectory;

		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform)
		{
			bool bOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				LOCTEXT("ExecutePython", "Execute Python Script").ToString(),
				DefaultDirectory,
				TEXT(""),
				TEXT("Python files|*.py|"),
				EFileDialogFlags::None,
				OpenedFiles
			);

			if (bOpened && OpenedFiles.Num() > 0)
			{
				if (DefaultDirectory != LastDirectory)
				{
					LastDirectory = DefaultDirectory;
					bRecentsFilesDirty = true;
				}

				GEngine->Exec(NULL, *FString::Printf(TEXT("py \"%s\""), *OpenedFiles.Last()));
			}
		}
	}

private:

	TArray<FString> RecentsFiles;
	FString LastDirectory;

	bool bRecentsFilesDirty;
};
#endif // WITH_EDITOR

#endif	// WITH_PYTHON

FPythonScriptPlugin::FPythonScriptPlugin()
#if WITH_PYTHON
	: CmdExec(this)
	, CmdREPLExec(this)
#endif	// WITH_PYTHON
{
}

bool FPythonScriptPlugin::IsPythonAvailable() const
{
#if WITH_PYTHON
	return bIsEnabled;
#else
	return false;
#endif
}

bool FPythonScriptPlugin::IsPythonConfigured() const
{
#if WITH_PYTHON
	return bIsConfigured;
#else
	return false;
#endif
}

bool FPythonScriptPlugin::IsPythonInitialized() const
{
#if WITH_PYTHON
	return bIsFullyInitialized;
#else
	return false;
#endif
}

bool FPythonScriptPlugin::ForceEnablePythonAtRuntime(UE::FSourceLocation Location)
{
	return ForceEnablePythonAtRuntime(UE::SourceLocation::Full(Location).ToString());
}

bool FPythonScriptPlugin::ForceEnablePythonAtRuntime(const FString& Location)
{
#if WITH_PYTHON
	if (!bIsForceEnabledAtRuntime)
	{
		// Promote the log to a warning if called when Python is disabled by default, or was explicitly disabled by the user
		if (!UE::Python::Private::bIsEnabledByDefault || GetDefault<UPythonScriptPluginUserSettings>()->EnablePythonOverride == EPythonEnabledOverrideState::Disable)
		{
			UE_LOGF(LogPython, Warning, "Python enabled via IPythonScriptPlugin::ForceEnablePythonAtRuntime: %ls", *Location);
		}
		else
		{
			UE_LOGF(LogPython, Log, "Python enabled via IPythonScriptPlugin::ForceEnablePythonAtRuntime: %ls", *Location);
		}
		bIsForceEnabledAtRuntime = true;

		if (bIsConfigured && !bIsInterpreterInitialized)
		{
			// ForceEnablePythonAtRuntime was called after OnPostEngineInit, so re-run this now that bIsForceEnabledAtRuntime is true
			ConfigureAndInitializePython();
			if (bIsInterpreterInitialized && bHasTicked)
			{
				// If we've already ticked once, then we can also run the start-up scripts
				RunStartupScripts();
			}
		}
	}
	return true;
#else
	return false;
#endif
}

void FPythonScriptPlugin::ForceDisablePythonAtRuntime(UE::FSourceLocation Location)
{
#if WITH_PYTHON
	if (bIsInterpreterInitialized)
	{
		const FString CallerLocation = UE::SourceLocation::Full(Location).ToString();
		if (!UE::Python::Private::bIsEnabledByDefault || !UE::Python::Private::bIsPythonLifeCycleDynamic)
		{
			UE_LOGF(LogPython, Warning, "Python disabled via IPythonScriptPlugin::DisablePythonAtRuntime: %ls", *CallerLocation);
		}
		else
		{
			UE_LOGF(LogPython, Log, "Python disabled via IPythonScriptPlugin::DisablePythonAtRuntime: %ls", *CallerLocation);
		}

		ShutdownPython();
		bIsForceEnabledAtRuntime = false;
	}
#endif
}

bool FPythonScriptPlugin::ExecPythonCommand(const TCHAR* InPythonCommand)
{
	FPythonCommandEx PythonCommand;
	PythonCommand.Command = InPythonCommand;
	return ExecPythonCommandEx(PythonCommand);
}

bool FPythonScriptPlugin::ExecPythonCommandEx(FPythonCommandEx& InOutPythonCommand)
{
#if WITH_PYTHON
	if (!IsPythonAvailable())
#endif
	{
		InOutPythonCommand.CommandResult = TEXT("Python is not available!");
		ensureAlwaysMsgf(false, TEXT("%s"), *InOutPythonCommand.CommandResult);
		return false;
	}

#if WITH_PYTHON
	if (!bIsInterpreterInitialized)
	{
		InOutPythonCommand.CommandResult =
			TEXT("Attempt to execute python command before PythonScriptPlugin is initialized. Ensure your call is after OnPythonInitialized.");

		UE_LOGF(LogPython, Warning, "%ls", *InOutPythonCommand.CommandResult);
		return false;
	}

	if (InOutPythonCommand.ExecutionMode == EPythonCommandExecutionMode::ExecuteFile)
	{
		// The EPythonCommandExecutionMode::ExecuteFile name is misleading as it is used to run literal code or a .py file. Detect
		// if the user supplied a .py files. The command can have the python pathname with spaces, command line parameter(s) and could be quoted.
		//   C:\My Scripts\Test.py -param1 -param2     -> Ok
		//   "C:\My Scripts\Test.py  " -param1 -param2 -> Ok
		//   "C:\My Scripts\Test.py"-param1 -param2    -> Ok
		//   C:\My Scripts\Test.py "param with spaces" -> OK
		//   C:\My Scripts\Test.py-param1 -param2      -> Error missing a space between .py and -param1
		//   "C:\My Scripts\Test.py                    -> Error missing closing quote.
		//   C:\My Scripts\Test.py  "                  -> Error missing opening quote.
		//   test_wrapper_types.py                     -> Search the 'sys.path' to find the script.
		auto TryExtractPathnameAndCommand = [&InOutPythonCommand](FString& OutExtractedFilename, FString& OutExtractedCommand) -> bool
		{
			const TCHAR* PyFileExtension = TEXT(".py");
			int32 Pos = UE::String::FindFirst(InOutPythonCommand.Command, PyFileExtension);
			if (Pos == INDEX_NONE)
			{
				return false; // no .py file extension found.
			}

			int32 EndPathnamePos = Pos + FCString::Strlen(PyFileExtension);
			OutExtractedFilename = InOutPythonCommand.Command.Left(EndPathnamePos);
			bool bCommandQuoted = false;
			
			// The caller may quote the pathname if it contains space(s). Trim a leading quote, if any. (Any trailing quote is already removed by the Left() command).
			if (OutExtractedFilename.StartsWith(TEXT("\"")))
			{
				OutExtractedFilename.RemoveAt(0);
				bCommandQuoted = true;
			}

			// If the pathname started with a quote, expect a closing quote after the .py.
			if (bCommandQuoted)
			{
				if (EndPathnamePos == InOutPythonCommand.Command.Len())
				{
					return false; // Missing the closing quote.
				}

				// Scan after the .py to find the closing quote.
				for (int32 CurrPos = EndPathnamePos; CurrPos < InOutPythonCommand.Command.Len(); ++CurrPos)
				{
					if (InOutPythonCommand.Command[CurrPos] == TEXT('\"'))
					{
						EndPathnamePos = CurrPos + 1; // +1 to put the end just after the quote like a std::end() iterator.
						break;
					}
					else if (FChar::IsWhitespace(InOutPythonCommand.Command[CurrPos]))
					{
						continue; // It is legal to have blank space after the .py.
					}
					else
					{
						return false; // Invalid character found after .py.
					}
				}
			}
			// Some characters appears after the .py
			else if (EndPathnamePos < InOutPythonCommand.Command.Len() && !FChar::IsWhitespace(InOutPythonCommand.Command[EndPathnamePos]))
			{
				return false; // Some non-blank characters are there and we don't expect a closing quote. This is not a valid command Ex: C:\MyScript.py-t
			}

			// Quote/re-quote the command. This allows Python to set sys.argv/sys.argc property if the pathname contains space. (So parts of the pathname are not interpreted as arguments)
			OutExtractedCommand = FString::Printf(TEXT("\"%s\""), *OutExtractedFilename);

			// Append the arguments (if any).
			if (EndPathnamePos < InOutPythonCommand.Command.Len())
			{
				OutExtractedCommand += InOutPythonCommand.Command.Mid(EndPathnamePos);
			}

			return true; // Pathname, command and argument were successfully parsed.
		};

		FString ExtractedFilename;
		FString ExtractedCommand;

		if (TryExtractPathnameAndCommand(ExtractedFilename, ExtractedCommand))
		{
			return RunFile(*ExtractedFilename, *ExtractedCommand, InOutPythonCommand);
		}
		else
		{
			return RunString(InOutPythonCommand);
		}
	}
	else
	{
		return RunString(InOutPythonCommand);
	}	
#endif	// WITH_PYTHON
}

FString FPythonScriptPlugin::GetInterpreterExecutablePath() const
{
#if WITH_PYTHON
	return PyUtil::GetInterpreterExecutablePath();
#else	// WITH_PYTHON
	return FString();
#endif	// WITH_PYTHON
}

FSimpleMulticastDelegate& FPythonScriptPlugin::OnPythonConfigured()
{
	return OnPythonConfiguredDelegate;
}

FSimpleMulticastDelegate& FPythonScriptPlugin::OnPythonInitialized()
{
	return OnPythonInitializedDelegate;
}

FSimpleMulticastDelegate& FPythonScriptPlugin::OnPythonShutdown()
{
	return OnPythonShutdownDelegate;
}

void FPythonScriptPlugin::StartupModule()
{
#if WITH_EDITOR
	FEditorPythonExecuter::OnStartupModule();
#endif

#if WITH_PYTHON
	LLM_SCOPE_BYNAME(TEXT("PythonScriptPlugin"));

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FPythonScriptPlugin::OnPostEngineInit);
#endif	// WITH_PYTHON
}

#if WITH_PYTHON
void FPythonScriptPlugin::OnPostEngineInit()
{
	TickOnceHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float DeltaTime)
	{
		bHasTicked = true;
		return false;
	}));

#if WITH_EDITOR
	// Register a menu handler to enable Python if it is currently disabled at runtime
	if (UToolMenus::IsToolMenuUIEnabled())
	{
		static const FName MenuName = "OutputLog.ConsoleInputBox.CmdExecMenu";
		if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(MenuName))
		{
			Menu->AddDynamicSection("DynamicPythonInit", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				FPythonScriptPlugin* This = FPythonScriptPlugin::Get();

				if (This->IsPythonConfigured() && !This->IsPythonAvailable())
				{
					FToolMenuSection& Section = InMenu->AddSection("PythonInit");
					Section.AddMenuEntry(
						"EnablePython",
						LOCTEXT("Menu.EnablePython.Label", "Enable Python"),
						LOCTEXT("Menu.EnablePython.ToolTip", "Enable Python for the current editor session.\nTo enable Python permanently, see 'Editor Preferences -> Python -> Enable Python Override'."),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([This] { This->ForceEnablePythonAtRuntime(); }),
							FCanExecuteAction::CreateLambda([] { return true; })
						),
						EUserInterfaceActionType::Button
					);
				}
			}));
		}
	}
#endif

	ConfigureAndInitializePython();
}

void FPythonScriptPlugin::ConfigureAndInitializePython()
{
	LLM_SCOPE_BYNAME(TEXT("PythonScriptPlugin"));

	// Determine whether Python is enabled in OnPostEngineInit so that other things have time to override the CVar
	// IsPythonAvailable() will always return false prior to configuring bIsEnabled
	bIsEnabled = bIsForceEnabledAtRuntime || UE::Python::Private::IsPythonEnabled();
	
	bIsConfigured = true;
	OnPythonConfiguredDelegate.Broadcast();
	OnPythonConfiguredDelegate.Clear();

	if (UE::Python::Private::bIsPythonLifeCycleDynamic && !bIsForceEnabledAtRuntime)
	{
		return;
	}

	if (!IsPythonAvailable())
	{
		return;
	}

	if (!InitializePython())
	{
		bIsEnabled = false;
		UE_LOGF(LogPython, Error, "Python could not be initialized and was disabled.")
		if (!FApp::IsUnattended() && !IsRunningCommandlet() && !GIsRunningUnattendedScript)
		{
			FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::EndOfEngineInit, &UE::Python::Private::ShowPythonDisabledNotification);
		}
		return;
	}

	IModularFeatures::Get().RegisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), &CmdExec);
	IModularFeatures::Get().RegisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), &CmdREPLExec);

	check(!RemoteExecution);
	RemoteExecution = MakeUnique<FPythonScriptRemoteExecution>(this);

	FCoreDelegates::OnPreExit.AddRaw(this, &FPythonScriptPlugin::ShutdownPython);

#if WITH_EDITOR
	FPythonScriptPluginEditorStyle::Get();
	if (UToolMenus::IsToolMenuUIEnabled())
	{
		CmdMenu = MakeUnique<FPythonCommandMenuImpl>();
		CmdMenu->OnStartupMenu();

		UToolMenus::Get()->RegisterStringCommandHandler("Python", FToolMenuExecuteString::CreateLambda([this](const FString& InString, const FToolMenuContext& InContext) {
			ExecPythonCommand(*InString);
		}));
	}
#endif // WITH_EDITOR
}

#endif // WITH_PYTHON

void FPythonScriptPlugin::ShutdownModule()
{
#if WITH_EDITOR
	FEditorPythonExecuter::OnShutdownModule();
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
#endif

#if WITH_PYTHON
	ShutdownPython();
#endif	// WITH_PYTHON
}

bool FPythonScriptPlugin::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_PYTHON
	if (FParse::Command(&Cmd, TEXT("PY")))
	{
		if (IsPythonInitialized())
		{
			ExecPythonCommand(Cmd);
		}
		else
		{
			DeferredCommands.Add(Cmd);
		}
		return true;
	}
#endif	// WITH_PYTHON
	return false;
}

void FPythonScriptPlugin::PreChange(const UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info)
{
}

void FPythonScriptPlugin::PostChange(const UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info)
{
#if WITH_PYTHON
	OnAssetUpdated(Enum);
#endif //WITH_PYTHON
}

void FPythonScriptPlugin::AddStartupScriptAllowPath(const FName Owner, const FString& Path)
{
	StartupScriptAllowedPaths.AddAllowListItem(Owner, Path);
}

void FPythonScriptPlugin::RemoveStartupScriptAllowPath(const FName Owner)
{
	StartupScriptAllowedPaths.UnregisterOwner(Owner);
}

#if WITH_PYTHON

bool FPythonScriptPlugin::InitializePython()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::InitializePython)

	bIsInterpreterInitialized = true;

	FScopedSlowTask SlowTask(1, LOCTEXT("InitializingPython", "Initializing Python..."));
	SlowTask.Visibility = ESlowTaskVisibility::Important; // this function can be very slow, users will benefit from our messages
	SlowTask.MakeDialog();

	const UPythonScriptPluginSettings* PythonPluginSettings = GetDefault<UPythonScriptPluginSettings>();

	// HACK: This env var must be cleared or it carries into python subprocesses and python sys.executable detection breaking venvs
	FPlatformMisc::SetEnvironmentVar(TEXT("PYTHONEXECUTABLE"),TEXT(""));

	// Set-up the correct program name
	{
		FString ProgramName = FPlatformProcess::GetCurrentWorkingDirectory() / FPlatformProcess::ExecutableName(false);
		FPaths::NormalizeFilename(ProgramName);
		PyProgramName = PyUtil::TCHARToPyApiBuffer(*ProgramName);
	}

	// Set-up the correct home path
	{
		// Build the full Python directory (UE_PYTHON_DIR may be relative to the engine directory for portability)
		FString PythonDir = UTF8_TO_TCHAR(UE_PYTHON_DIR);
		PythonDir.ReplaceInline(TEXT("{ENGINE_DIR}"), *FPaths::EngineDir(), ESearchCase::CaseSensitive);
		FPaths::NormalizeDirectoryName(PythonDir);
		FPaths::RemoveDuplicateSlashes(PythonDir);
		PyHomePath = PyUtil::TCHARToPyApiBuffer(*PythonDir);
	}

	// Initialize the Python interpreter
	{
		static_assert(PY_MAJOR_VERSION >= 3, "Unreal Engine Python integration doesn't support versions prior to Python 3.x");
		UE_LOGF(LogPython, Log, "Using Python %d.%d.%d", PY_MAJOR_VERSION, PY_MINOR_VERSION, PY_MICRO_VERSION);

		FScopedEncodingGuard EncodingGuard;

		// Check if the interpreter is should run in isolation mode.
		int IsolatedInterpreterFlag = PythonPluginSettings->bIsolateInterpreterEnvironment ? 1 : 0;

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
		// Pre-initialize python with utf-8 encoding and possibly isolated mode
		PyPreConfig PreConfig;
		PyPreConfig_InitIsolatedConfig(&PreConfig);

		PreConfig.parse_argv = 0;
		PreConfig.utf8_mode = 1;
		PreConfig.isolated = IsolatedInterpreterFlag;
		PreConfig.use_environment = !IsolatedInterpreterFlag;

		if (PyStatus Status = Py_PreInitialize(&PreConfig);
			!ensure(!PyStatus_Exception(Status)))
		{
			PyUtil::LogPythonError();
			UE_LOGF(LogPython, Error, "Py_PreInitialize failed! Error: '%s'. Python will be disabled.", Status.err_msg)
			return false;
		}

		// Create empty init config
		PyConfig_InitIsolatedConfig(&ModulePyConfig);
		ModulePyConfig.use_environment = !IsolatedInterpreterFlag;
#else
		Py_IgnoreEnvironmentFlag = IsolatedInterpreterFlag; // If not zero, ignore all PYTHON* environment variables, e.g. PYTHONPATH, PYTHONHOME, that might be set.
#endif

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
		ModulePyConfig.isolated = IsolatedInterpreterFlag;
		ModulePyConfig.stdio_encoding = Utf8String.GetData();
#elif PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 4
		Py_IsolatedFlag = IsolatedInterpreterFlag; // If not zero, sys.path contains neither the script's directory nor the user's site-packages directory.
		Py_SetStandardStreamEncoding("utf-8", nullptr);
#endif	// PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 4

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
		ModulePyConfig.program_name = PyProgramName.GetData();
		ModulePyConfig.home = PyHomePath.GetData();
		ModulePyConfig.install_signal_handlers = 0;
		ModulePyConfig.safe_path = 0;

		if (PyStatus Status = Py_InitializeFromConfig(&ModulePyConfig);
			!ensure(!PyStatus_Exception(Status)))
		{
			PyUtil::LogPythonError();
			UE_LOGF(LogPython, Error, "Py_InitializeFromConfig failed! Error: '%s'. Python will be disabled.", Status.err_msg)
			return false;
		}
#else
		Py_SetProgramName(PyProgramName.GetData());
		Py_SetPythonHome(PyHomePath.GetData());
		Py_InitializeEx(0); // 0 so Python doesn't override any signal handling
#endif

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION < 7
		// NOTE: Since 3.7, those functions are called by Py_InitializeEx()
		// 
		// Ensure Python supports multiple threads via the GIL, as UE GC runs over multiple threads, 
		// which may invoke FPyReferenceCollector::AddReferencedObjects on a background thread...
		if (!PyEval_ThreadsInitialized())
		{
			PyEval_InitThreads();
		}
#endif // PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION < 7
	}
	
	// Setup UE conventions for the embedded interpreter environment
	{
#if PY_MAJOR_VERSION >=3 && PY_MINOR_VERSION >= 11
		// Set default argv to [""]
		FPyObjectPtr PyArgvList = UE::Python::Private::MakeEmptyArgvList();
		if (!ensure(PySys_SetObject("argv", PyArgvList.Get()) == 0))
		{
			PyUtil::LogPythonError();
			UE_LOGF(LogPython, Error, "PySys_SetObject(\"argv\") failed! Python will be disabled.")
			return false;
		}
#else
		PySys_SetArgvEx(1, UE::Python::Private::NullPyArgPtrs, 0);
#endif

		// Enable developer warnings if requested
		if (IsDeveloperModeEnabled())
		{
			PyUtil::EnableDeveloperWarnings();
			ensure(!PyUtil::LogPythonError());
		}

		// Check if the user wants type hinting. (In the stub and/or Docstrings).
		PyGenUtil::SetTypeHintingMode(GetTypeHintingMode());

		// Initialize our custom method type as we'll need it when generating bindings
		if (!ensure(InitializePyMethodWithClosure()))
		{
			return false;
		}

		// Initialize our custom constant type as we'll need it when generating bindings
		if (!ensure(InitializePyConstant()))
		{
			return false;
		}

		// Some users have PyImport_AddModule return null below, so we want to make sure errors are cleared before calling it
		ensure(!PyUtil::LogPythonError());

		PyObject* PyMainModule = PyImport_AddModule("__main__");
		if (!ensure(PyMainModule))
		{
			PyUtil::LogPythonError();
			UE_LOGF(LogPython, Error, "Failed to initialize '__main__'! This typically means the Python SDK is missing or could not be loaded")
			return false;
		}
		PyDefaultGlobalDict = FPyObjectPtr::NewReference(PyModule_GetDict(PyMainModule));
		PyDefaultLocalDict = PyDefaultGlobalDict;

		PyConsoleGlobalDict = FPyObjectPtr::StealReference(PyDict_Copy(PyDefaultGlobalDict));
		PyConsoleLocalDict = PyConsoleGlobalDict;

#if WITH_EDITOR
		FEditorSupportDelegates::PrepareToCleanseEditorObject.AddRaw(this, &FPythonScriptPlugin::OnPrepareToCleanseEditorObject);
#endif	// WITH_EDITOR
	}

	// Set-up the known Python script paths
	{
		PyUtil::AddSystemPath(FPaths::ConvertRelativePathToFull(FPlatformProcess::UserDir() / FApp::GetEpicProductIdentifier() / TEXT("Python")));

		TArray<FString> RootPaths;
		FPackageName::QueryRootContentPaths(RootPaths);
		for (const FString& RootPath : RootPaths)
		{
			const FString RootFilesystemPath = FPackageName::LongPackageNameToFilename(RootPath);
			RegisterModulePaths(RootFilesystemPath);
		}

		for (const FDirectoryPath& AdditionalPath : PythonPluginSettings->AdditionalPaths)
		{
			PyUtil::AddSystemPath(FPaths::ConvertRelativePathToFull(AdditionalPath.Path));
		}

		TArray<FString> SystemEnvPaths;
		FPlatformMisc::GetEnvironmentVariable(TEXT("UE_PYTHONPATH")).ParseIntoArray(SystemEnvPaths, FPlatformMisc::GetPathVarDelimiter());
		for (const FString& SystemEnvPath : SystemEnvPaths)
		{
			PyUtil::AddSystemPath(SystemEnvPath);
		}

		FPackageName::OnContentPathMounted().AddRaw(this, &FPythonScriptPlugin::OnContentPathMounted);
		FPackageName::OnContentPathDismounted().AddRaw(this, &FPythonScriptPlugin::OnContentPathDismounted);
		FCoreUObjectDelegates::OnPackageReloaded.AddRaw(this, &FPythonScriptPlugin::OnAssetReload);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		AssetRegistryModule.Get().OnAssetRenamed().AddRaw(this, &FPythonScriptPlugin::OnAssetRenamed);
		AssetRegistryModule.Get().OnAssetRemoved().AddRaw(this, &FPythonScriptPlugin::OnAssetRemoved);
	}

	// Initialize the Unreal Python module
	{
		// Create the top-level "unreal" module
		PyUnrealModule = FPyObjectPtr::NewReference(PyImport_AddModule("unreal"));

		// Import "unreal" into the console by default
		PyDict_SetItemString(PyConsoleGlobalDict, "unreal", PyUnrealModule);

		// Initialize the and import the "core" module
		PyCore::InitializeModule();
		ImportUnrealModule(TEXT("core"));

		// Initialize the and import the "slate" module
		PySlate::InitializeModule();
		ImportUnrealModule(TEXT("slate"));

		// Initialize the and import the "engine" module
		PyEngine::InitializeModule();
		ImportUnrealModule(TEXT("engine"));

#if WITH_EDITOR
		// Initialize the and import the "editor" module
		PyEditor::InitializeModule();
		ImportUnrealModule(TEXT("editor"));
#endif	// WITH_EDITOR

		FPyWrapperTypeRegistry::Get().OnModuleDirtied().AddRaw(this, &FPythonScriptPlugin::OnModuleDirtied);
		FModuleManager::Get().OnModulesChanged().AddRaw(this, &FPythonScriptPlugin::OnModulesChanged);


		UE::Tasks::FTask PipTask;
		// We don't support pip installs in a restrictive environment at the moment.
		if (!UE::Python::Private::bIsPythonInRestrictiveMode)
		{
#if WITH_EDITOR
			IPipInstall& PipInstaller = IPipInstall::Get();
			PipTask =
				UE::Tasks::Launch(
					TEXT("PipInstallAsync"),
					[&PipInstaller]()
					{
						// Init PipInstall task
						PipInstaller.InitPipInstall();
					}
				);
		}
#endif // WITH_EDITOR

		// We only allow that during boot
		const bool bAllowMultithreadedGeneration = true;

		// Initialize the wrapped types
		FPyWrapperTypeRegistry::Get().GenerateWrappedTypes(bAllowMultithreadedGeneration);

#if WITH_EDITOR
		// We don't support pip installs in a restrictive environment at the moment.
		if (!UE::Python::Private::bIsPythonInRestrictiveMode)
		{
			PipTask.Wait();

			IPipInstall& PipInstaller = IPipInstall::Get();
			// Always add pip install site-package path since actual installs may be deferred
			PipInstaller.RegisterPipSitePackagesPath();

			// Check settings/cmd-line for whether pip installer is forced to run before initialization finished
			bool bPipInstallOnInit = GetDefault<UPythonScriptPluginSettings>()->bRunPipInstallOnStartup;
			bool bCmdLineForceInstall = FParse::Param(FCommandLine::Get(), TEXT("ForcePipInstallOnInit"));
			bool bForceInstall = bPipInstallOnInit || bCmdLineForceInstall;
			const int32 NumPkgs = PipInstaller.GetNumPackagesToInstall();
			if (bForceInstall && NumPkgs > 0)
			{
				// Run PipInstall (in game thread) if any of the force options are on
				PipInstallLauncher::StartSync(PipInstaller);
			}
		}
#endif

		// Initialize the tick handler
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float DeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FPythonScriptPlugin_Tick);
			Tick(DeltaTime);
			return true;
		}));

		if (IsRunningCommandlet())
		{
			// Commandlets in general do not tick the engine, so if the plugin is initialized, register a callback to run startup scripts 
			// before the commandlet executes. 
			// This allows, for example, creation of content validators in Python that will be executed in a commandlet. 
			FCoreDelegates::OnCommandletPreMain.AddLambda([this](){
				RunStartupScripts();
			});
		}
	}

	// Release the GIL taken by Py_Initialize now that initialization has finished, to allow other threads access to Python
	// We have to take this again prior to calling Py_Finalize, and all other code will lock on-demand via FPyScopedGIL
	PyMainThreadState = PyEval_SaveThread();

	// Hook into the UE GC so that we can clean-up Python cycles that may be keeping UObjects alive
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FPythonScriptPlugin::OnPreGarbageCollect);

#if WITH_EDITOR
	// Initialize the Content Browser integration
	if (GIsEditor && !IsRunningCommandlet() && GetDefault<UPythonScriptPluginUserSettings>()->bEnableContentBrowserIntegration)
	{
		ContentBrowserFileData::FFileConfigData PythonFileConfig;
		{
			auto PyItemPreview = [this](const FName InFilePath, const FString& InFilename)
			{
				ExecPythonCommand(*InFilename);
				return true;
			};

			ContentBrowserFileData::FDirectoryActions PyDirectoryActions;
			PyDirectoryActions.PassesFilter.BindStatic(&ContentBrowserFileData::FDefaultFileActions::ItemPassesFilter, false);
			PyDirectoryActions.GetAttribute.BindStatic(&ContentBrowserFileData::FDefaultFileActions::GetItemAttribute);
			PythonFileConfig.SetDirectoryActions(PyDirectoryActions);

			ContentBrowserFileData::FFileActions PyFileActions;
			PyFileActions.TypeExtension = TEXT("py");
			PyFileActions.TypeName = FTopLevelAssetPath(TEXT("/Script/Python.Python")); // Fake path to satisfy FFileActions requirements
			PyFileActions.TypeDisplayName = LOCTEXT("PythonTypeName", "Python");
			PyFileActions.TypeShortDescription = LOCTEXT("PythonTypeShortDescription", "Python Script");
			PyFileActions.TypeFullDescription = LOCTEXT("PythonTypeFullDescription", "A file used to script the editor using Python");
			PyFileActions.DefaultNewFileName = TEXT("new_python_script");
			PyFileActions.TypeColor = FColor(255, 156, 0);
			PyFileActions.PassesFilter.BindStatic(&ContentBrowserFileData::FDefaultFileActions::ItemPassesFilter, true);
			PyFileActions.GetAttribute.BindStatic(&ContentBrowserFileData::FDefaultFileActions::GetItemAttribute);
			PyFileActions.Preview.BindLambda(PyItemPreview);
			PythonFileConfig.RegisterFileActions(PyFileActions);
		}

		PythonFileDataSource.Reset(NewObject<UContentBrowserFileDataSource>(GetTransientPackage(), "PythonData"));
		PythonFileDataSource->Initialize(PythonFileConfig);

		TArray<FString> RootPaths;
		FPackageName::QueryRootContentPaths(RootPaths);
		for (const FString& RootPath : RootPaths)
		{
			const FString RootFilesystemPath = FPackageName::LongPackageNameToFilename(RootPath);
			PythonFileDataSource->AddFileMount(*(RootPath / TEXT("Python")), RootFilesystemPath / TEXT("Python"));
		}

		{
			FToolMenuOwnerScoped OwnerScoped(this);
			if (UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.ItemContextMenu.PythonData"))
			{
				Menu->AddDynamicSection(TEXT("DynamicSection_PythonScriptPlugin"), FNewToolMenuDelegate::CreateRaw(this, &FPythonScriptPlugin::PopulatePythonFileContextMenu));
			}
		}
	}
#endif	// WITH_EDITOR
	return true;
}

void FPythonScriptPlugin::ShutdownPython()
{
	if (!bIsInterpreterInitialized)
	{
		return;
	}

	FCoreDelegates::OnPreExit.RemoveAll(this);

	RemoteExecution.Reset();

#if WITH_EDITOR
	if (CmdMenu)
	{
		CmdMenu->OnShutdownMenu();
		CmdMenu.Reset();
	}

	if (UToolMenus* ToolMenus = UToolMenus::TryGet())
	{
		ToolMenus->UnregisterStringCommandHandler("Python");
	}
#endif // WITH_EDITOR

	IModularFeatures::Get().UnregisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), &CmdExec);
	IModularFeatures::Get().UnregisterModularFeature(IConsoleCommandExecutor::ModularFeatureName(), &CmdREPLExec);


	if (PyMainThreadState)
	{
		// We need to restore the original GIL prior to calling Py_Finalize
		PyEval_RestoreThread(PyMainThreadState);
		PyMainThreadState = nullptr;
	}

#if WITH_EDITOR
	// Remove the Content Browser integration
	UToolMenus::UnregisterOwner(this);
	PythonFileDataSource.Reset();
#endif	// WITH_EDITOR

	// Notify any external listeners
	OnPythonShutdownDelegate.Broadcast();
	OnPythonShutdownDelegate.Clear();

	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);

	FTSTicker::RemoveTicker(TickHandle);
	FTSTicker::RemoveTicker(TickOnceHandle);
	if (ModuleDelayedHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ModuleDelayedHandle);
	}

	FPyWrapperTypeRegistry::Get().OnModuleDirtied().RemoveAll(this);
	FModuleManager::Get().OnModulesChanged().RemoveAll(this);

	FPackageName::OnContentPathMounted().RemoveAll(this);
	FPackageName::OnContentPathDismounted().RemoveAll(this);
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
		}
	}

#if WITH_EDITOR
	FEditorSupportDelegates::PrepareToCleanseEditorObject.RemoveAll(this);
#endif	// WITH_EDITOR

#if WITH_EDITOR
	PyEditor::ShutdownModule();
#endif	// WITH_EDITOR
	PyEngine::ShutdownModule();
	PySlate::ShutdownModule();
	PyCore::ShutdownModule();

	PyUnrealModule.Reset();
	PyDefaultGlobalDict.Reset();
	PyDefaultLocalDict.Reset();
	PyConsoleGlobalDict.Reset();
	PyConsoleLocalDict.Reset();

	ShutdownPyMethodWithClosure();

	if (Py_IsInitialized())
	{
		Py_Finalize();
	}

	ShutdownPyWrappers();
	FPyReferenceCollector::Get().Shutdown();
	FPyReferenceCollector::Get().PurgeUnrealGeneratedTypes();

	bIsInterpreterInitialized = false;
	bIsFullyInitialized = false;
	bIsForceEnabledAtRuntime = false;
	bHasTicked = false;
	bIsEnabled = false;
}

void FPythonScriptPlugin::RequestStubCodeGeneration()
{
	// Ignore requests made before the startup scripts have ran
	if (!bIsFullyInitialized)
	{
		return;
	}

	// Delay 2 seconds before generating as this may be triggered by loading several modules at once
	static const float Delay = 2.0f;

	// If there is an existing pending notification, remove it so that it can be reset
	if (ModuleDelayedHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ModuleDelayedHandle);
		ModuleDelayedHandle.Reset();
	}

	// Set new tick
	ModuleDelayedHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
		[this](float DeltaTime)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FPythonScriptPlugin_ModuleDelayed);

			// Once ticked, the delegate will be removed so reset the handle to indicate that it isn't set.
			ModuleDelayedHandle.Reset();

			// Call the event now that the delay has passed.
			GenerateStubCode();

			// Don't reschedule to run again.
			return false;
		}),
		Delay);
}

void FPythonScriptPlugin::GenerateStubCode()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::GenerateStubCode)

	if (IsDeveloperModeEnabled() && GIsEditor && !IsRunningCommandlet())
	{
		// Generate stub code if developer mode enabled
		FPyWrapperTypeRegistry::Get().GenerateStubCodeForWrappedTypes();
	}
}

void FPythonScriptPlugin::RunStartupScripts()
{
	if (bIsFullyInitialized || !bIsInterpreterInitialized)
	{
		return;
	}


	// Run start-up scripts now
	TArray<FString> PySysPaths;
	{
		FPyScopedGIL GIL;
		PySysPaths = PyUtil::GetSystemPaths();
	}

	double InitTotalTime = 0.0;

	auto RunTimedStartupScript = [this, &InitTotalTime](const FString& StartupScript, const FText& StartupScriptInfoText, const bool bRunAsFile)
	{
		FScopedDurationTimeCustomLogger TimeLogger(*StartupScriptInfoText.ToString(), InitTotalTime, [](const TCHAR* Msg) { UE_LOGF(LogPython, Display, "%ls", Msg) });
		if (bRunAsFile)
		{
			// Execute these files in the "public" scope, as if their contents had been run directly in the console
			// This allows them to be used to set-up an editor environment for the console
			FPythonCommandEx InitUnrealPythonCommand;
			InitUnrealPythonCommand.FileExecutionScope = EPythonFileExecutionScope::Public;

			RunFile(*StartupScript, *InitUnrealPythonCommand.Command, InitUnrealPythonCommand);
		}
		else
		{
			ExecPythonCommand(*StartupScript);
		}
	};

	FPathPermissionList FolderPermissionList;
	FolderPermissionList.Append(StartupScriptAllowedPaths);
	FolderPermissionList.Append(IAssetTools::Get().GetFolderPermissionList().Get());

	auto FilePathPassPermissionList = [FolderPermissionList](const FString& FilenamePath) -> bool
		{
#if WITH_EDITOR
			if (!UE::Python::Private::bIsPythonInRestrictiveMode)
			{
				return true;
			}

			bool bIsPathPermitted = false;

			FString OutPackagePath;
			if (FPackageName::TryConvertFilenameToLongPackageName(FilenamePath, OutPackagePath))
			{
				// Writable content folders sort of act as a working user space.
				bIsPathPermitted = FolderPermissionList.PassesStartsWithFilter(OutPackagePath);
			}

			return bIsPathPermitted;
		
#else // WITH_EDITOR
			return true;
#endif
		};


	FScopedSlowTask Progress(PySysPaths.Num() + GetDefault<UPythonScriptPluginSettings>()->StartupScripts.Num(), LOCTEXT("PythonScriptPluginInitScripts", "Running Python start-up scripts..."), true);
	Progress.MakeDialogDelayed(0.1f);
	for (const FString& PySysPath : PySysPaths)
	{
		const FString PotentialFilePath = PySysPath / TEXT("init_unreal.py");
		if (FilePathPassPermissionList(PotentialFilePath) && FPaths::FileExists(PotentialFilePath))
		{
			FText StartupScriptInfoText = FText::Format(LOCTEXT("PythonScriptPluginInitScripts_Running", "Running start-up script {0}..."), FText::FromString(PotentialFilePath));
			Progress.EnterProgressFrame(1.0f, StartupScriptInfoText);
			Progress.ForceRefresh();

			RunTimedStartupScript(PotentialFilePath, StartupScriptInfoText, /*bRunAsFile*/true);
		}
		else
		{
			Progress.EnterProgressFrame();
		}
	}

	for (const FString& StartupScript : GetDefault<UPythonScriptPluginSettings>()->StartupScripts)
	{
		if (FilePathPassPermissionList(StartupScript))
		{
			FText StartupScriptInfoText = FText::Format(LOCTEXT("PythonScriptPluginInitScripts_Running", "Running start-up script {0}..."), FText::FromString(StartupScript));
			Progress.EnterProgressFrame(1.0f, StartupScriptInfoText);

			RunTimedStartupScript(StartupScript, StartupScriptInfoText, /*bRunAsFile*/false);
		}
	}

	// Run any deferred commands now
	for (const FString& DeferredCommand : DeferredCommands)
	{
		ExecPythonCommand(*DeferredCommand);
	}
	DeferredCommands.Reset();

	bIsFullyInitialized = true;

	// Notify any external listeners
	OnPythonInitializedDelegate.Broadcast();
	OnPythonInitializedDelegate.Clear();

#if WITH_EDITOR
	// Activate the Content Browser integration (now that editor subsystems are available)
	if (PythonFileDataSource)
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		ContentBrowserData->ActivateDataSource("PythonData");
	}

	// Register to generate stub code after a short delay
	RequestStubCodeGeneration();
#endif	// WITH_EDITOR
}

void FPythonScriptPlugin::Tick(const float InDeltaTime)
{
	bHasTicked = true;

	RunStartupScripts();

	RemoteExecution->Tick(InDeltaTime);

	FPyWrapperTypeReinstancer::Get().ProcessPending();
}

void FPythonScriptPlugin::SyncRemoteExecutionToSettings()
{
	RemoteExecution->SyncToSettings();
}

void FPythonScriptPlugin::ImportUnrealModule(const TCHAR* InModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FPythonScriptPlugin::ImportUnrealModule %s"), InModuleName));

	const FString PythonModuleName = FString::Printf(TEXT("unreal_%s"), InModuleName);
	const FString NativeModuleName = FString::Printf(TEXT("_unreal_%s"), InModuleName);

	FPyScopedGIL GIL;

	const TCHAR* ModuleNameToImport = nullptr;
	PyObject* ModuleToReload = nullptr;
	if (PyUtil::IsModuleAvailableForImport(*PythonModuleName, &PyUtil::GetOnDiskUnrealModulesCache()))
	{
		// Python modules that are already loaded should be reloaded if we're requested to import them again
		if (!PyUtil::IsModuleImported(*PythonModuleName, &ModuleToReload))
		{
			ModuleNameToImport = *PythonModuleName;
		}
	}
	else if (PyUtil::IsModuleAvailableForImport(*NativeModuleName, &PyUtil::GetOnDiskUnrealModulesCache()))
	{
		ModuleNameToImport = *NativeModuleName;
	}

	FPyObjectPtr PyModule;
	if (ModuleToReload)
	{
		PyModule = FPyObjectPtr::StealReference(PyImport_ReloadModule(ModuleToReload));
	}
	else if (ModuleNameToImport)
	{
		PyModule = FPyObjectPtr::StealReference(PyImport_ImportModule(TCHAR_TO_UTF8(ModuleNameToImport)));
	}

	if (PyModule)
	{
		check(PyUnrealModule);
		PyObject* PyUnrealModuleDict = PyModule_GetDict(PyUnrealModule);

		// Hoist every public symbol from this module into the top-level "unreal" module
		{
			PyObject* PyModuleDict = PyModule_GetDict(PyModule);

			PyObject* PyObjKey = nullptr;
			PyObject* PyObjValue = nullptr;
			Py_ssize_t ModuleDictIndex = 0;
			while (PyDict_Next(PyModuleDict, &ModuleDictIndex, &PyObjKey, &PyObjValue))
			{
				if (PyObjKey)
				{
					const FString Key = PyUtil::PyObjectToUEString(PyObjKey);
					if (Key.Len() > 0 && Key[0] != TEXT('_'))
					{
						PyDict_SetItem(PyUnrealModuleDict, PyObjKey, PyObjValue);
					}
				}
			}
		}
	}
	else
	{
		PyUtil::LogPythonError(nullptr, /*bInteractive*/true);
	}
}

PyObject* FPythonScriptPlugin::EvalString(const TCHAR* InStr, const TCHAR* InContext, const int InMode)
{
	return EvalString(InStr, InContext, InMode, PyConsoleGlobalDict, PyConsoleLocalDict);
}

PyObject* FPythonScriptPlugin::EvalString(const TCHAR* InStr, const TCHAR* InContext, const int InMode, PyObject* InGlobalDict, PyObject* InLocalDict)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::EvalString)

	FPyObjectPtr PyCodeObj = FPyObjectPtr::StealReference(Py_CompileString(TCHAR_TO_UTF8(InStr), TCHAR_TO_UTF8(InContext), InMode));
	if (!PyCodeObj)
	{
		return nullptr;
	}

	FScopedEncodingGuard EncodingGuard;
	TGuardValue<bool> IsRunningUserScriptGuard(UE::Python::Private::GIsRunningUserScript, true);
	PyUtil::FEvalStack::Get().PushContext(PyUtil::FEvalStack::FEvalContext{ InContext, InGlobalDict, InLocalDict});
	PyObject* PyEvalResult = PyEval_EvalCode((PyUtil::FPyCodeObjectType*)PyCodeObj.Get(), InGlobalDict, InLocalDict);
	PyUtil::FEvalStack::Get().PopContext();

	return PyEvalResult;
}

bool FPythonScriptPlugin::RunString(FPythonCommandEx& InOutPythonCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::RunString)

	// Execute Python code within this block
	{
		FPyScopedGIL GIL;
		TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, GIsRunningUnattendedScript || EnumHasAnyFlags(InOutPythonCommand.Flags, EPythonCommandFlags::Unattended));

		int PyExecMode = 0;
		switch (InOutPythonCommand.ExecutionMode)
		{
		case EPythonCommandExecutionMode::ExecuteFile:
			PyExecMode = Py_file_input;
			break;
		case EPythonCommandExecutionMode::ExecuteStatement:
			PyExecMode = Py_single_input;
			break;
		case EPythonCommandExecutionMode::EvaluateStatement:
			PyExecMode = Py_eval_input;
			break;
		default:
			checkf(false, TEXT("Invalid EPythonCommandExecutionMode!"));
			break;
		}

		FDelegateHandle LogCaptureHandle = PyCore::GetPythonLogCapture().AddLambda([&InOutPythonCommand](EPythonLogOutputType InLogType, const TCHAR* InLogString) { InOutPythonCommand.LogOutput.Add(FPythonLogOutputEntry{ InLogType, InLogString }); });
		FPyObjectPtr PyResult = FPyObjectPtr::StealReference(EvalString(*InOutPythonCommand.Command, TEXT("<string>"), PyExecMode));
		PyCore::GetPythonLogCapture().Remove(LogCaptureHandle);
		
		if (PyResult)
		{
			InOutPythonCommand.CommandResult = PyUtil::PyObjectToUEStringRepr(PyResult);
		}
		else if (PyUtil::LogPythonError(&InOutPythonCommand.CommandResult))
		{
			return false;
		}
	}

	FPyWrapperTypeReinstancer::Get().ProcessPending();
	return true;
}

bool FPythonScriptPlugin::RunFile(const TCHAR* InFile, const TCHAR* InArgs, FPythonCommandEx& InOutPythonCommand)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FPythonScriptPlugin::RunFile(%s)"), InFile ? InFile : TEXT("null")));

	auto ResolveFilePath = [InFile]() -> FString
	{
		// Favor the CWD
		if (FPaths::FileExists(InFile))
		{
			return FPaths::ConvertRelativePathToFull(InFile);
		}

		// Execute Python code within this block
		{
			FPyScopedGIL GIL;

			// Then test against each system path in order (as Python would)
			const TArray<FString> PySysPaths = PyUtil::GetSystemPaths();
			for (const FString& PySysPath : PySysPaths)
			{
				const FString PotentialFilePath = PySysPath / InFile;
				if (FPaths::FileExists(PotentialFilePath))
				{
					return PotentialFilePath;
				}
			}
		}

		// Didn't find a match... we know this file doesn't exist, but we'll use this path in the error reporting
		return FPaths::ConvertRelativePathToFull(InFile);
	};

	const FString ResolvedFilePath = ResolveFilePath();

	FString FileStr;
	bool bLoaded = FFileHelper::LoadFileToString(FileStr, *ResolvedFilePath);
#if WITH_EDITOR
	// We don't want to update the menu entries for the recent files when running the startups scripts
	if (CmdMenu && bIsFullyInitialized)
	{
		CmdMenu->OnRunFile(ResolvedFilePath, bLoaded);
	}
#endif // WITH_EDITOR

	if (!bLoaded)
	{
		InOutPythonCommand.CommandResult = FString::Printf(TEXT("Could not load Python file '%s' (resolved from '%s')"), *ResolvedFilePath, InFile);
		UE_LOGF(LogPython, Error, "%ls", *InOutPythonCommand.CommandResult);
		return false;
	}

	// Execute Python code within this block
	double ElapsedSeconds = 0.0;
	{
		FPyScopedGIL GIL;
		TGuardValue<bool> UnattendedScriptGuard(GIsRunningUnattendedScript, GIsRunningUnattendedScript || EnumHasAnyFlags(InOutPythonCommand.Flags, EPythonCommandFlags::Unattended));

		FPyObjectPtr PyFileGlobalDict = PyConsoleGlobalDict;
		FPyObjectPtr PyFileLocalDict = PyConsoleLocalDict;
		if (InOutPythonCommand.FileExecutionScope == EPythonFileExecutionScope::Private)
		{
			PyFileGlobalDict = FPyObjectPtr::StealReference(PyDict_Copy(PyDefaultGlobalDict));
			PyFileLocalDict = PyFileGlobalDict;
		}
		{
			FPyObjectPtr PyResolvedFilePath;
			if (PyConversion::Pythonize(ResolvedFilePath, PyResolvedFilePath.Get(), PyConversion::ESetErrorState::No))
			{
				PyDict_SetItemString(PyFileGlobalDict, "__file__", PyResolvedFilePath);
			}
		}

		FPyObjectPtr PyResult;
		{
			FScopedDurationTimer Timer(ElapsedSeconds);
			FPythonScopedArgv ScopedArgv(InArgs);

			FDelegateHandle LogCaptureHandle = PyCore::GetPythonLogCapture().AddLambda([&InOutPythonCommand](EPythonLogOutputType InLogType, const TCHAR* InLogString) { InOutPythonCommand.LogOutput.Add(FPythonLogOutputEntry{ InLogType, InLogString }); });
			PyResult = FPyObjectPtr::StealReference(EvalString(*FileStr, *ResolvedFilePath, Py_file_input, PyFileGlobalDict, PyFileLocalDict)); // We can't just use PyRun_File here as Python isn't always built against the same version of the CRT as UE, so we get a crash at the CRT layer
			PyCore::GetPythonLogCapture().Remove(LogCaptureHandle);
		}

		PyDict_DelItemString(PyFileGlobalDict, "__file__");

		if (PyResult)
		{
			InOutPythonCommand.CommandResult = PyUtil::PyObjectToUEStringRepr(PyResult);
		}
		else if (PyUtil::LogPythonError(&InOutPythonCommand.CommandResult))
		{
			TArray<FString> ErrorLines;
			InOutPythonCommand.CommandResult.ParseIntoArrayLines(ErrorLines);

			for (const FString& ErrorLine : ErrorLines)
			{
				InOutPythonCommand.LogOutput.Add(FPythonLogOutputEntry{ EPythonLogOutputType::Error, ErrorLine });
			}
			return false;
		}
	}

	FPyWrapperTypeReinstancer::Get().ProcessPending();

	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("Duration"), ElapsedSeconds));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("PythonScriptPlugin"), EventAttributes);
	}

	return true;
}

void FPythonScriptPlugin::OnModuleDirtied(FName InModuleName)
{
	ImportUnrealModule(*InModuleName.ToString());
}

void FPythonScriptPlugin::OnModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason)
{
	LLM_SCOPE_BYNAME(TEXT("PythonScriptPlugin"));
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnModulesChanged)

	switch (InModuleChangeReason)
	{
	case EModuleChangeReason::ModuleLoaded:
		FPyWrapperTypeRegistry::Get().GenerateWrappedTypesForModule(InModuleName);
#if WITH_EDITOR
		// Register to generate stub code after a short delay
		RequestStubCodeGeneration();
#endif	// WITH_EDITOR
		break;

	case EModuleChangeReason::ModuleUnloaded:
		FPyWrapperTypeRegistry::Get().OrphanWrappedTypesForModule(InModuleName);
#if WITH_EDITOR
		// Register to generate stub code after a short delay
		RequestStubCodeGeneration();
#endif	// WITH_EDITOR
		break;

	default:
		break;
	}
}

void FPythonScriptPlugin::OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnContentPathMounted)

	{
		FPyScopedGIL GIL;
		RegisterModulePaths(InFilesystemPath);
	}

#if WITH_EDITOR
	if (PythonFileDataSource)
	{
		PythonFileDataSource->AddFileMount(*(InAssetPath / TEXT("Python")), InFilesystemPath / TEXT("Python"));
	}
#endif	// WITH_EDITOR
}

void FPythonScriptPlugin::OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnContentPathDismounted)

	{
		FPyScopedGIL GIL;
		UnregisterModulePaths(InFilesystemPath);
	}

#if WITH_EDITOR
	if (PythonFileDataSource)
	{
		PythonFileDataSource->RemoveFileMount(*(InAssetPath / TEXT("Python")));
	}
#endif	// WITH_EDITOR
}

void FPythonScriptPlugin::RegisterModulePaths(const FString& InFilesystemPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::RegisterModulePaths)

	const FString PythonContentPath = FPaths::ConvertRelativePathToFull(InFilesystemPath / TEXT("Python"));
	if (IFileManager::Get().DirectoryExists(*PythonContentPath))
	{
		PyUtil::AddSystemPath(PythonContentPath);

		for (const FString& SitePkgDir : PyUtil::GetSitePackageSubdirs())
		{
			const FString CheckSubdir = PythonContentPath / SitePkgDir;
			PyUtil::AddSitePackagesPath(CheckSubdir);
		}

		PyUtil::GetOnDiskUnrealModulesCache().AddModules(*PythonContentPath);
	}
}

void FPythonScriptPlugin::UnregisterModulePaths(const FString& InFilesystemPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::UnregisterModulePaths)

	const FString PythonContentPath = FPaths::ConvertRelativePathToFull(InFilesystemPath / TEXT("Python"));
	PyUtil::RemoveSystemPath(PythonContentPath);

	for (const FString& SitePkgDir : PyUtil::GetSitePackageSubdirs())
	{
		const FString CheckSubdir = PythonContentPath / SitePkgDir;
		PyUtil::RemoveSystemPath(CheckSubdir);
	}

	PyUtil::GetOnDiskUnrealModulesCache().RemoveModules(*PythonContentPath);
}

bool FPythonScriptPlugin::IsDeveloperModeEnabled()
{
	return GetDefault<UPythonScriptPluginSettings>()->bDeveloperMode || GetDefault<UPythonScriptPluginUserSettings>()->bDeveloperMode;
}

ETypeHintingMode FPythonScriptPlugin::GetTypeHintingMode()
{
	return GetDefault<UPythonScriptPluginUserSettings>()->TypeHintingMode;
}

void FPythonScriptPlugin::OnAssetRenamed(const FAssetData& Data, const FString& OldName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnAssetRenamed)

	FPyWrapperTypeRegistry& PyWrapperTypeRegistry = FPyWrapperTypeRegistry::Get();
	
	// If this asset has an associated Python type, then we need to rename it
	if (PyWrapperTypeRegistry.HasWrappedTypeForObjectName(OldName))
	{
		if (const UObject* AssetPtr = PyGenUtil::GetAssetTypeRegistryType(Data.GetAsset()))
		{
			PyWrapperTypeRegistry.UpdateGenerateWrappedTypeForRename(OldName, AssetPtr);
			OnAssetUpdated(AssetPtr);
		}
		else
		{
			PyWrapperTypeRegistry.RemoveGenerateWrappedTypeForDelete(OldName);
		}
	}
}

void FPythonScriptPlugin::OnAssetRemoved(const FAssetData& Data)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnAssetRemoved)

	FPyWrapperTypeRegistry& PyWrapperTypeRegistry = FPyWrapperTypeRegistry::Get();
	
	// If this asset has an associated Python type, then we need to remove it
	const FSoftObjectPath AssetPath = Data.GetSoftObjectPath();
	if (PyWrapperTypeRegistry.HasWrappedTypeForObjectName(AssetPath))
	{
		PyWrapperTypeRegistry.RemoveGenerateWrappedTypeForDelete(AssetPath);
	}
}

void FPythonScriptPlugin::OnAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnAssetReload)

	if (InPackageReloadPhase == EPackageReloadPhase::PostPackageFixup)
	{
		// Get the primary asset in this package
		// Use the new package as it has the correct name
		const UPackage* NewPackage = InPackageReloadedEvent->GetNewPackage();
		const UObject* NewAsset = StaticFindObject(UObject::StaticClass(), (UPackage*)NewPackage, *FPackageName::GetLongPackageAssetName(NewPackage->GetName()));
		OnAssetUpdated(NewAsset);
	}
}

void FPythonScriptPlugin::OnAssetUpdated(const UObject* InObj)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPythonScriptPlugin::OnAssetUpdated)

	if (const UObject* AssetPtr = PyGenUtil::GetAssetTypeRegistryType(InObj))
	{
		// If this asset has an associated Python type, then we need to re-generate it
		FPyWrapperTypeRegistry& PyWrapperTypeRegistry = FPyWrapperTypeRegistry::Get();
		if (PyWrapperTypeRegistry.HasWrappedTypeForObject(AssetPtr))
		{
			FPyWrapperTypeRegistry::FGeneratedWrappedTypeReferences GeneratedWrappedTypeReferences;
			TSet<FName> DirtyModules;

			PyWrapperTypeRegistry.GenerateWrappedTypeForObject(AssetPtr, GeneratedWrappedTypeReferences, DirtyModules, EPyTypeGenerationFlags::IncludeBlueprintGeneratedTypes | EPyTypeGenerationFlags::OverwriteExisting);

			PyWrapperTypeRegistry.GenerateWrappedTypesForReferences(GeneratedWrappedTypeReferences, DirtyModules);
			PyWrapperTypeRegistry.NotifyModulesDirtied(DirtyModules);
		}
	}
}

void FPythonScriptPlugin::OnPreGarbageCollect()
{
	FPyScopedGIL GIL;
	PyUtil::CollectGarbage();
}

#if WITH_EDITOR

void FPythonScriptPlugin::OnPrepareToCleanseEditorObject(UObject* InObject)
{
	FPyReferenceCollector::Get().PurgeUnrealObjectReferences(InObject, true);
}

void FPythonScriptPlugin::PopulatePythonFileContextMenu(UToolMenu* InMenu)
{
	const UContentBrowserDataMenuContext_FileMenu* ContextObject = InMenu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_FileMenu was missing!"));

	if (!PythonFileDataSource)
	{
		return;
	}

	// Extract the internal file paths that belong to this data source from the full list of selected paths given in the context
	TArray<TSharedRef<const FContentBrowserFileItemDataPayload>> SelectedPythonFiles;
	for (const FContentBrowserItem& SelectedItem : ContextObject->SelectedItems)
	{
		if (const FContentBrowserItemData* ItemDataPtr = SelectedItem.GetPrimaryInternalItem())
		{
			if (TSharedPtr<const FContentBrowserFileItemDataPayload> FilePayload = ContentBrowserFileData::GetFileItemPayload(PythonFileDataSource.Get(), *ItemDataPtr))
			{
				SelectedPythonFiles.Add(FilePayload.ToSharedRef());
			}
		}
	}

	// Only add the file items if we have a file path selected
	if (SelectedPythonFiles.Num() > 0)
	{
		// Run
		{
			FToolMenuSection& Section = InMenu->AddSection("PythonScript", LOCTEXT("PythonScriptMenuHeading", "Python Script"));
			Section.InsertPosition.Position = EToolMenuInsertType::First;

			const FExecuteAction ExecuteRunAction = FExecuteAction::CreateLambda([this, SelectedPythonFiles]()
			{
				for (const TSharedRef<const FContentBrowserFileItemDataPayload>& SelectedPythonFile : SelectedPythonFiles)
				{
					ExecPythonCommand(*SelectedPythonFile->GetFilename());
				}
			});

			Section.AddMenuEntry(
				"RunPythonScript",
				LOCTEXT("RunPythonScript", "Run..."),
				LOCTEXT("RunPythonScriptToolTip", "Run this script."),
				FSlateIcon(),
				FUIAction(ExecuteRunAction)
			);
		}
	}
}

#endif	// WITH_EDITOR

#endif	// WITH_PYTHON

IMPLEMENT_MODULE(FPythonScriptPlugin, PythonScriptPlugin)

#undef LOCTEXT_NAMESPACE
