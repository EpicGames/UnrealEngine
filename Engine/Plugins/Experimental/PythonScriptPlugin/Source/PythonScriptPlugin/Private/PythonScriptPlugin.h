// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "IPythonScriptPlugin.h"
#include "PyUtil.h"
#include "PyPtr.h"
#include "Containers/Ticker.h"
#include "Misc/CoreMisc.h"
#include "HAL/IConsoleManager.h"
#include "Framework/Commands/InputChord.h"
#include "Kismet2/EnumEditorUtils.h"
#include "Misc/NamePermissionList.h"
#include "UObject/StrongObjectPtr.h"
#include "PythonScriptPluginSettings.h"

class FPythonScriptPlugin;
class FPythonScriptRemoteExecution;
class FPackageReloadedEvent;
class UContentBrowserFileDataSource;
class UToolMenu;

struct FAssetData;

enum class EPackageReloadPhase : uint8;

#if WITH_PYTHON

/**
 * Executor for "Python" commands
 */
class FPythonCommandExecutor : public IConsoleCommandExecutor
{
public:
	FPythonCommandExecutor(IPythonScriptPlugin* InPythonScriptPlugin);

	static FName StaticName();
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDescription() const override;
	virtual FText GetHintText() const override;
	virtual void GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out) override;
	virtual void GetExecHistory(TArray<FString>& Out) override;
	virtual bool Exec(const TCHAR* Input) override;
	virtual bool AllowHotKeyClose() const override;
	virtual bool AllowMultiLine() const override;
	virtual FInputChord GetHotKey() const override;
	virtual FInputChord GetIterateExecutorHotKey() const override;

private:
	IPythonScriptPlugin* PythonScriptPlugin;
};

/**
 * Executor for "Python (REPL)" commands
 */
class FPythonREPLCommandExecutor : public IConsoleCommandExecutor
{
public:
	FPythonREPLCommandExecutor(IPythonScriptPlugin* InPythonScriptPlugin);

	static FName StaticName();
	virtual FName GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetDescription() const override;
	virtual FText GetHintText() const override;
	virtual void GetSuggestedCompletions(const TCHAR* Input, TArray<FConsoleSuggestion>& Out) override;
	virtual void GetExecHistory(TArray<FString>& Out) override;
	virtual bool Exec(const TCHAR* Input) override;
	virtual bool AllowHotKeyClose() const override;
	virtual bool AllowMultiLine() const override;
	virtual FInputChord GetHotKey() const override;
	virtual FInputChord GetIterateExecutorHotKey() const override;

private:
	IPythonScriptPlugin* PythonScriptPlugin;
};

/**
 *
 */
struct IPythonCommandMenu
{
	virtual ~IPythonCommandMenu() = default;

	virtual void OnStartupMenu() = 0;
	virtual void OnShutdownMenu() = 0;

	virtual void OnRunFile(const FString& InFile, bool bAdd) = 0;
};
#endif	// WITH_PYTHON

class FPythonScriptPlugin final
	: public IPythonScriptPlugin
	, public FSelfRegisteringExec
	, public FEnumEditorUtils::INotifyOnEnumChanged
{
public:
	FPythonScriptPlugin();

	/** Get this module */
	static FPythonScriptPlugin* Get()
	{
		return static_cast<FPythonScriptPlugin*>(IPythonScriptPlugin::Get());
	}

	//~ IPythonScriptPlugin interface
	virtual bool IsPythonAvailable() const override;
	virtual bool IsPythonConfigured() const override;
	virtual bool IsPythonInitialized() const override;
	virtual bool ForceEnablePythonAtRuntime(UE::FSourceLocation Location = UE::FSourceLocation::Current()) override;
	virtual void ForceDisablePythonAtRuntime(UE::FSourceLocation Location = UE::FSourceLocation::Current()) override;
	virtual bool ExecPythonCommand(const TCHAR* InPythonCommand) override;
	virtual bool ExecPythonCommandEx(FPythonCommandEx& InOutPythonCommand) override;
	virtual FString GetInterpreterExecutablePath() const override;
	virtual FSimpleMulticastDelegate& OnPythonConfigured() override;
	virtual FSimpleMulticastDelegate& OnPythonInitialized() override;
	virtual FSimpleMulticastDelegate& OnPythonShutdown() override;

	bool ForceEnablePythonAtRuntime(const FString& Location);

	//~ IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ FEnumEditorUtils::INotifyOnEnumChanged interface
	virtual void PreChange(const UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info) override;
	virtual void PostChange(const UUserDefinedEnum* Enum, FEnumEditorUtils::EEnumEditorChangeInfo Info) override;

	virtual void AddStartupScriptAllowPath(FName Owner, const FString& Path) override;
	virtual void RemoveStartupScriptAllowPath(FName Owner) override;

#if WITH_PYTHON
	void RunStartupScripts();

	/** Sync the remote execution environment to the current settings, starting or stopping it as required */
	void SyncRemoteExecutionToSettings();

	/** 
	 * Import the given module into the "unreal" package.
	 * This function will take the given name and attempt to import either "unreal_{name}" or "_unreal_{name}" into the "unreal" package as "unreal.{name}".
	 */
	void ImportUnrealModule(const TCHAR* InModuleName);

	/** Evaluate/Execute a Python string, and return the result */
	PyObject* EvalString(const TCHAR* InStr, const TCHAR* InContext, const int InMode);
	PyObject* EvalString(const TCHAR* InStr, const TCHAR* InContext, const int InMode, PyObject* InGlobalDict, PyObject* InLocalDict);

	/** Run literal Python script */
	bool RunString(FPythonCommandEx& InOutPythonCommand);

	/** Run a Python file */
	bool RunFile(const TCHAR* InFile, const TCHAR* InArgs, FPythonCommandEx& InOutPythonCommand);

	PyObject* GetDefaultGlobalDict() { return PyDefaultGlobalDict.Get(); }
	PyObject* GetDefaultLocalDict()  { return PyDefaultLocalDict.Get();  }
	PyObject* GetConsoleGlobalDict() { return PyConsoleGlobalDict.Get(); }
	PyObject* GetConsoleLocalDict()  { return PyConsoleLocalDict.Get();  }
#endif	// WITH_PYTHON

protected:
	//~ FSelfRegisteringExec interface
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

private:
#if WITH_PYTHON
	void OnPostEngineInit();

	void ConfigureAndInitializePython();

	bool InitializePython();

	void ShutdownPython();

	void RequestStubCodeGeneration();

	void GenerateStubCode();

	void Tick(const float InDeltaTime);

	void OnModuleDirtied(FName InModuleName);

	void OnModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason);

	void OnContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath);

	void OnContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath);

	static void RegisterModulePaths(const FString& InFilesystemPath);

	static void UnregisterModulePaths(const FString& InFilesystemPath);

	static bool IsDeveloperModeEnabled();

	static ETypeHintingMode GetTypeHintingMode();

	void OnAssetRenamed(const FAssetData& Data, const FString& OldName);

	void OnAssetRemoved(const FAssetData& Data);

	void OnAssetReload(const EPackageReloadPhase InPackageReloadPhase, FPackageReloadedEvent* InPackageReloadedEvent);

	void OnAssetUpdated(const UObject* InObj);

	void OnPreGarbageCollect();

#if WITH_EDITOR
	void OnPrepareToCleanseEditorObject(UObject* InObject);

	void PopulatePythonFileContextMenu(UToolMenu* InMenu);

	TStrongObjectPtr<UContentBrowserFileDataSource> PythonFileDataSource;
#endif	// WITH_EDITOR

	TUniquePtr<FPythonScriptRemoteExecution> RemoteExecution;
	FPythonCommandExecutor CmdExec;
	FPythonREPLCommandExecutor CmdREPLExec;
	TUniquePtr<IPythonCommandMenu> CmdMenu = nullptr;
	FTSTicker::FDelegateHandle TickHandle;
	FTSTicker::FDelegateHandle TickOnceHandle;
	FTSTicker::FDelegateHandle ModuleDelayedHandle;
	TArray<FString> DeferredCommands;

	PyUtil::FPyApiBuffer PyProgramName;
	PyUtil::FPyApiBuffer PyHomePath;
	FPyObjectPtr PyDefaultGlobalDict;
	FPyObjectPtr PyDefaultLocalDict;
	FPyObjectPtr PyConsoleGlobalDict;
	FPyObjectPtr PyConsoleLocalDict;
	FPyObjectPtr PyUnrealModule;
	PyThreadState* PyMainThreadState = nullptr;
	bool bIsConfigured = false;
	bool bIsInterpreterInitialized = false; // Python interpreter is available; may not yet have run startup scripts
	bool bIsFullyInitialized = false; // Python interpreter is available, and start-up scripts have run
	bool bIsEnabled = false;
	bool bIsForceEnabledAtRuntime = false;
	bool bHasTicked = false;

#if PY_MAJOR_VERSION >= 3 && PY_MINOR_VERSION >= 11
	PyConfig ModulePyConfig;
	static PyUtil::FPyApiBuffer Utf8String;
#endif // 3.11 upgrade
#endif	// WITH_PYTHON

	FSimpleMulticastDelegate OnPythonConfiguredDelegate;
	FSimpleMulticastDelegate OnPythonInitializedDelegate;
	FSimpleMulticastDelegate OnPythonShutdownDelegate;

	FPathPermissionList StartupScriptAllowedPaths;
};
