// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonScriptPluginSettings.h"
#include "PythonScriptPlugin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PythonScriptPluginSettings)

#define LOCTEXT_NAMESPACE "PythonScriptPlugin"

UPythonScriptPluginSettings::UPythonScriptPluginSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName  = TEXT("Python");

	RemoteExecutionMulticastGroupEndpoint = TEXT("239.0.0.1:6766");
	RemoteExecutionMulticastBindAddress = TEXT("127.0.0.1");
	RemoteExecutionSendBufferSizeBytes = 2 * 1024 * 1024;
	RemoteExecutionReceiveBufferSizeBytes = 2 * 1024 * 1024;
	RemoteExecutionMulticastTtl = 0;
}

#if WITH_EDITOR

bool UPythonScriptPluginSettings::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEditChange = Super::CanEditChange(InProperty);

	if (bCanEditChange && InProperty)
	{
		if (InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionMulticastGroupEndpoint) ||
			InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionMulticastBindAddress) ||
			InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionSendBufferSizeBytes) || 
			InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionReceiveBufferSizeBytes) ||
			InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionMulticastTtl)
			)
		{
			bCanEditChange &= bRemoteExecution;
		}
	}

	return bCanEditChange;
}

void UPythonScriptPluginSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

#if WITH_PYTHON
	if (PropertyChangedEvent.MemberProperty)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, bRemoteExecution) ||
			PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionMulticastGroupEndpoint) ||
			PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionMulticastBindAddress) ||
			PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionSendBufferSizeBytes) ||
			PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionReceiveBufferSizeBytes) ||
			PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UPythonScriptPluginSettings, RemoteExecutionMulticastTtl)
			)
		{
			FPythonScriptPlugin::Get()->SyncRemoteExecutionToSettings();
		}
	}
#endif	// WITH_PYTHON
}

FText UPythonScriptPluginSettings::GetSectionText() const
{
	return LOCTEXT("SettingsDisplayName", "Python");
}

#endif	// WITH_EDITOR

UPythonScriptPluginUserSettings::UPythonScriptPluginUserSettings()
{
	CategoryName = TEXT("Plugins");
	SectionName = TEXT("Python");
}

#if WITH_EDITOR
FText UPythonScriptPluginUserSettings::GetSectionText() const
{
	return LOCTEXT("UserSettingsDisplayName", "Python");
}

#endif	// WITH_EDITOR

#undef LOCTEXT_NAMESPACE


