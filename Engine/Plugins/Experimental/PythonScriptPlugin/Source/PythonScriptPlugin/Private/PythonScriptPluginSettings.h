// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "PythonScriptPluginSettings.generated.h"

UENUM()
enum class ETypeHintingMode : uint8
{
	/** Turn off type hinting. */
	Off,

	/**
	 * When generating the Python stub and to some extend the Docstrings, enables type hinting (PEP 484) to get the best experience
	 * with a Python IDE auto-completion. The hinting will list the exact input types, omit type coercions and will assume all reflected
	 * unreal.Object cannot be None which is not true, but will let the function signature easy to read.
	 */
	AutoCompletion,

	/**
	 * Enables type hinting for static type checking. Hint as close as possible the real supported types including
	 * possible type coercions. Because the UE reflection API doesn't provide all the required information, some tradeoffs
	 * are required that do not always reflect the reality. For example, reflected UObject will always be marked as
	 * 'possibly None'. While this is true in some contexts, it is not true all the time.
	 */
	TypeChecker,
};

UENUM()
enum class EPythonEnabledOverrideState : uint8
{
	/** Python will be enabled based on the current project settings and any command line overrides. */
	None,

	/** Enable Python, even if the project has it disabled by default (behaves like '-EnablePython' on the command line). Note: Does not take precedence over '-DisablePython' on the command line. */
	Enable,

	/** Disable Python, even if the project has it enabled by default (behaves like '-DisablePython' on the command line). Note: Does not take precedence over '-EnablePython' on the command line. */
	Disable,
};

/**
 * Configure the Python plug-in.
 */
UCLASS(config=Engine, defaultconfig)
class UPythonScriptPluginSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPythonScriptPluginSettings();

#if WITH_EDITOR
	//~ UObject interface
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	//~ UDeveloperSettings interface
	virtual FText GetSectionText() const override;
#endif

	/** Array of Python scripts to run at start-up (run before the first Tick after the Engine has initialized). */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true, MultiLine=true))
	TArray<FString> StartupScripts;

	/** Array of additional paths to add to the Python system paths. */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true, RelativePath))
	TArray<FDirectoryPath> AdditionalPaths;

	/**
	 * Should the embedded interpreter be run in isolation mode. In isolation, the standard PYTHON* environment variables (PYTHONPATH,
	 * PYTHONHOME, etc), the script's directory and the user's site-packages directory are ignored by the interpreter. This
	 * prevents incompabible software to crash the engine. Consider turning this option off if you tightly control your Python
	 * environment and you are sure everything is compatible. Note that the UE_PYTHONPATH environment variable is added to 'sys.path'
	 * whether the interpreter runs in insolation mode or not.
	 */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true))
	bool bIsolateInterpreterEnvironment = true;

	/**
	 * Should Developer Mode be enabled on the Python interpreter *for all users of the project*
	 * Note: Most of the time you want to enable bDeveloperMode in the Editor Preferences instead!
	 *
	 * (will also enable extra warnings (e.g., for deprecated code), and enable stub code generation for
	 * use with external IDEs).
	 */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true, DisplayName="Developer Mode (all users)"), AdvancedDisplay)
	bool bDeveloperMode;

	/**
	 * Should the pip install task be run on python engine initialization.
	 *
	 * NOTE: The project pip install directory: <ProjectDir>/Intermediate/PipInstall/Lib/site-packages
	 *       will still be added to site package path, to allow for pre-populated or deferred installs.
	 *
	 *       See <ProjectDir>/Intermediate/PipInstall/merged_requirements.in for listing of required packages.
	 */
	UPROPERTY(config, EditAnywhere, Category=PythonPipInstall, meta=(ConfigRestartRequired=true))
	bool bRunPipInstallOnStartup = true;

	/** Require pip to use strict hash checking for all packages
	 *  WARNING: Disabling this setting is a security risk, particularly when combined with IndexUrl override
	*/
	UPROPERTY(config, EditAnywhere, Category=PythonPipInstall, AdvancedDisplay, meta=(ConfigRestartRequired=true))
	bool bPipStrictHashCheck = true;

	/** Only run pip to generate merged requirements and/or validate requested packages are installed */
	UPROPERTY(config, EditAnywhere, Category=PythonPipInstall, AdvancedDisplay, meta=(ConfigRestartRequired=true))
	bool bOfflineOnly = false;

	/** Override all index/extra-index URLs, this is useful for internal NAT installs (using e.g. devpi or similar index cache)
	 *  WARNING: Strict hash checks should be enabled if an override index url is used
	*/
	UPROPERTY(config, EditAnywhere, Category=PythonPipInstall, AdvancedDisplay, meta=(ConfigRestartRequired=true))
	FString OverrideIndexURL;
	
	/** Additional arguments passed to main pip install call, useful to add e.g. --cert or other proxy options for restrictive firewalls
	* 	NOTE: Do not use this to add --index-url or --extra-index-url, instead use OverrideIndexURL setting or ExtraIndexUrls uplugin property, respectively
	*/
	UPROPERTY(config, EditAnywhere, Category=PythonPipInstall, AdvancedDisplay, meta=(ConfigRestartRequired=true))
	FString ExtraInstallArgs;

	/** Should remote Python execution be enabled? */
	UPROPERTY(config, EditAnywhere, Category=PythonRemoteExecution, meta=(DisplayName="Enable Remote Execution?"))
	bool bRemoteExecution;

	/** The multicast group endpoint (in the form of IP_ADDRESS:PORT_NUMBER) that the UDP multicast socket should join */
	UPROPERTY(config, EditAnywhere, Category=PythonRemoteExecution, AdvancedDisplay, meta=(DisplayName="Multicast Group Endpoint"))
	FString RemoteExecutionMulticastGroupEndpoint;

	/** The adapter address that the UDP multicast socket should bind to, or 0.0.0.0 to bind to all adapters */
	UPROPERTY(config, EditAnywhere, Category=PythonRemoteExecution, AdvancedDisplay, meta=(DisplayName="Multicast Bind Address"))
	FString RemoteExecutionMulticastBindAddress;

	/** Size of the send buffer for the remote endpoint connection */
	UPROPERTY(config, EditAnywhere, Category=PythonRemoteExecution, AdvancedDisplay, meta=(DisplayName="Send Buffer Size", Units="Bytes"))
	int32 RemoteExecutionSendBufferSizeBytes;

	/** Size of the receive buffer for the remote endpoint connection */
	UPROPERTY(config, EditAnywhere, Category=PythonRemoteExecution, AdvancedDisplay, meta=(DisplayName="Receive Buffer Size", Units="Bytes"))
	int32 RemoteExecutionReceiveBufferSizeBytes;

	/** The TTL that the UDP multicast socket should use (0 is limited to the local host, 1 is limited to the local subnet) */
	UPROPERTY(config, EditAnywhere, Category=PythonRemoteExecution, AdvancedDisplay, meta=(DisplayName="Multicast Time-To-Live"))
	uint8 RemoteExecutionMulticastTtl;
};


UCLASS(config=EditorPerProjectUserSettings)
class UPythonScriptPluginUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPythonScriptPluginUserSettings();

#if WITH_EDITOR
	//~ UDeveloperSettings interface
	virtual FText GetSectionText() const override;
#endif

	/**
	 * Should we override the default enabled state for Python for this project?
	 */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true))
	EPythonEnabledOverrideState EnablePythonOverride = EPythonEnabledOverrideState::None;

	/**
	 * Should Developer Mode be enabled on the Python interpreter?
	 *
	 * (will also enable extra warnings (e.g., for deprecated code), and enable stub code generation for
	 * use with external IDEs).
	 */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true))
	bool bDeveloperMode;

	/**
	 * Should the generated Python stub and API documentation have type hints. This enables standard Python type hinting (PEP 484) for the classes,
	 * structs, methods, properties, constants, etc. exposed by the engine. If the developer mode is enabled and the Python IDE configured to use
	 * the generated Python stub, types will be displayed in auto-completion popup and used by the IDE static type checkers. This has no effects on
	 * the execution of the code. (Requires Python >= 3.7)
	 */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true))
	ETypeHintingMode TypeHintingMode = ETypeHintingMode::AutoCompletion;

	/** Should Python scripts be available in the Content Browser? */
	UPROPERTY(config, EditAnywhere, Category=Python, meta=(ConfigRestartRequired=true))
	bool bEnableContentBrowserIntegration;
};
