// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonScriptPluginStyle.h"

#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"

FName FPythonScriptPluginEditorStyle::StyleName("PythonScriptPluginEditorStyle");

FPythonScriptPluginEditorStyle::FPythonScriptPluginEditorStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D IconSize(16.0f, 16.0f);

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/PythonScriptPlugin/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Set("Icons.PythonExecute", new IMAGE_BRUSH_SVG("PythonExecute", IconSize));
	Set("Icons.PythonRecent",  new IMAGE_BRUSH_SVG("PythonRecent", IconSize));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FPythonScriptPluginEditorStyle::~FPythonScriptPluginEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FPythonScriptPluginEditorStyle& FPythonScriptPluginEditorStyle::Get()
{
	static FPythonScriptPluginEditorStyle Inst;
	return Inst;
}


