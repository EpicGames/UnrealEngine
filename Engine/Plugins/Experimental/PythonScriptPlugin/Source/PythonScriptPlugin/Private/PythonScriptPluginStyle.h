// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

/**
 * Slate style set that defines the styles for Python UI in the Editor 
 */
class FPythonScriptPluginEditorStyle
	: public FSlateStyleSet
{
public:
	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FPythonScriptPluginEditorStyle& Get();

private:

	FPythonScriptPluginEditorStyle();
	~FPythonScriptPluginEditorStyle();
};
