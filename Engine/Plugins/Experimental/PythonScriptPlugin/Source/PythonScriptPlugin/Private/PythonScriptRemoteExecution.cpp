// Copyright Epic Games, Inc. All Rights Reserved.

#include "PythonScriptRemoteExecution.h"
#include "IPythonScriptPlugin.h"
#include "PythonScriptPluginSettings.h"

#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Misc/EngineVersion.h"
#include "Logging/LogMacros.h"

#include "Sockets.h"
#include "Common/TcpSocketBuilder.h"
#include "Common/UdpSocketBuilder.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"
#include "Policies/CondensedJsonPrintPolicy.h"

#define LOCTEXT_NAMESPACE "PythonScriptRemoteExecution"

#if WITH_PYTHON

/**
 * Remote Execution protocol definition.
 *
 * Messages are a UTF-8 encoded JSON object with the following layout:
 *	{
 *		"version": 1,		// Required - Integer - Protocol version number (currently 1)
 *		"magic": "ue_py",	// Required - String  - Protocol magic identifier (currently "ue_py")
 * 		"type": "...",		// Required - String  - Identifier for the message type (see below)
 * 		"source": "...",	// Required - String  - Unique identifier of the node that sent the message (typically a GUID)
 * 		"dest": "...",		// Optional - String  - Unique identifier of the node that should receive the message (typically a GUID)
 * 		"data": "..."		// Optional - Any	  - Message specific payload data (see below)
 *	}
 *
 * Message types:
 *	"ping" (UDP) - Service discovery request
 *
 *	"pong" (UDP) - Service discovery response
 *		"dest" - Required - String - The node that requested the discovery
 *		"data" - Required - Object - Payload data:
 *			{
 *				"user": "...",				// Required - String - The name of the user that owns the engine instance
 *				"machine": "...",			// Required - String - The name of the machine that owns the engine instance
 *				"engine_version": "...",	// Required - String - The version number of the engine
 *				"engine_root": "...",		// Required - String - The absolute on-disk root of the engine
 *				"project_root: "...",		// Optional - String - The absolute on-disk root of the project (if a project is loaded)
 *				"project_name": "..."		// Optional - String - The name of the project (if a project is loaded)
 *			}
 *
 *	"open_connection" (UDP) - Open a TCP command connection with the requested server
 *		"dest" - Required - String - The node that should open a connection to the server
 *		"data" - Required - Object - Payload data:
 *			{
 *				"command_ip": "...",		// Required - String  - The IP address of the TCP command server
 *				"command_port": 0			// Required - Integer - The port of the TCP command server
 *			}
 *
 *	"close_connection" (UDP) - Close any active TCP command connection
 *		"dest" - Required - String - The node that should close its connection to the server
 *
 *	"command" (TCP) - Execute a remote Python command
 *		"dest" - Required - String - The node that should execute the command
 *		"data" - Required - Object - Payload data:
 *			{
 *				"command": "...",			// Required - String  - The command to execute
 *				"unattended": True,			// Required - Boolean - Whether to run the command in "unattended" mode (suppressing some UI)
 *				"exec_mode": "..."			// Required - String  - The execution mode to use as a string value (see EPythonCommandExecutionMode, and LexToString, LexFromString)
 *			}
 *
 *	"command_result" (TCP) - Result of executing a remote Python command
 *		"dest" - Required - String - The node that should receive the result
 *		"data" - Required - Object - Payload data:
 *			{
 *				"success": True,			// Required - Boolean - Whether the command executed successfully (without raising an uncaught exception)
 *				"command": "...",			// Required - String  - The command that was executed
 *				"result": "...",			// Required - String  - The result of running the command
 *				"output": [{				// Required - List    - The log output from running the command
 *					"type: "...",				// Required - String - The type of output (Info, Warning, or Error)
 *					"output": "..."				// Required - String - The actual log output
 *					}]
 *			}
 */

DEFINE_LOG_CATEGORY_STATIC(LogPythonRemoteExecution, Log, All);

struct FPythonScriptRemoteExecutionMessage;

namespace PythonScriptRemoteExecutionUtil
{
	const int32 ProtocolVersion = 1;
	const FString ProtocolMagic = TEXT("ue_py");
	const FString TypePing = TEXT("ping");
	const FString TypePong = TEXT("pong");
	const FString TypeOpenConnection = TEXT("open_connection");
	const FString TypeCloseConnection = TEXT("close_connection");
	const FString TypeCommand = TEXT("command");
	const FString TypeCommandResult = TEXT("command_result");

	bool SerializeJsonObject(const TSharedRef<FJsonObject>& InJsonObject, FString& OutJsonString, FText* OutError = nullptr);
	bool DeserializeJsonObject(const FString& InJsonString, TSharedPtr<FJsonObject>& OutJsonObject, FText* OutError = nullptr);
	bool ParseMessageData(const char* InMessageData, FPythonScriptRemoteExecutionMessage& OutMessage);
	bool ParseMessageData(const TCHAR* InMessageData, FPythonScriptRemoteExecutionMessage& OutMessage);
	bool WriteMessageData(const FPythonScriptRemoteExecutionMessage& InMessage, FString& OutMessageData);
	void DestroySocket(ISocketSubsystem* InSocketSubsystem, FSocket*& InOutSocket);
}

struct FPythonScriptRemoteExecutionMessage
{
	FPythonScriptRemoteExecutionMessage() = default;

	FPythonScriptRemoteExecutionMessage(FString InType, FString InSource, FString InDest = FString(), TSharedPtr<FJsonValue> InData = nullptr)
		: Type(MoveTemp(InType))
		, Source(MoveTemp(InSource))
		, Dest(MoveTemp(InDest))
		, Data(MoveTemp(InData))
	{
	}

	bool PassesReceiveFilter(const FString& InNodeId) const
	{
		return !Source.Equals(InNodeId) && (Dest.IsEmpty() || Dest.Equals(InNodeId));
	}

	bool ToJsonObject(TSharedPtr<FJsonObject>& OutJsonObject, FText* OutError = nullptr) const
	{
		// Validate required fields
		if (Type.IsEmpty())
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Message.ToJson.Error.TypeCannotBeEmpty", "'type' cannot be empty!");
			}
			return false;
		}
		if (Source.IsEmpty())
		{
			if (OutError)
			{
				*OutError = LOCTEXT("Message.ToJson.Error.SourceCannotBeEmpty", "'source' cannot be empty!");
			}
			return false;
		}

		OutJsonObject = MakeShared<FJsonObject>();

		// Write required protocol version information
		OutJsonObject->SetNumberField(TEXT("version"), PythonScriptRemoteExecutionUtil::ProtocolVersion);
		OutJsonObject->SetStringField(TEXT("magic"), PythonScriptRemoteExecutionUtil::ProtocolMagic);

		// Write required fields
		OutJsonObject->SetStringField(TEXT("type"), Type);
		OutJsonObject->SetStringField(TEXT("source"), Source);

		// Write optional fields
		if (!Dest.IsEmpty())
		{
			OutJsonObject->SetStringField(TEXT("dest"), Dest);
		}
		if (Data.IsValid())
		{
			OutJsonObject->SetField(TEXT("data"), Data);
		}

		return true;
	}

	bool ToJsonString(FString& OutJsonString, FText* OutError = nullptr) const
	{
		TSharedPtr<FJsonObject> JsonObject;
		return ToJsonObject(JsonObject, OutError) && PythonScriptRemoteExecutionUtil::SerializeJsonObject(JsonObject.ToSharedRef(), OutJsonString, OutError);
	}

	bool FromJsonObject(const TSharedRef<FJsonObject>& InJsonObject, FText* OutError = nullptr)
	{
		// Read and validate required protocol version information
		{
			int32 LocalVersion = 0;
			if (!InJsonObject->TryGetNumberField(TEXT("version"), LocalVersion) || LocalVersion != PythonScriptRemoteExecutionUtil::ProtocolVersion)
			{
				if (OutError)
				{
					*OutError = FText::Format(LOCTEXT("Message.FromJson.Error.MissingOrInvalidVersion", "'version' is missing or incorrect (got {0}, expected {1})!"), LocalVersion, PythonScriptRemoteExecutionUtil::ProtocolVersion);
				}
				return false;
			}

			FString LocalMagic;
			if (!InJsonObject->TryGetStringField(TEXT("magic"), LocalMagic) || !LocalMagic.Equals(PythonScriptRemoteExecutionUtil::ProtocolMagic))
			{
				if (OutError)
				{
					*OutError = FText::Format(LOCTEXT("Message.FromJson.Error.MissingOrInvalidMagic", "'magic' is missing or incorrect (got '{0}', expected '{1}')!"), FText::AsCultureInvariant(LocalMagic), FText::AsCultureInvariant(PythonScriptRemoteExecutionUtil::ProtocolMagic));
				}
				return false;
			}
		}

		// Read required fields
		{
			FString LocalType;
			if (!InJsonObject->TryGetStringField(TEXT("type"), LocalType))
			{
				if (OutError)
				{
					*OutError = LOCTEXT("Message.FromJson.Error.MissingType", "'type' is missing!");
				}
				return false;
			}

			FString LocalSource;
			if (!InJsonObject->TryGetStringField(TEXT("source"), LocalSource))
			{
				if (OutError)
				{
					*OutError = LOCTEXT("Message.FromJson.Error.MissingSource", "'source' is missing!");
				}
				return false;
			}

			Type = MoveTemp(LocalType);
			Source = MoveTemp(LocalSource);
		}

		// Read optional fields
		InJsonObject->TryGetStringField(TEXT("dest"), Dest);
		Data = InJsonObject->TryGetField(TEXT("data"));

		return true;
	}

	bool FromJsonString(const FString& InJsonString, FText* OutError = nullptr)
	{
		TSharedPtr<FJsonObject> JsonObject;
		return PythonScriptRemoteExecutionUtil::DeserializeJsonObject(InJsonString, JsonObject, OutError) && FromJsonObject(JsonObject.ToSharedRef(), OutError);
	}

	/** Identifier for the message type */
	FString Type;

	/** Unique identifier of the node that sent the message (typically a GUID) */
	FString Source;

	/** Unique identifier of the node that should receive the message (typically a GUID) */
	FString Dest;

	/** Message specific payload data */
	TSharedPtr<FJsonValue> Data;
};

class FPythonScriptRemoteExecutionBroadcastConnection
{
public:
	FPythonScriptRemoteExecutionBroadcastConnection(FPythonScriptRemoteExecution* InRemoteExecution)
		: RemoteExecution(InRemoteExecution)
		, BroadcastSocket(nullptr)
	{
		check(RemoteExecution);

		NodeInfo = MakeShared<FJsonObject>();
		NodeInfo->SetStringField(TEXT("user"), FApp::GetSessionOwner());
		NodeInfo->SetStringField(TEXT("machine"), FPlatformProcess::ComputerName());
		NodeInfo->SetStringField(TEXT("engine_version"), FEngineVersion::Current().ToString());
		NodeInfo->SetStringField(TEXT("engine_root"), FPaths::ConvertRelativePathToFull(FPaths::EngineDir()));
		if (FApp::HasProjectName())
		{
			NodeInfo->SetStringField(TEXT("project_root"), FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()));
			NodeInfo->SetStringField(TEXT("project_name"), FApp::GetProjectName());
		}

		const UPythonScriptPluginSettings* PythonScriptPluginSettings = GetDefault<UPythonScriptPluginSettings>();

		bool bValidEndpointAddresses = true;

		FIPv4Endpoint MulticastGroupEndpoint(FIPv4Address::Any, 0);
		if (!FIPv4Endpoint::Parse(PythonScriptPluginSettings->RemoteExecutionMulticastGroupEndpoint, MulticastGroupEndpoint))
		{
			bValidEndpointAddresses = false;
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to parse multicast group endpoint '%ls'. Remote execution will be disabled!", *PythonScriptPluginSettings->RemoteExecutionMulticastGroupEndpoint);
		}

		FIPv4Address MulticastBindAddress = FIPv4Address::Any;
		if (!FIPv4Address::Parse(PythonScriptPluginSettings->RemoteExecutionMulticastBindAddress, MulticastBindAddress))
		{
			bValidEndpointAddresses = false;
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to parse multicast bind address '%ls'. Remote execution will be disabled!", *PythonScriptPluginSettings->RemoteExecutionMulticastBindAddress);
		}

		if (bValidEndpointAddresses)
		{
			BroadcastGroupAddr = MulticastGroupEndpoint.ToInternetAddr();

			BroadcastSocket = FUdpSocketBuilder(TEXT("PythonRemoteExecutionBroadcastSocket"))
				.AsReusable()
				.BoundToAddress(MulticastBindAddress)
				.BoundToPort(MulticastGroupEndpoint.Port)
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
				.JoinedToGroup(MulticastGroupEndpoint.Address, MulticastBindAddress)
				.WithMulticastLoopback()
#endif
				.WithSendBufferSize(PythonScriptPluginSettings->RemoteExecutionSendBufferSizeBytes)
				.WithReceiveBufferSize(PythonScriptPluginSettings->RemoteExecutionReceiveBufferSizeBytes)
				.WithMulticastTtl(PythonScriptPluginSettings->RemoteExecutionMulticastTtl)
				.WithMulticastInterface(MulticastBindAddress);
		}
	}

	~FPythonScriptRemoteExecutionBroadcastConnection()
	{
		PythonScriptRemoteExecutionUtil::DestroySocket(RemoteExecution->SocketSubsystem, BroadcastSocket);
	}

	bool HasConnection() const
	{
		return BroadcastSocket != nullptr;
	}

	void Tick(const float InDeltaTime)
	{
		if (BroadcastSocket)
		{
			uint32 PendingDataSize = 0;
			while (BroadcastSocket->HasPendingData(PendingDataSize) && PendingDataSize > 0)
			{
				TArray<uint8, TInlineAllocator<256>> ReceivedData;
				ReceivedData.SetNumZeroed(PendingDataSize + 1);

				int32 BytesRead = 0;
				if (BroadcastSocket->Recv(ReceivedData.GetData(), ReceivedData.Num(), BytesRead) && BytesRead > 0)
				{
					ReceivedData.Last() = 0; // Ensure null terminator

					FPythonScriptRemoteExecutionMessage Message;
					if (PythonScriptRemoteExecutionUtil::ParseMessageData((char*)ReceivedData.GetData(), Message))
					{
						HandleMessage(Message);
					}
				}
			}
		}
	}

private:
	void HandleMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		if (!InMessage.PassesReceiveFilter(RemoteExecution->NodeId))
		{
			return;
		}

		if (InMessage.Type.Equals(PythonScriptRemoteExecutionUtil::TypePing))
		{
			BroadcastPongMessage(InMessage.Source);
			return;
		}

		if (InMessage.Type.Equals(PythonScriptRemoteExecutionUtil::TypeOpenConnection))
		{
			HandleOpenConnectionMessage(InMessage);
			return;
		}

		if (InMessage.Type.Equals(PythonScriptRemoteExecutionUtil::TypeCloseConnection))
		{
			HandleCloseConnectionMessage(InMessage);
			return;
		}

		UE_LOGF(LogPythonRemoteExecution, VeryVerbose, "Unhandled remote execution message type '%ls'", *InMessage.Type);
	}

	void HandleOpenConnectionMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		const TSharedPtr<FJsonObject>* ConnectionData = nullptr;
		if (InMessage.Data && InMessage.Data->TryGetObject(ConnectionData) && *ConnectionData)
		{
			FIPv4Address CommandIp;
			{
				FString CommandIpString;
				if (!(*ConnectionData)->TryGetStringField(TEXT("command_ip"), CommandIpString))
				{
					UE_LOGF(LogPythonRemoteExecution, Error, "Failed to find required data field 'command_ip' (string) for the 'open_connection' message!");
					return;
				}
				if (!FIPv4Address::Parse(CommandIpString, CommandIp))
				{
					UE_LOGF(LogPythonRemoteExecution, Error, "Failed to parse command address '%ls'!", *CommandIpString);
					return;
				}
			}

			uint32 CommandPort = 0;
			if (!(*ConnectionData)->TryGetNumberField(TEXT("command_port"), CommandPort))
			{
				UE_LOGF(LogPythonRemoteExecution, Error, "Failed to find required data field 'command_port' (integer) for the 'open_connection' message!");
				return;
			}

			RemoteExecution->OpenCommandConnection(InMessage.Source, FIPv4Endpoint(CommandIp, (uint16)CommandPort));
		}
	}

	void HandleCloseConnectionMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		RemoteExecution->CloseCommandConnection(InMessage.Source);
	}

	void BroadcastMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		FString MessageStr;
		FText WriteErrorMsg;
		if (!InMessage.ToJsonString(MessageStr, &WriteErrorMsg))
		{
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to write remote execution message: %ls", *WriteErrorMsg.ToString());
			return;
		}
		BroadcastMessage(TCHAR_TO_UTF8(*MessageStr));
	}

	void BroadcastMessage(const char* InMessage)
	{
		check(BroadcastSocket);

		int32 BytesSent = 0;
		BroadcastSocket->SendTo((uint8*)InMessage, FCStringAnsi::Strlen(InMessage), BytesSent, *BroadcastGroupAddr);
	}

	void BroadcastPongMessage(const FString& InDestinationNodeId)
	{
		BroadcastMessage(FPythonScriptRemoteExecutionMessage(PythonScriptRemoteExecutionUtil::TypePong, RemoteExecution->NodeId, InDestinationNodeId, MakeShared<FJsonValueObject>(NodeInfo)));
	}

	/** Owner remote execution instance */
	FPythonScriptRemoteExecution* RemoteExecution;

	/** The information about this node instance (used in "pong" messages) */
	TSharedPtr<FJsonObject> NodeInfo;

	/** The address of the UDP multicast group used for broadcast */
	TSharedPtr<FInternetAddr> BroadcastGroupAddr;

	/** The UDP multicast socket used for broadcast */
	FSocket* BroadcastSocket;
};

class FPythonScriptRemoteExecutionCommandConnection
{
public:
	FPythonScriptRemoteExecutionCommandConnection(FPythonScriptRemoteExecution* InRemoteExecution, const FString& InRemoteNodeId, const FIPv4Endpoint& InCommandEndpoint)
		: RemoteExecution(InRemoteExecution)
		, RemoteNodeId(InRemoteNodeId)
		, CommandEndpoint(InCommandEndpoint)
		, CommandChannelSocket(nullptr)
	{
		check(RemoteExecution);

		const UPythonScriptPluginSettings* PythonScriptPluginSettings = GetDefault<UPythonScriptPluginSettings>();

		CommandChannelSocket = FTcpSocketBuilder(TEXT("PythonRemoteExecutionCommandSocket"))
			.WithSendBufferSize(PythonScriptPluginSettings->RemoteExecutionSendBufferSizeBytes)
			.WithReceiveBufferSize(PythonScriptPluginSettings->RemoteExecutionReceiveBufferSizeBytes);

		if (!CommandChannelSocket->Connect(*CommandEndpoint.ToInternetAddr()))
		{
			PythonScriptRemoteExecutionUtil::DestroySocket(RemoteExecution->SocketSubsystem, CommandChannelSocket);
		}
	}

	~FPythonScriptRemoteExecutionCommandConnection()
	{
		PythonScriptRemoteExecutionUtil::DestroySocket(RemoteExecution->SocketSubsystem, CommandChannelSocket);
	}

	bool HasConnection() const
	{
		return CommandChannelSocket != nullptr;
	}

	bool IsConnectedTo(const FString& InRemoteNodeId)
	{
		return RemoteNodeId.Equals(InRemoteNodeId);
	}

	bool IsConnectedTo(const FIPv4Endpoint& InCommandEndpoint)
	{
		return CommandEndpoint == InCommandEndpoint;
	}

	bool IsConnectedTo(const FString& InRemoteNodeId, const FIPv4Endpoint& InCommandEndpoint)
	{
		return IsConnectedTo(InRemoteNodeId)
			&& IsConnectedTo(InCommandEndpoint);
	}

	void Tick(const float InDeltaTime)
	{
		if (CommandChannelSocket)
		{
			uint32 PendingDataSize = 0;
			while (CommandChannelSocket->HasPendingData(PendingDataSize) && PendingDataSize > 0)
			{
				TArray<uint8, TInlineAllocator<256>> ReceivedData;
				ReceivedData.SetNumZeroed(PendingDataSize + 1);

				int32 BytesRead = 0;
				if (CommandChannelSocket->Recv(ReceivedData.GetData(), ReceivedData.Num(), BytesRead) && BytesRead > 0)
				{
					ReceivedData.Last() = 0; // Ensure null terminator

					FPythonScriptRemoteExecutionMessage Message;
					if (PythonScriptRemoteExecutionUtil::ParseMessageData((char*)ReceivedData.GetData(), Message))
					{
						HandleMessage(Message);
					}
				}
			}
		}
	}

private:
	void HandleMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		if (!InMessage.PassesReceiveFilter(RemoteExecution->NodeId))
		{
			return;
		}

		if (InMessage.Type.Equals(PythonScriptRemoteExecutionUtil::TypeCommand))
		{
			HandleCommandMessage(InMessage);
			return;
		}

		UE_LOGF(LogPythonRemoteExecution, VeryVerbose, "Unhandled remote execution message type '%ls'", *InMessage.Type);
	}

	void HandleCommandMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		const TSharedPtr<FJsonObject>* CommandData = nullptr;
		if (InMessage.Data && InMessage.Data->TryGetObject(CommandData) && *CommandData)
		{
			FPythonCommandEx PythonCommand;

			if (!(*CommandData)->TryGetStringField(TEXT("command"), PythonCommand.Command))
			{
				UE_LOGF(LogPythonRemoteExecution, Error, "Failed to find required data field 'command' (string) for the 'command' message!");
				return;
			}

			{
				bool bUnattended = false;
				if (!(*CommandData)->TryGetBoolField(TEXT("unattended"), bUnattended))
				{
					UE_LOGF(LogPythonRemoteExecution, Error, "Failed to find required data field 'unattended' (bool) for the 'command' message!");
					return;
				}
				PythonCommand.Flags |= bUnattended ? EPythonCommandFlags::Unattended : EPythonCommandFlags::None;
			}

			{
				FString ExecModeString;
				if (!(*CommandData)->TryGetStringField(TEXT("exec_mode"), ExecModeString))
				{
					UE_LOGF(LogPythonRemoteExecution, Error, "Failed to find required data field 'exec_mode' (string) for the 'command' message!");
					return;
				}
				if (!LexTryParseString(PythonCommand.ExecutionMode, *ExecModeString))
				{
					UE_LOGF(LogPythonRemoteExecution, Error, "Failed to find parse execution mode '%ls'!", *ExecModeString);
					return;
				}
			}

			{
				const bool bCommandSuccess = RemoteExecution->PythonScriptPlugin->ExecPythonCommandEx(PythonCommand);
				SendCommandResultMessage(InMessage.Source, bCommandSuccess, PythonCommand.Command, PythonCommand.CommandResult, PythonCommand.LogOutput);
			}
		}
	}

	void SendMessage(const FPythonScriptRemoteExecutionMessage& InMessage)
	{
		FString MessageStr;
		FText WriteErrorMsg;
		if (!InMessage.ToJsonString(MessageStr, &WriteErrorMsg))
		{
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to write remote execution message: %ls", *WriteErrorMsg.ToString());
			return;
		}
		SendMessage(TCHAR_TO_UTF8(*MessageStr));
	}

	void SendMessage(const char* InMessage)
	{
		check(CommandChannelSocket);

		int32 BytesSent = 0;
		CommandChannelSocket->Send((uint8*)InMessage, FCStringAnsi::Strlen(InMessage), BytesSent);
	}

	void SendCommandResultMessage(const FString& InDestinationNodeId, const bool bSuccess, const FString& InCommand, const FString& InResult, const TArray<FPythonLogOutputEntry>& InLogOutput)
	{
		TArray<TSharedPtr<FJsonValue>> JsonLogOutput;
		for (const FPythonLogOutputEntry& LogEntry : InLogOutput)
		{
			TSharedRef<FJsonObject> JsonLogEntry = MakeShared<FJsonObject>();
			JsonLogEntry->SetStringField(TEXT("type"), LexToString(LogEntry.Type));
			JsonLogEntry->SetStringField(TEXT("output"), LogEntry.Output);
			JsonLogOutput.Add(MakeShared<FJsonValueObject>(JsonLogEntry));
		}

		TSharedRef<FJsonObject> ResultInfo = MakeShared<FJsonObject>();
		ResultInfo->SetBoolField(TEXT("success"), bSuccess);
		ResultInfo->SetStringField(TEXT("command"), InCommand);
		ResultInfo->SetStringField(TEXT("result"), InResult);
		ResultInfo->SetArrayField(TEXT("output"), JsonLogOutput);

		SendMessage(FPythonScriptRemoteExecutionMessage(PythonScriptRemoteExecutionUtil::TypeCommandResult, RemoteExecution->NodeId, InDestinationNodeId, MakeShared<FJsonValueObject>(ResultInfo)));
	}

	/** Owner remote execution instance */
	FPythonScriptRemoteExecution* RemoteExecution;

	/** The ID of the remote node that we connected to */
	FString RemoteNodeId;

	/** Endpoint that this command channel should be connected to */
	FIPv4Endpoint CommandEndpoint;

	/** The TCP socket used to handle command requests */
	FSocket* CommandChannelSocket;
};

FPythonScriptRemoteExecution::FPythonScriptRemoteExecution(IPythonScriptPlugin* InPythonScriptPlugin)
	: PythonScriptPlugin(InPythonScriptPlugin)
	, SocketSubsystem(ISocketSubsystem::Get())
	, NodeId(FApp::GetInstanceId().ToString())
{
	check(PythonScriptPlugin);
	check(SocketSubsystem);

	if (GetDefault<UPythonScriptPluginSettings>()->bRemoteExecution)
	{
		Start();
	}
}

FPythonScriptRemoteExecution::~FPythonScriptRemoteExecution()
{
	Stop();
}

bool FPythonScriptRemoteExecution::Start()
{
	Stop();

	BroadcastConnection = MakeUnique<FPythonScriptRemoteExecutionBroadcastConnection>(this);
	if (!BroadcastConnection->HasConnection())
	{
		UE_LOGF(LogPythonRemoteExecution, Error, "Failed to open remote execution broadcast channel! Check your settings and the output log for more information.");
		BroadcastConnection.Reset();
	}

	return BroadcastConnection.IsValid();
}

void FPythonScriptRemoteExecution::Stop()
{
	CommandConnection.Reset();
	BroadcastConnection.Reset();
}

void FPythonScriptRemoteExecution::SyncToSettings()
{
	Stop();

	if (GetDefault<UPythonScriptPluginSettings>()->bRemoteExecution)
	{
		Start();
	}
}

void FPythonScriptRemoteExecution::Tick(const float InDeltaTime)
{
	if (CommandConnection)
	{
		CommandConnection->Tick(InDeltaTime);
	}

	if (BroadcastConnection)
	{
		BroadcastConnection->Tick(InDeltaTime);
	}
}

void FPythonScriptRemoteExecution::OpenCommandConnection(const FString& InRemoteNodeId, const FIPv4Endpoint& InCommandEndpoint)
{
	if (!CommandConnection || !CommandConnection->IsConnectedTo(InRemoteNodeId, InCommandEndpoint))
	{
		CommandConnection = MakeUnique<FPythonScriptRemoteExecutionCommandConnection>(this, InRemoteNodeId, InCommandEndpoint);
		if (!CommandConnection->HasConnection())
		{
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to open remote execution command channel to '%ls' (%ls)!", *InRemoteNodeId, *InCommandEndpoint.ToString());
			CommandConnection.Reset();
		}
	}
}

void FPythonScriptRemoteExecution::CloseCommandConnection(const FString& InRemoteNodeId)
{
	if (CommandConnection && CommandConnection->IsConnectedTo(InRemoteNodeId))
	{
		CommandConnection.Reset();
	}
}

namespace PythonScriptRemoteExecutionUtil
{
	bool SerializeJsonObject(const TSharedRef<FJsonObject>& InJsonObject, FString& OutJsonString, FText* OutError)
	{
		TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&OutJsonString);
		const bool bDidSerialize = FJsonSerializer::Serialize(InJsonObject, JsonWriter);
		JsonWriter->Close();

		if (!bDidSerialize)
		{
			if (OutError)
			{
				*OutError = LOCTEXT("SerializeJsonObject.Error.FailedToSerialize", "Failed to serialize JSON object!");
			}
			return false;
		}

		return true;
	}

	bool DeserializeJsonObject(const FString& InJsonString, TSharedPtr<FJsonObject>& OutJsonObject, FText* OutError)
	{
		TSharedRef<TJsonReader<TCHAR>> JsonWriter = TJsonReaderFactory<TCHAR>::Create(InJsonString);
		if (!FJsonSerializer::Deserialize(JsonWriter, OutJsonObject))
		{
			if (OutError)
			{
				*OutError = FText::Format(LOCTEXT("DeserializeJsonObject.Error.FailedToDeserialize", "Failed to deserialize JSON object! {0}"), FText::AsCultureInvariant(JsonWriter->GetErrorMessage()));
			}
			return false;
		}

		return true;
	}

	bool ParseMessageData(const char* InMessageData, FPythonScriptRemoteExecutionMessage& OutMessage)
	{
		return ParseMessageData(UTF8_TO_TCHAR(InMessageData), OutMessage);
	}

	bool ParseMessageData(const TCHAR* InMessageData, FPythonScriptRemoteExecutionMessage& OutMessage)
	{
		FText ReadErrorMsg;
		if (!OutMessage.FromJsonString(InMessageData, &ReadErrorMsg))
		{
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to read remote execution message '%ls': %ls", InMessageData, *ReadErrorMsg.ToString());
			return false;
		}
		return true;
	}

	bool WriteMessageData(const FPythonScriptRemoteExecutionMessage& InMessage, FString& OutMessageData)
	{
		FText WriteErrorMsg;
		if (!InMessage.ToJsonString(OutMessageData, &WriteErrorMsg))
		{
			UE_LOGF(LogPythonRemoteExecution, Error, "Failed to write remote execution message: %ls", *WriteErrorMsg.ToString());
			return false;
		}
		return true;
	}

	void DestroySocket(ISocketSubsystem* InSocketSubsystem, FSocket*& InOutSocket)
	{
		check(InSocketSubsystem);

		if (InOutSocket)
		{
			if (InOutSocket->GetConnectionState() == SCS_Connected)
			{
				InOutSocket->Close();
			}

			InSocketSubsystem->DestroySocket(InOutSocket);
			InOutSocket = nullptr;
		}
	}
}

#endif	// WITH_PYTHON

#undef LOCTEXT_NAMESPACE
