// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IncludePython.h"
#include "Templates/UniquePtr.h"

struct FIPv4Endpoint;

class ISocketSubsystem;
class IPythonScriptPlugin;
class FPythonScriptRemoteExecutionCommandConnection;
class FPythonScriptRemoteExecutionBroadcastConnection;

#if WITH_PYTHON

class FPythonScriptRemoteExecution
{
	friend class FPythonScriptRemoteExecutionCommandConnection;
	friend class FPythonScriptRemoteExecutionBroadcastConnection;

public:
	explicit FPythonScriptRemoteExecution(IPythonScriptPlugin* InPythonScriptPlugin);
	~FPythonScriptRemoteExecution();

	/** Start remote execution based on the current settings (also called during construction if remote execution should be active) */
	bool Start();

	/** Stop remote execution (also called during destruction if remote execution is active) */
	void Stop();

	/** Sync the remote execution environment to the current settings, starting or stopping it as required */
	void SyncToSettings();

	/** Tick, processing and sending messages as required */
	void Tick(const float InDeltaTime);

private:
	/** Open a command connection to the given remote node, at the given endpoint */
	void OpenCommandConnection(const FString& InRemoteNodeId, const FIPv4Endpoint& InCommandEndpoint);

	/** Close any existing command connection to the given remote node */
	void CloseCommandConnection(const FString& InRemoteNodeId);

	/** The Python plugin that owns this instance */
	IPythonScriptPlugin* PythonScriptPlugin;

	/** Cached socket subsystem pointer */
	ISocketSubsystem* SocketSubsystem;

	/** The ID of this remote execution node */
	FString NodeId;

	/** Connection handling TCP commands */
	TUniquePtr<FPythonScriptRemoteExecutionCommandConnection> CommandConnection;

	/** Connection handling UDP broadcast */
	TUniquePtr<FPythonScriptRemoteExecutionBroadcastConnection> BroadcastConnection;
};

#endif	// WITH_PYTHON
