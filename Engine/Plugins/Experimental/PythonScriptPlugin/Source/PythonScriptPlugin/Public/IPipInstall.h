// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

/**
 * Interface for wrapping a notifier that can return info from pip install
 */
struct ICmdProgressNotifier
{
	virtual void UpdateProgress(float UpdateWorkDone, float UpdateTotalWork, const FText& Status) = 0;
	virtual void Completed(bool bSuccess) = 0;

	virtual ~ICmdProgressNotifier()= default;
};

/**
 * Interface to pip installer used by python script plugin for installing plugin python dependencies
 */
class IPipInstall
{
public:
	static PYTHONSCRIPTPLUGIN_API IPipInstall& Get();
	
	// Initialize the internal pip virtual env and check enabled plugins for python dependencies
	virtual bool InitPipInstall() = 0;
	// Run pip to install all missing python dependencies for enabled plugins
	virtual bool LaunchPipInstall(bool RunAsync, TSharedPtr<ICmdProgressNotifier> CmdProgressNotifier = nullptr, FSimpleDelegate OnCompleted = FSimpleDelegate()) = 0;
	// Check if a background install is running
	virtual bool IsInstalling() = 0;
	
	// Get number of missing python packages to install
	virtual int32 GetNumPackagesToInstall() = 0;
	// Get list of python packages to install
	virtual bool GetPackageInstallList(TArray<FString>& PyPackages) = 0;

	// Register the site-packages path with embedded python env
	virtual bool RegisterPipSitePackagesPath() const = 0;
	
	virtual ~IPipInstall()= default;
};
