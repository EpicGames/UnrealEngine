// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "PythonScriptTypes.h"
#include "Misc/SourceLocation.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

class IPythonScriptPlugin : public IModuleInterface
{
public:
	/** Get this module */
	static IPythonScriptPlugin* Get()
	{
		static const FName ModuleName = "PythonScriptPlugin";
		return FModuleManager::GetModulePtr<IPythonScriptPlugin>(ModuleName);
	}

	/**
	 * Check to see whether the plugin has Python support enabled.
	 * @note This may return false until IsPythonConfigured is true.
	 */
	virtual bool IsPythonAvailable() const = 0;

	/**
	 * Check to see whether Python has been configured.
	 * @note Python may be configured but not yet be initialized (@see IsPythonInitialized).
	 */
	virtual bool IsPythonConfigured() const = 0;

	/**
	 * Check to see whether Python has been initialized and is ready to use.
	 */
	virtual bool IsPythonInitialized() const = 0;

	/**
	 * Force Python to be enabled and initialized, regardless of the settings that control its default enabled state.
	 * @return False if Python can't be enabled ever. Use IsPythonInitialized to verify that it actually initialized.
	 */
	virtual bool ForceEnablePythonAtRuntime(UE::FSourceLocation Location = UE::FSourceLocation::Current()) = 0;

	/**
	 * Force Python to shutdown.
	 */
	virtual void ForceDisablePythonAtRuntime(UE::FSourceLocation Location = UE::FSourceLocation::Current()) = 0;

	/**
	 * Execute the given Python command.
	 * This may be literal Python code, or a file (with optional arguments) that you want to run.
	 * @return true if the command ran successfully, false if there were errors (the output log will show the errors).
	 */
	virtual bool ExecPythonCommand(const TCHAR* InPythonCommand) = 0;

	/**
	 * Execute the given Python command.
	 * @return true if the command ran successfully, false if there were errors.
	 */
	virtual bool ExecPythonCommandEx(FPythonCommandEx& InOutPythonCommand) = 0;
	
	/**
	 * Get the path to the Python interpreter executable of the Python SDK this plugin was compiled against.
	 */
	virtual FString GetInterpreterExecutablePath() const = 0;

	/**
	 * Delegate called after Python has been configured.
	 */
	virtual FSimpleMulticastDelegate& OnPythonConfigured() = 0;

	/**
	 * Delegate called after Python has been initialized.
	 */
	virtual FSimpleMulticastDelegate& OnPythonInitialized() = 0;

	/**
	 * Delegate called before Python is shutdown.
	 */
	virtual FSimpleMulticastDelegate& OnPythonShutdown() = 0;

	/**
	 * Wrapper around OnPythonConfigured that will either register the callback, or call it immediately if IsPythonConfigured is already true.
	 */
	void RegisterOnPythonConfigured(FSimpleDelegate&& Callback)
	{
		if (IsPythonConfigured())
		{
			Callback.ExecuteIfBound();
		}
		else
		{
			OnPythonConfigured().Add(MoveTemp(Callback));
		}
	}

	/**
	 * Wrapper around OnPythonInitialized that will either register the callback, or call it immediately if IsPythonInitialized is already true.
	 */
	void RegisterOnPythonInitialized(FSimpleDelegate&& Callback)
	{
		if (IsPythonInitialized())
		{
			Callback.ExecuteIfBound();
		}
		else
		{
			OnPythonInitialized().Add(MoveTemp(Callback));
		}
	}

	/**
	* Add allowed paths for startup script filtering in restrictive mode.
	* @param Owner Owner name for cleanup
	* @param Path The path prefix to allow
	*/
	virtual void AddStartupScriptAllowPath(FName Owner, const FString& Path) = 0;

	/**
	 * Remove all startup script allow paths registered by the given owner.
	 * @param Owner - Owner name whose entries should be removed
	 */
	virtual void RemoveStartupScriptAllowPath(FName Owner) = 0;
};
