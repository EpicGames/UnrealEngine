// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPipInstall.h"

#define UE_API PYTHONSCRIPTPLUGIN_API

// PipInstall dialog responses
enum EPipInstallDialogResult
{
	BackgroundInstall,
	Finished,
	Canceled,
	Error,
};

/**
 * Simplified UI helper for creating a modal dialog and running (background) installs
 */
class FPipInstallHelper
{
public:
	// Check if pip install is required
	static UE_API int32 GetNumPackagesToInstall();
	
	// Show notification that pip install is required
	static UE_API EPipInstallDialogResult ShowPipInstallDialog(bool bAllowBackgroundInstall = true, FSimpleDelegate OnCompleted = FSimpleDelegate());

	// Run a headless pip install (for commandlets/build machines)
	static UE_API bool LaunchHeadlessPipInstall();
};

/**
 * Helpers for handling standard delegate pipeline pattern for python/pip install
 */
class FPythonScriptInitHelper
{
public:
	static UE_API void InitPython(FSimpleDelegate OnInitialized, FSimpleDelegate OnPythonUnavailable = FSimpleDelegate());
	static UE_API void InitPythonAndPipInstall(FSimpleDelegate OnPipInstalled, FSimpleDelegate OnPythonUnavailable = FSimpleDelegate());
};

#undef UE_API
