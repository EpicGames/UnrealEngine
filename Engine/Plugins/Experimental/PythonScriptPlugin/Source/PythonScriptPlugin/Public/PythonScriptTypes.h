// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "PythonScriptTypes.generated.h"

// Note: The types in this header are used by IPythonScriptPlugin, so MUST be inline as they CANNOT require linkage to use

/** Types of log output that Python can give. */
UENUM()
enum class EPythonLogOutputType : uint8
{
	/** This log was informative. */
	Info,
	/** This log was a warning. */
	Warning,
	/** This log was an error. */
	Error,
};

/** Flags that can be specified when running Python commands. */
UENUM()
enum class EPythonCommandFlags : uint8
{
	/** No special behavior. */
	None = 0,
	/** Run the Python command in "unattended" mode (GIsRunningUnattendedScript set to true), which will suppress certain pieces of UI. */
	Unattended = 1<<0,
};
ENUM_CLASS_FLAGS(EPythonCommandFlags);

/** Controls the execution mode used for the Python command. */
UENUM()
enum class EPythonCommandExecutionMode : uint8
{
	/** Execute the Python command as a file. This allows you to execute either a literal Python script containing multiple statements, or a file with optional arguments. */
	ExecuteFile,
	/** Execute the Python command as a single statement. This will execute a single statement and print the result. This mode cannot run files. */
	ExecuteStatement,
	/** Evaluate the Python command as a single statement. This will evaluate a single statement and return the result. This mode cannot run files. */
	EvaluateStatement,
};

/** Controls the scope used when executing Python files. */
UENUM()
enum class EPythonFileExecutionScope : uint8
{
	/** Execute the Python file with its own unique locals and globals dict to isolate any changes it makes to the environment (like imports). */
	Private,
	/** Execute the Python file with the shared locals and globals dict as used by the console, so that executing it behaves as if you'd ran the file contents directly in the console. */
	Public,
};

/** Log output entry captured from Python. */
USTRUCT(BlueprintType)
struct FPythonLogOutputEntry
{
	GENERATED_BODY()

	/** The type of the log output. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Python|Output")
	EPythonLogOutputType Type = EPythonLogOutputType::Info;

	/** The log output string. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Python|Output")
	FString Output;
};

/** Extended information when executing Python commands. */
struct FPythonCommandEx
{
	/** Flags controlling how the command should be run. */
	EPythonCommandFlags Flags = EPythonCommandFlags::None;

	/** Controls the mode used to execute the command. */
	EPythonCommandExecutionMode ExecutionMode = EPythonCommandExecutionMode::ExecuteFile;

	/** Controls the scope used when executing Python files. */
	EPythonFileExecutionScope FileExecutionScope = EPythonFileExecutionScope::Private;

	/** The command to run. This may be literal Python code, or a file (with optional arguments) to run. */
	FString Command;

	/** The result of running the command. On success, for EvaluateStatement mode this will be the actual result of running the command, and will be None in all other cases. On failure, this will be the error information (typically a Python exception trace). */
	FString CommandResult;

	/** The log output captured while running the command. */
	TArray<FPythonLogOutputEntry> LogOutput;
};

inline const TCHAR* LexToString(EPythonLogOutputType InType)
{
	switch (InType)
	{
	case EPythonLogOutputType::Info:
		return TEXT("Info");
	case EPythonLogOutputType::Warning:
		return TEXT("Warning");
	case EPythonLogOutputType::Error:
		return TEXT("Error");
	default:
		break;
	}
	return TEXT("<Unknown EPythonLogOutputType>");
}

inline const TCHAR* LexToString(EPythonCommandExecutionMode InMode)
{
	switch (InMode)
	{
	case EPythonCommandExecutionMode::ExecuteFile:
		return TEXT("ExecuteFile");
	case EPythonCommandExecutionMode::ExecuteStatement:
		return TEXT("ExecuteStatement");
	case EPythonCommandExecutionMode::EvaluateStatement:
		return TEXT("EvaluateStatement");
	default:
		break;
	}
	return TEXT("<Unknown EPythonCommandExecutionMode>");
}

inline bool LexTryParseString(EPythonCommandExecutionMode& OutMode, const TCHAR* InBuffer)
{
	if (FCString::Stricmp(InBuffer, TEXT("ExecuteFile")) == 0)
	{
		OutMode = EPythonCommandExecutionMode::ExecuteFile;
		return true;
	}
	if (FCString::Stricmp(InBuffer, TEXT("ExecuteStatement")) == 0)
	{
		OutMode = EPythonCommandExecutionMode::ExecuteStatement;
		return true;
	}
	if (FCString::Stricmp(InBuffer, TEXT("EvaluateStatement")) == 0)
	{
		OutMode = EPythonCommandExecutionMode::EvaluateStatement;
		return true;
	}
	return false;
}

inline void LexFromString(EPythonCommandExecutionMode& OutMode, const TCHAR* InBuffer)
{
	OutMode = EPythonCommandExecutionMode::ExecuteFile;
	LexTryParseString(OutMode, InBuffer);
}
