// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class PythonScriptPlugin : ModuleRules
	{
		public PythonScriptPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Analytics",
					"AssetRegistry",
					"Projects",
					"Python3",
					"Slate",
					"SlateCore",
					"InputCore",
					"Sockets",
					"Networking",
					"Json",
					"DeveloperSettings"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PublicDependencyModuleNames.AddRange(
					new string[] {
						"ToolMenus"
					}
				);

				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"DesktopPlatform",
						"LevelEditor",
						"EditorFramework",
						"UnrealEd",
						"EditorSubsystem",
						"BlueprintGraph",
						"KismetCompiler",
						"AssetTools",
						"ContentBrowserData",
						"ContentBrowserFileDataSource",
						"SourceControl",
					}
				);
			}
		}
	}
}
