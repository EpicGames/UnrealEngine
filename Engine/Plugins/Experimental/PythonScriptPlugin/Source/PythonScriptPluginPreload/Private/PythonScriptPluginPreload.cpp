// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#if WITH_PYTHON
THIRD_PARTY_INCLUDES_START
// We include this file to get PY_MAJOR_VERSION
// We don't include Python.h as that will trigger a link dependency which we don't want
// as this module exists to pre-load the Python DLLs, so can't link to Python itself
#include "patchlevel.h"
THIRD_PARTY_INCLUDES_END
#endif	// WITH_PYTHON

class FPythonScriptPluginPreload : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
#if WITH_PYTHON
		LoadPythonLibraries();
#endif	// WITH_PYTHON
	}

	virtual void ShutdownModule() override
	{
#if WITH_PYTHON
		UnloadPythonLibraries();
#endif	// WITH_PYTHON
	}

private:
#if WITH_PYTHON
	void LoadSharedDSO(const FString& PythonDSOWildcard, const FString& PythonDir)
	{
		auto FindPythonDSOs = [&PythonDSOWildcard](const FString& InPath)
		{
			TArray<FString> PythonDSONames;
			IFileManager::Get().FindFiles(PythonDSONames, *(InPath / PythonDSOWildcard), true, false);
			for (FString& PythonDSOName : PythonDSONames)
			{
				PythonDSOName = InPath / PythonDSOName;
				FPaths::NormalizeFilename(PythonDSOName);
			}
			return PythonDSONames;
		};

		TArray<FString> PythonDSOPaths = FindPythonDSOs(PythonDir);
#if PLATFORM_WINDOWS
		if (PythonDSOPaths.Num() == 0)
		{
			// If we didn't find anything, check the Windows directory as the DLLs can sometimes be installed there
			FString WinDir = FPlatformMisc::GetEnvironmentVariable(TEXT("WINDIR"));
			if (!WinDir.IsEmpty())
			{
				PythonDSOPaths = FindPythonDSOs(WinDir / TEXT("System32"));
			}
		}
#endif

		for (const FString& PythonDSOPath : PythonDSOPaths)
		{
			void* DLLHandle = FPlatformProcess::GetDllHandle(*PythonDSOPath);
			check(DLLHandle != nullptr);
			DLLHandles.Add(DLLHandle);
		}
	}

	void LoadPythonLibraries()
	{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
		// Load the DSOs
		{
			// Build the full Python directory (UE_PYTHON_DIR may be relative to the engine directory for portability)
			FString PythonDir = UTF8_TO_TCHAR(UE_PYTHON_DIR);
			PythonDir.ReplaceInline(TEXT("{ENGINE_DIR}"), *FPaths::EngineDir(), ESearchCase::CaseSensitive);
			FPaths::NormalizeDirectoryName(PythonDir);
			FPaths::RemoveDuplicateSlashes(PythonDir);

#if PLATFORM_WINDOWS
			const FString PythonDSOWildcard = FString::Printf(TEXT("python%d*.dll"), PY_MAJOR_VERSION);
#elif PLATFORM_LINUX
			const FString PythonDSOWildcard = FString::Printf(TEXT("libpython%d*.so.1.0"), PY_MAJOR_VERSION);
			PythonDir /= TEXT("lib");
#endif
			LoadSharedDSO(PythonDSOWildcard, PythonDir);
		}
#endif	// PLATFORM_WINDOWS || PLATFORM_LINUX
	}

	void UnloadPythonLibraries()
	{
		for (void* DLLHandle : DLLHandles)
		{
			FPlatformProcess::FreeDllHandle(DLLHandle);
		}
		DLLHandles.Reset();
	}

	TArray<void*> DLLHandles;
#endif	// WITH_PYTHON
};

IMPLEMENT_MODULE(FPythonScriptPluginPreload, PythonScriptPluginPreload)
