// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class PythonScriptPluginPreload : ModuleRules
	{
		public PythonScriptPluginPreload(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateIncludePathModuleNames.AddRange(
					new string[] {
						"Python3",
					}
				);
			}
			else
			{
				PrivateDefinitions.Add("WITH_PYTHON=0");
			}
		}
	}
}
